{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Replication System Test Plugin",
	"Description" : "Unit and functional tests for the network replication system.",
	"Category" : "Networking",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "ReplicationSystemTestPlugin",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default"
		}
	]
}