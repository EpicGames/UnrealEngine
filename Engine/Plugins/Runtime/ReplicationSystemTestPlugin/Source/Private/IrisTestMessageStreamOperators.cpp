// Copyright Epic Games, Inc. All Rights Reserved.

#include "IrisTestMessageStreamOperators.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"

namespace UE::Net
{

FTestMessage& operator<<(FTestMessage& Message, ENetFilterStatus Status)
{
	return Message << (Status == ENetFilterStatus::Disallow ? TEXT("ENetFilterStatus::Disallow") : TEXT("ENetFilterStatus::Allow"));
}

}
