// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TestMessage.h"

namespace UE::Net
{
	enum class ENetFilterStatus : uint32;
}


namespace UE::Net
{

REPLICATIONSYSTEMTESTPLUGIN_API FTestMessage& operator<<(FTestMessage& Message, ENetFilterStatus);

}
