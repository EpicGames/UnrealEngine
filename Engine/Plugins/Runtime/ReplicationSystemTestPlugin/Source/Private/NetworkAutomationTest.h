// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PreprocessorHelpers.h"
#include "Misc/AutomationTest.h"

#if UE_NET_WITH_LOW_LEVEL_TESTS
#include "TestHarness.h"
#endif

namespace UE::Net
{

struct FNetworkAutomationTestStats
{
	SIZE_T FailureCount;
	SIZE_T WarningCount;
};

struct FNetworkAutomationTestConfig
{
	static void SetVerboseLogging(bool bVerbose) { bVerboseLogging = bVerbose; }
	static bool GetVerboseLogging() { return bVerboseLogging; }

private:
	static bool bVerboseLogging;
};

class FNetworkAutomationTestSuiteFixture
{
public:
	virtual ~FNetworkAutomationTestSuiteFixture();

	virtual const TCHAR* GetName() const = 0;

	void RunTest();

protected:
	FNetworkAutomationTestSuiteFixture();

	// Run just before test case
	virtual void SetUp();

	// Run just after test case
	virtual void TearDown();

	virtual void RunTestImpl() = 0;

	// Stats
	void SetSuppressWarningsFromSummary(bool bSuppress);

	void AddTestFailure();
	void AddTestWarning();

private:
	void PreSetUp();
	void PostTearDown();

	template<class Fixture, const TCHAR* Name>
	friend class TCatchFixtureWrapper;

	FNetworkAutomationTestStats Stats = {};
	bool bSuppressWarningsFromSummary;
};

#if WITH_AUTOMATION_WORKER

class FNetworkAutomationTestWrapper : private FAutomationTestBase
{
public:
	FNetworkAutomationTestWrapper(FNetworkAutomationTestSuiteFixture& TestSuite, const TCHAR* Name);

private:
	virtual EAutomationTestFlags GetTestFlags() const override;
	virtual uint32 GetRequiredDeviceNum() const override;

	virtual FString GetBeautifiedTestName() const override;

	virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const override;
	virtual bool RunTest(const FString&) override;

private:
	FNetworkAutomationTestSuiteFixture& TestSuite;
};

#endif

}

#if UE_NET_WITH_LOW_LEVEL_TESTS

namespace UE::Net
{

template<class Fixture, const TCHAR* Name>
class TCatchFixtureWrapper : public Fixture
{
public:
	TCatchFixtureWrapper() : Fixture()
	{
		this->PreSetUp();
		Fixture::SetUp();
	}

	~TCatchFixtureWrapper()
	{
		Fixture::TearDown();
		this->PostTearDown();
	}

	virtual const TCHAR* GetName() const override { return Name; }
	virtual void RunTestImpl() override {} // Unused in Catch-based tests
};

}

#define UE_NET_TEST_CATCH_INTERNAL(TestSuite, TestCase, BaseClass) \
	constexpr TCHAR TestSuite ## _ ## TestCase ## Name[] = TEXT("Net." #TestSuite "." #TestCase); \
	INTERNAL_CATCH_TEST_CASE_METHOD2(Catch::DefaultGroup, TestSuite ## _ ## TestCase, (TCatchFixtureWrapper<BaseClass, TestSuite ## _ ## TestCase ## Name>), UE_STRINGIZE(TestSuite ## _ ## TestCase))

#define UE_NET_TEST_FIXTURE(TestFixture, TestCase) UE_NET_TEST_CATCH_INTERNAL(TestFixture, TestCase, TestFixture)
#define UE_NET_TEST(TestSuite, TestCase) UE_NET_TEST_CATCH_INTERNAL(TestSuite, TestCase, UE::Net::FNetworkAutomationTestSuiteFixture)




#elif WITH_AUTOMATION_WORKER

#define UE_NET_TEST_INTERNAL(TestSuite, TestCase, BaseClass) \
class TestSuite ## _ ## TestCase : public BaseClass \
{ \
public: \
	TestSuite ## _ ## TestCase(); \
	virtual const TCHAR* GetName() const override; \
private: \
	virtual void RunTestImpl() override; \
}; \
TestSuite ## _ ## TestCase :: TestSuite ## _ ## TestCase () \
{ \
} \
const TCHAR* \
TestSuite ## _ ## TestCase :: GetName() const \
{ \
	return TEXT("Net." #TestSuite "." #TestCase); \
} \
static TestSuite ## _ ## TestCase TestSuite ## _ ## TestCase ##Instance; \
static UE::Net::FNetworkAutomationTestWrapper TestSuite ## _ ## TestCase ##AutomationTestWrapperInstance(TestSuite ## _ ## TestCase ##Instance, TestSuite ## _ ## TestCase ##Instance .GetName()); \
void TestSuite ## _ ## TestCase :: RunTestImpl()

#define UE_NET_TEST(TestSuite, TestCase) UE_NET_TEST_INTERNAL(TestSuite, TestCase, UE::Net::FNetworkAutomationTestSuiteFixture)
#define UE_NET_TEST_FIXTURE(TestFixture, TestCase) UE_NET_TEST_INTERNAL(TestFixture, TestCase, TestFixture)

#else

#define UE_NET_TEST_INTERNAL(TestSuite, TestCase, BaseClass) \
class TestSuite ## _ ## TestCase : private BaseClass \
{ \
private: \
	void DummyFunctionThatWillNeverBeCalled(); \
}; \
void TestSuite ## _ ## TestCase :: DummyFunctionThatWillNeverBeCalled()

#define UE_NET_TEST(TestSuite, TestCase) UE_NET_TEST_INTERNAL(TestSuite, TestCase, FNetworkAutomationTestSuiteFixture)
#define UE_NET_TEST_FIXTURE(TestFixture, TestCase) UE_NET_TEST_INTERNAL(TestFixture, TestCase, TestFixture)

#endif
