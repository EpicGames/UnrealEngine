// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTestMacros.h"
#include "NetworkAutomationTest.h"
#include "Misc/AssertionMacros.h"
#include "ReplicationSystemTestPlugin/NetworkAutomationTest.h"

DEFINE_LOG_CATEGORY(LogNetworkAutomationTest);

extern CORE_API bool GIgnoreDebugger;

namespace UE::Net
{

FTestEnsureScope::FTestEnsureScope()
: FEnsureScope(
	[](const FEnsureHandlerArgs& Args)
	{
		UE_LOGF(LogNetworkAutomationTest, Display, "Ensure condition failed: %s\n%ls\n", Args.Expression, Args.Message);
		return true;
	}
  )
, bIgnoreDebugger(GIgnoreDebugger)
{
	GIgnoreDebugger = true;
}

FTestEnsureScope::~FTestEnsureScope()
{
	GIgnoreDebugger = bIgnoreDebugger;
}

FTestResult::FTestResult()
: TestResult(ETestResult_Success)
{
}

class FTestResultSuccess final : public FTestResult
{
public:
	FTestResultSuccess() { TestResult = ETestResult_Success; }
};

class FTestResultFailure final : public FTestResult
{
public:
	FTestResultFailure() { TestResult = ETestResult_FatalError; }
};


FTestResult
CreateTestSuccess()
{
	return FTestResultSuccess();
}

FTestResult
CreateTestFailure()
{
	return FTestResultFailure();
}

FTestMessageLog::FTestMessageLog(FNetworkAutomationTestSuiteFixture& InTest, ELogVerbosity::Type InLogVerbosity)
: Test(InTest)
, LogVerbosity(InLogVerbosity)
{
}

#define UE_TEST_MESSAGE_LOG_(Verbosity) UE_LOGF(LogNetworkAutomationTest, Verbosity, "TestCase %ls: %ls", Test.GetName(), Message.C_Str())

FTestMessageLog::~FTestMessageLog()
{
	// UE_LOG requires a ELogVerbosity literal, without the namespace. Since we don't want a separate class per log verbosity we switch on the verbosity instead.
	switch (LogVerbosity)
	{
	case ELogVerbosity::Fatal:
		UE_TEST_MESSAGE_LOG_(Fatal);
		break;
	case ELogVerbosity::Error:
		UE_TEST_MESSAGE_LOG_(Error);
		break;
	case ELogVerbosity::Warning:
		UE_TEST_MESSAGE_LOG_(Warning);
		break;
	case ELogVerbosity::Display:
		UE_TEST_MESSAGE_LOG_(Display);
		break;
	case ELogVerbosity::Log:
		UE_TEST_MESSAGE_LOG_(Log);
		break;
	case ELogVerbosity::Verbose:
		UE_TEST_MESSAGE_LOG_(Verbose);
		break;
	case ELogVerbosity::VeryVerbose:
		UE_TEST_MESSAGE_LOG_(VeryVerbose);
		break;
	default:
		break;
	}
}

#undef UE_TEST_MESSAGE_LOG_

void
FTestMessageLog::operator=(const FTestMessage& InMessage)
{
	Message << InMessage;
}

}
