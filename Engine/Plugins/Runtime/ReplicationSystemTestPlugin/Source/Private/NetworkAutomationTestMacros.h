// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HAL/PreprocessorHelpers.h"
#include "TestMessage.h"
#include "Logging/LogVerbosity.h"
#include "Tests/EnsureScope.h"

#include <catch2/catch_test_macros.hpp>

namespace UE::Net
{
	class FNetworkAutomationTestSuiteFixture;
}

namespace UE::Net
{

class FTestEnsureScope : public FEnsureScope
{
public:
	FTestEnsureScope();
	~FTestEnsureScope();

private:
	bool bIgnoreDebugger;
};

enum ETestResult
{
	ETestResult_Success,
	ETestResult_Error,
	ETestResult_FatalError,
};

class FTestResult
{
public:
	template<typename T> FTestResult& operator<<(T Value) { Message << Value; return *this; }

	operator bool() const { return TestResult == ETestResult_Success; }
	bool operator!() const { return TestResult != ETestResult_Success; }

	FTestMessage& GetMessage() { return Message; }

protected:
	FTestResult();

	FTestMessage Message;
	ETestResult TestResult;
};

class FTestMessageLog
{
public:
	FTestMessageLog(FNetworkAutomationTestSuiteFixture& Test, ELogVerbosity::Type LogVerbosity);

	template<typename T>
	FTestMessageLog(FNetworkAutomationTestSuiteFixture& InTest, ELogVerbosity::Type InLogVerbosity, const T& InMessage)
		: Test(InTest)
		, LogVerbosity(InLogVerbosity)
	{
		Message << InMessage;
	}

	~FTestMessageLog();

	void operator=(const FTestMessage&);

private:
	FNetworkAutomationTestSuiteFixture& Test;
	FTestMessage Message;
	ELogVerbosity::Type LogVerbosity;
};

FTestResult CreateTestSuccess();
FTestResult CreateTestFailure();

inline FTestResult
BooleanTestResult(bool Value, bool ExpectedValue, const char* ValueText)
{
	if (Value == ExpectedValue)
	{
		return CreateTestSuccess();
	}
	else
	{
		return CreateTestFailure() << ValueText <<  TEXT(" is ") << (Value ? TEXT("true") : TEXT("false")) << TEXT(". Expected ") << (ExpectedValue ? TEXT("true. ") : TEXT("false. "));
	}
}

}

#define UE_NET_COMPARE_RETURN_TESTRESULT(Name, Operator) \
template<typename T> \
UE::Net::FTestResult \
Name(const T& Value, const T& ExpectedValue, const char* ValueText, const char* ExpectedValueText) \
{ \
	if (Value Operator ExpectedValue) \
	{ \
		return UE::Net::CreateTestSuccess(); \
	} \
	else \
	{ \
		return UE::Net::CreateTestFailure() << TEXT("Expected ") << ValueText << TEXT(' ') << TEXT(#Operator) <<  TEXT(' ') << ExpectedValueText << TEXT(". ") << Value << TEXT(" was compared to ") << ExpectedValue << TEXT(". "); \
	} \
} \
template<typename T, typename U, typename TEnableIf<TIsPointer<T>::Value && std::is_same_v<U, TYPE_OF_NULLPTR>, U>::Type X = nullptr> \
UE::Net::FTestResult \
Name(const T& Value, U ExpectedValue, const char* ValueText, const char* ExpectedValueText) \
{ \
	if (Value Operator ExpectedValue) \
	{ \
		return UE::Net::CreateTestSuccess(); \
	} \
	else \
	{ \
		return UE::Net::CreateTestFailure() << TEXT("Expected ") << ValueText << TEXT(' ') << TEXT(#Operator) <<  TEXT(' ') << ExpectedValueText << TEXT(". ") << Value << TEXT(" was compared to ") << ExpectedValue << TEXT(". "); \
	} \
}

#if UE_NET_WITH_LOW_LEVEL_TESTS

#define UE_NET_TEST_MSG_INTERNAL(Message) \
	UE::Net::FTestMessage UE_JOIN(TestMessage, __LINE__); \
	UE_JOIN(TestMessage, __LINE__) << Message; \
	UNSCOPED_INFO(UE_JOIN(TestMessage, __LINE__).C_Str())

#define UE_NET_ASSERT_EQ(V1, V2) REQUIRE(V1 == V2)
#define UE_NET_ASSERT_NE(V1, V2) REQUIRE(V1 != V2)
#define UE_NET_ASSERT_LT(V1, V2) REQUIRE(V1 < V2)
#define UE_NET_ASSERT_LE(V1, V2) REQUIRE(V1 <= V2)
#define UE_NET_ASSERT_GT(V1, V2) REQUIRE(V1 > V2)
#define UE_NET_ASSERT_GE(V1, V2) REQUIRE(V1 >= V2)
#define UE_NET_ASSERT_TRUE(V) REQUIRE(V)
#define UE_NET_ASSERT_FALSE(V) REQUIRE_FALSE(V)

#define UE_NET_ASSERT_EQ_MSG(V1, V2, Message) UE_NET_TEST_MSG_INTERNAL(Message); REQUIRE(V1 == V2)
#define UE_NET_ASSERT_NE_MSG(V1, V2, Message) UE_NET_TEST_MSG_INTERNAL(Message); REQUIRE(V1 != V2)
#define UE_NET_ASSERT_LT_MSG(V1, V2, Message) UE_NET_TEST_MSG_INTERNAL(Message); REQUIRE(V1 < V2)
#define UE_NET_ASSERT_LE_MSG(V1, V2, Message) UE_NET_TEST_MSG_INTERNAL(Message); REQUIRE(V1 <= V2)
#define UE_NET_ASSERT_GT_MSG(V1, V2, Message) UE_NET_TEST_MSG_INTERNAL(Message); REQUIRE(V1 > V2)
#define UE_NET_ASSERT_GE_MSG(V1, V2, Message) UE_NET_TEST_MSG_INTERNAL(Message); REQUIRE(V1 >= V2)
#define UE_NET_ASSERT_TRUE_MSG(V, Message) UE_NET_TEST_MSG_INTERNAL(Message); REQUIRE(V)
#define UE_NET_ASSERT_FALSE_MSG(V, Message) UE_NET_TEST_MSG_INTERNAL(Message); REQUIRE_FALSE(V)

#define UE_NET_EXPECT_EQ(V1, V2) CHECK(V1 == V2)
#define UE_NET_EXPECT_NE(V1, V2) CHECK(V1 != V2)
#define UE_NET_EXPECT_LT(V1, V2) CHECK(V1 < V2)
#define UE_NET_EXPECT_LE(V1, V2) CHECK(V1 <= V2)
#define UE_NET_EXPECT_GT(V1, V2) CHECK(V1 > V2)
#define UE_NET_EXPECT_GE(V1, V2) CHECK(V1 >= V2)
#define UE_NET_EXPECT_TRUE(V) CHECK(V)
#define UE_NET_EXPECT_FALSE(V) CHECK_FALSE(V)

#define UE_NET_EXPECT_EQ_MSG(V1, V2, Message) UE_NET_TEST_MSG_INTERNAL(Message); CHECK(V1 == V2)
#define UE_NET_EXPECT_NE_MSG(V1, V2, Message) UE_NET_TEST_MSG_INTERNAL(Message); CHECK(V1 != V2)
#define UE_NET_EXPECT_LT_MSG(V1, V2, Message) UE_NET_TEST_MSG_INTERNAL(Message); CHECK(V1 < V2)
#define UE_NET_EXPECT_LE_MSG(V1, V2, Message) UE_NET_TEST_MSG_INTERNAL(Message); CHECK(V1 <= V2)
#define UE_NET_EXPECT_GT_MSG(V1, V2, Message) UE_NET_TEST_MSG_INTERNAL(Message); CHECK(V1 > V2)
#define UE_NET_EXPECT_GE_MSG(V1, V2, Message) UE_NET_TEST_MSG_INTERNAL(Message); CHECK(V1 >= V2)
#define UE_NET_EXPECT_TRUE_MSG(V, Message) UE_NET_TEST_MSG_INTERNAL(Message); CHECK(V)
#define UE_NET_EXPECT_FALSE_MSG(V, Message) UE_NET_TEST_MSG_INTERNAL(Message); CHECK_FALSE(V)

#define UE_NET_FAIL(Message) FAIL(Message)
#define UE_NET_WARN(Message) FAIL_CHECK(Message)

#else

UE_NET_COMPARE_RETURN_TESTRESULT(TCmpEqual, ==)
UE_NET_COMPARE_RETURN_TESTRESULT(TCmpNotEqual, !=)
UE_NET_COMPARE_RETURN_TESTRESULT(TCmpLess, <)
UE_NET_COMPARE_RETURN_TESTRESULT(TCmpLessOrEqual, <=)
UE_NET_COMPARE_RETURN_TESTRESULT(TCmpGreater, >)
UE_NET_COMPARE_RETURN_TESTRESULT(TCmpGreaterOrEqual, >=)

#define UE_NET_FAIL_MSG(Message) return UE_DEBUG_BREAK(), this->AddTestFailure(), UE::Net::FTestMessageLog(*this, ELogVerbosity::Error) = UE::Net::FTestMessage() << Message
#define UE_NET_WARN_MSG(Message) this->AddTestWarning(), UE::Net::FTestMessageLog(*this, ELogVerbosity::Warning) = UE::Net::FTestMessage() << Message

#define UE_NET_FAIL(Message) return UE_DEBUG_BREAK(), this->AddTestFailure(), void(UE::Net::FTestMessageLog(*this, ELogVerbosity::Error, Message))
#define UE_NET_WARN(Message) this->AddTestWarning(), void(UE::Net::FTestMessageLog(*this, ELogVerbosity::Warning, Message))

#define UE_NET_TEST_(CompareWithTestResult, V1, V2, FailFunction) if (UE::Net::FTestResult Result_ = CompareWithTestResult(V1, V2, #V1, #V2)) {} else FailFunction(Result_.GetMessage())
#define UE_NET_ASSERT_INTERNAL_MSG(CompareWithTestResult, V1, V2) UE_NET_TEST_(CompareWithTestResult, V1, V2, UE_NET_FAIL_MSG)
#define UE_NET_EXPECT_INTERNAL_MSG(CompareWithTestResult, V1, V2) UE_NET_TEST_(CompareWithTestResult, V1, V2, UE_NET_WARN_MSG)

#define UE_NET_ASSERT_INTERNAL(CompareWithTestResult, V1, V2) UE_NET_TEST_(CompareWithTestResult, V1, V2, UE_NET_FAIL)
#define UE_NET_EXPECT_INTERNAL(CompareWithTestResult, V1, V2) UE_NET_TEST_(CompareWithTestResult, V1, V2, UE_NET_WARN)

#define UE_NET_TEST_BOOL_(V1, V2, FailFunction) if (UE::Net::FTestResult Result_ = UE::Net::BooleanTestResult(V1, V2, #V1)) {} else FailFunction(Result_.GetMessage())
#define UE_NET_ASSERT_BOOL_INTERNAL_MSG(V1, V2) UE_NET_TEST_BOOL_(V1, V2, UE_NET_FAIL_MSG)
#define UE_NET_EXPECT_BOOL_INTERNAL_MSG(V1, V2) UE_NET_TEST_BOOL_(V1, V2, UE_NET_WARN_MSG)

#define UE_NET_ASSERT_BOOL_INTERNAL(V1, V2) UE_NET_TEST_BOOL_(V1, V2, UE_NET_FAIL)
#define UE_NET_EXPECT_BOOL_INTERNAL(V1, V2) UE_NET_TEST_BOOL_(V1, V2, UE_NET_WARN)

// Causes a build error if the tested expression isn't compatible with Catch/low-level tests, without evaluating the expression at runtime.
#define UE_NET_VALIDATE_CATCH_EXPR(V) (false ? (void)(Catch::Decomposer() <= V) : (void)0)

#define UE_NET_ASSERT_EQ(V1, V2) UE_NET_VALIDATE_CATCH_EXPR(V1 == V2); UE_NET_ASSERT_INTERNAL(TCmpEqual, V1, V2)
#define UE_NET_ASSERT_NE(V1, V2) UE_NET_VALIDATE_CATCH_EXPR(V1 != V2); UE_NET_ASSERT_INTERNAL(TCmpNotEqual, V1, V2)
#define UE_NET_ASSERT_LT(V1, V2) UE_NET_VALIDATE_CATCH_EXPR(V1 < V2); UE_NET_ASSERT_INTERNAL(TCmpLess, V1, V2)
#define UE_NET_ASSERT_LE(V1, V2) UE_NET_VALIDATE_CATCH_EXPR(V1 <= V2); UE_NET_ASSERT_INTERNAL(TCmpLessOrEqual, V1, V2)
#define UE_NET_ASSERT_GT(V1, V2) UE_NET_VALIDATE_CATCH_EXPR(V1 > V2); UE_NET_ASSERT_INTERNAL(TCmpGreater, V1, V2)
#define UE_NET_ASSERT_GE(V1, V2) UE_NET_VALIDATE_CATCH_EXPR(V1 >= V2); UE_NET_ASSERT_INTERNAL(TCmpGreaterOrEqual, V1, V2)
#define UE_NET_ASSERT_TRUE(V) UE_NET_VALIDATE_CATCH_EXPR(V); UE_NET_ASSERT_BOOL_INTERNAL(V, true)
#define UE_NET_ASSERT_FALSE(V) UE_NET_VALIDATE_CATCH_EXPR(V); UE_NET_ASSERT_BOOL_INTERNAL(V, false)

#define UE_NET_ASSERT_EQ_MSG(V1, V2, Message) UE_NET_VALIDATE_CATCH_EXPR(V1 == V2); UE_NET_ASSERT_INTERNAL_MSG(TCmpEqual, V1, V2) << Message
#define UE_NET_ASSERT_NE_MSG(V1, V2, Message) UE_NET_VALIDATE_CATCH_EXPR(V1 != V2); UE_NET_ASSERT_INTERNAL_MSG(TCmpNotEqual, V1, V2) << Message
#define UE_NET_ASSERT_LT_MSG(V1, V2, Message) UE_NET_VALIDATE_CATCH_EXPR(V1 < V2); UE_NET_ASSERT_INTERNAL_MSG(TCmpLess, V1, V2) << Message
#define UE_NET_ASSERT_LE_MSG(V1, V2, Message) UE_NET_VALIDATE_CATCH_EXPR(V1 <= V2); UE_NET_ASSERT_INTERNAL_MSG(TCmpLessOrEqual, V1, V2) << Message
#define UE_NET_ASSERT_GT_MSG(V1, V2, Message) UE_NET_VALIDATE_CATCH_EXPR(V1 > V2); UE_NET_ASSERT_INTERNAL_MSG(TCmpGreater, V1, V2) << Message
#define UE_NET_ASSERT_GE_MSG(V1, V2, Message) UE_NET_VALIDATE_CATCH_EXPR(V1 >= V2); UE_NET_ASSERT_INTERNAL_MSG(TCmpGreaterOrEqual, V1, V2) << Message
#define UE_NET_ASSERT_TRUE_MSG(V, Message) UE_NET_VALIDATE_CATCH_EXPR(V); UE_NET_ASSERT_BOOL_INTERNAL_MSG(V, true) << Message
#define UE_NET_ASSERT_FALSE_MSG(V, Message) UE_NET_VALIDATE_CATCH_EXPR(V); UE_NET_ASSERT_BOOL_INTERNAL_MSG(V, false) << Message

#define UE_NET_EXPECT_EQ(V1, V2) UE_NET_VALIDATE_CATCH_EXPR(V1 == V2); UE_NET_EXPECT_INTERNAL(TCmpEqual, V1, V2)
#define UE_NET_EXPECT_NE(V1, V2) UE_NET_VALIDATE_CATCH_EXPR(V1 != V2); UE_NET_EXPECT_INTERNAL(TCmpNotEqual, V1, V2)
#define UE_NET_EXPECT_LT(V1, V2) UE_NET_VALIDATE_CATCH_EXPR(V1 < V2); UE_NET_EXPECT_INTERNAL(TCmpLess, V1, V2)
#define UE_NET_EXPECT_LE(V1, V2) UE_NET_VALIDATE_CATCH_EXPR(V1 <= V2); UE_NET_EXPECT_INTERNAL(TCmpLessOrEqual, V1, V2)
#define UE_NET_EXPECT_GT(V1, V2) UE_NET_VALIDATE_CATCH_EXPR(V1 > V2); UE_NET_EXPECT_INTERNAL(TCmpGreater, V1, V2)
#define UE_NET_EXPECT_GE(V1, V2) UE_NET_VALIDATE_CATCH_EXPR(V1 >= V2); UE_NET_EXPECT_INTERNAL(TCmpGreaterOrEqual, V1, V2)
#define UE_NET_EXPECT_TRUE(V) UE_NET_VALIDATE_CATCH_EXPR(V); UE_NET_EXPECT_BOOL_INTERNAL(V, true)
#define UE_NET_EXPECT_FALSE(V) UE_NET_VALIDATE_CATCH_EXPR(V); UE_NET_EXPECT_BOOL_INTERNAL(V, false)

#define UE_NET_EXPECT_EQ_MSG(V1, V2, Message) UE_NET_VALIDATE_CATCH_EXPR(V1 == V2); UE_NET_EXPECT_INTERNAL_MSG(TCmpEqual, V1, V2) << Message
#define UE_NET_EXPECT_NE_MSG(V1, V2, Message) UE_NET_VALIDATE_CATCH_EXPR(V1 != V2); UE_NET_EXPECT_INTERNAL_MSG(TCmpNotEqual, V1, V2) << Message
#define UE_NET_EXPECT_LT_MSG(V1, V2, Message) UE_NET_VALIDATE_CATCH_EXPR(V1 < V2); UE_NET_EXPECT_INTERNAL_MSG(TCmpLess, V1, V2) << Message
#define UE_NET_EXPECT_LE_MSG(V1, V2, Message) UE_NET_VALIDATE_CATCH_EXPR(V1 <= V2); UE_NET_EXPECT_INTERNAL_MSG(TCmpLessOrEqual, V1, V2) << Message
#define UE_NET_EXPECT_GT_MSG(V1, V2, Message) UE_NET_VALIDATE_CATCH_EXPR(V1 > V2); UE_NET_EXPECT_INTERNAL_MSG(TCmpGreater, V1, V2) << Message
#define UE_NET_EXPECT_GE_MSG(V1, V2, Message) UE_NET_VALIDATE_CATCH_EXPR(V1 >= V2); UE_NET_EXPECT_INTERNAL_MSG(TCmpGreaterOrEqual, V1, V2) << Message
#define UE_NET_EXPECT_TRUE_MSG(V, Message) UE_NET_VALIDATE_CATCH_EXPR(V); UE_NET_EXPECT_BOOL_INTERNAL_MSG(V, true) << Message
#define UE_NET_EXPECT_FALSE_MSG(V, Message) UE_NET_VALIDATE_CATCH_EXPR(V); UE_NET_EXPECT_BOOL_INTERNAL_MSG(V, false) << Message

#endif

#define UE_NET_LOG(Message) FTestMessageLog(*this, ELogVerbosity::Display) = FTestMessage() << Message

#define UE_NET_STRASSERT_EQ
#define UE_NET_STRASSERT_NE
#define UE_NET_STRCASEASSERT_EQ
#define UE_NET_STRCASEASSERT_NE
