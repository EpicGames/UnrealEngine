// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSystemTestPlugin/ReplicationSystemTestPlugin.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"

#include "Iris/ReplicationSystem/NetObjectFactoryRegistry.h"
#include "Tests/ReplicationSystem/ReplicatedTestObjectFactory.h"

class FReplicationSystemTestPlugin: public IReplicationSystemTestPlugin
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE( FReplicationSystemTestPlugin, ReplicationSystemTestPlugin )

void FReplicationSystemTestPlugin::StartupModule()
{
	UE::Net::FNetObjectFactoryRegistry::RegisterFactory(UReplicatedTestObjectFactory::StaticClass(), UReplicatedTestObjectFactory::GetFactoryName());
}

void FReplicationSystemTestPlugin::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

	UE::Net::FNetObjectFactoryRegistry::UnregisterFactory(UReplicatedTestObjectFactory::GetFactoryName());
}

