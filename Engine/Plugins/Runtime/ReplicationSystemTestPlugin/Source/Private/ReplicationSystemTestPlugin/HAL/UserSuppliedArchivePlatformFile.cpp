// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSystemTestPlugin/HAL/UserSuppliedArchivePlatformFile.h"
#include "Serialization/Archive.h"

/**
 * IFileHandle implementation backed by a borrowed FArchive.
 *
 * Each handle maintains its own Position and seeks the archive to that
 * position before every read. This means multiple handles opened on the
 * same archive each have an independent logical cursor, provided they are
 * not used concurrently from multiple threads.
 */
class FUserSuppliedArchiveFileHandle : public IFileHandle
{
public:
	explicit FUserSuppliedArchiveFileHandle(FArchive* InArchive)
		: Archive(InArchive)
	{}

	virtual int64 Tell() override
	{
		return Position;
	}

	virtual bool Seek(int64 NewPosition) override
	{
		Position = NewPosition;
		return true;
	}

	virtual bool SeekFromEnd(int64 NewPositionRelativeToEnd = 0) override
	{
		Position = Archive->TotalSize() + NewPositionRelativeToEnd;
		return true;
	}

	virtual bool Read(uint8* Destination, int64 BytesToRead) override
	{
		Archive->Seek(Position);
		Archive->Serialize(Destination, BytesToRead);
		if (!Archive->IsError())
		{
			Position += BytesToRead;
			return true;
		}
		return false;
	}

	virtual bool ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset) override
	{
		// Read at the given offset without affecting the handle's current position.
		Archive->Seek(Offset);
		Archive->Serialize(Destination, BytesToRead);
		return !Archive->IsError();
	}

	virtual bool Write(const uint8* Source, int64 BytesToWrite) override { return false; }
	virtual bool Flush(const bool bFullFlush = false) override { return true; }
	virtual bool Truncate(int64 NewSize) override { return false; }

	virtual int64 Size() override
	{
		return Archive->TotalSize();
	}

private:
	FArchive* Archive = nullptr;
	int64 Position = 0;
};


void FUserSuppliedArchivePlatformFile::AddArchive(const TCHAR* Filename, FArchive* Archive)
{
	Archives.Add(NormalizePath(Filename), Archive);
}

bool FUserSuppliedArchivePlatformFile::FileExists(const TCHAR* Filename)
{
	if (Archives.Contains(NormalizePath(Filename)))
	{
		return true;
	}
	return LowerLevel ? LowerLevel->FileExists(Filename) : false;
}

int64 FUserSuppliedArchivePlatformFile::FileSize(const TCHAR* Filename)
{
	FArchive** Archive = Archives.Find(NormalizePath(Filename));
	if (Archive)
	{
		return (*Archive)->TotalSize();
	}
	return LowerLevel ? LowerLevel->FileSize(Filename) : -1LL;
}

bool FUserSuppliedArchivePlatformFile::DeleteFile(const TCHAR* Filename)
{
	if (Archives.Contains(NormalizePath(Filename)))
	{
		return false;
	}
	return LowerLevel ? LowerLevel->DeleteFile(Filename) : false;
}

bool FUserSuppliedArchivePlatformFile::IsReadOnly(const TCHAR* Filename)
{
	if (Archives.Contains(NormalizePath(Filename)))
	{
		return true;
	}
	return LowerLevel ? LowerLevel->IsReadOnly(Filename) : false;
}

bool FUserSuppliedArchivePlatformFile::SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue)
{
	if (Archives.Contains(NormalizePath(Filename)))
	{
		return false;
	}
	return LowerLevel ? LowerLevel->SetReadOnly(Filename, bNewReadOnlyValue) : false;
}

FDateTime FUserSuppliedArchivePlatformFile::GetTimeStamp(const TCHAR* Filename)
{
	if (Archives.Contains(NormalizePath(Filename)))
	{
		return FDateTime::Now();
	}
	return LowerLevel ? LowerLevel->GetTimeStamp(Filename) : FDateTime::MinValue();
}

void FUserSuppliedArchivePlatformFile::SetTimeStamp(const TCHAR* Filename, FDateTime DateTime)
{
	if (!Archives.Contains(NormalizePath(Filename)) && LowerLevel)
	{
		LowerLevel->SetTimeStamp(Filename, DateTime);
	}
}

FDateTime FUserSuppliedArchivePlatformFile::GetAccessTimeStamp(const TCHAR* Filename)
{
	if (Archives.Contains(NormalizePath(Filename)))
	{
		return FDateTime::Now();
	}
	return LowerLevel ? LowerLevel->GetAccessTimeStamp(Filename) : FDateTime::MinValue();
}

FString FUserSuppliedArchivePlatformFile::GetFilenameOnDisk(const TCHAR* Filename)
{
	if (Archives.Contains(NormalizePath(Filename)))
	{
		return FString(Filename);
	}
	return LowerLevel ? LowerLevel->GetFilenameOnDisk(Filename) : FString(TEXT(""));
}

IFileHandle* FUserSuppliedArchivePlatformFile::OpenRead(const TCHAR* Filename, bool bAllowWrite)
{
	FArchive** Archive = Archives.Find(NormalizePath(Filename));
	if (Archive)
	{
		return new FUserSuppliedArchiveFileHandle(*Archive);
	}
	return LowerLevel ? LowerLevel->OpenRead(Filename, bAllowWrite) : nullptr;
}

FFileStatData FUserSuppliedArchivePlatformFile::GetStatData(const TCHAR* FilenameOrDirectory)
{
	FArchive** Archive = Archives.Find(NormalizePath(FilenameOrDirectory));
	if (Archive)
	{
		const FDateTime Now = FDateTime::Now();
		return FFileStatData(
			Now,	// CreationTime
			Now,	// AccessTime
			Now,	// ModificationTime
			(*Archive)->TotalSize(),
			/*bIsDirectory=*/false,
			/*bIsReadOnly=*/true
		);
	}
	if (LowerLevel)
	{
		return LowerLevel->GetStatData(FilenameOrDirectory);
	}
	FFileStatData Data;
	Data.bIsValid = false;
	return Data;
}
