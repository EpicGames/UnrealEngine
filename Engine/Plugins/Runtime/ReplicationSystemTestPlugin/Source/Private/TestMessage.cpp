// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestMessage.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Misc/StringBuilder.h"
#include "Net/Core/NetHandle/NetHandle.h"
#include "Net/Core/NetToken/NetToken.h"
#include "UObject/RemoteObjectTypes.h"

namespace UE::Net
{

FTestMessage& operator<<(FTestMessage& Message, const FNetHandle& NetHandle)
{
	TStringBuilder<64> StringBuilder;
	StringBuilder << NetHandle;
	return Message << StringBuilder;
}

FTestMessage& operator<<(FTestMessage& Message, const FNetRefHandle& NetRefHandle)
{
	TStringBuilder<64> StringBuilder;
	StringBuilder << NetRefHandle;
	return Message << StringBuilder;
}

FTestMessage& operator<<(FTestMessage& Message, const FNetToken& NetToken)
{
	return Message << NetToken.ToString();
}

FTestMessage& operator<<(FTestMessage& Message, const FRemoteServerId& RemoteServerId)
{
	return Message << RemoteServerId.ToString();
}

FTestMessage& operator<<(FTestMessage& Message, const FRemoteObjectId& RemoteObjectId)
{
	return Message << RemoteObjectId.ToString();
}

}
