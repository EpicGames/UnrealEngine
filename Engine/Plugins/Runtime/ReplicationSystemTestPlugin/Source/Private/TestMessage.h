// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "Math/Vector.h"
#include <cinttypes>

struct FRemoteServerId;
struct FRemoteObjectId;

namespace UE::Net
{
	class FNetHandle;
	class FNetRefHandle;
	class FNetToken;
}

namespace UE::Net
{

class FTestMessage
{
public:
	inline const TCHAR* C_Str() const { return *String; }
	inline const TCHAR* operator*() const { return *String; }

	inline FTestMessage& operator<<(const FString& InString) { String.Append(InString); return *this; }
	inline FTestMessage& operator<<(const TCHAR* InString) { String.Append(InString); return *this; }
	inline FTestMessage& operator<<(TCHAR Char) { String.AppendChar(Char); return *this; }
	inline FTestMessage& operator<<(FStringBuilderBase& StringBuilder) { String.Append(StringBuilder); return *this; }

	inline FTestMessage& operator<<(const FTestMessage& Message) { String.Append(Message.String); return *this; }

private:
	FString String;
};

inline FTestMessage& operator<<(FTestMessage& Message, int Value)
{
	return Message << FString::FromInt(Value);
}

inline FTestMessage& operator<<(FTestMessage& Message, unsigned Value)
{
	return Message << TStringBuilder<32>().Appendf(TEXT("%u"), Value);
}

inline FTestMessage& operator<<(FTestMessage& Message, int8 Value)
{
	return Message << FString::FromInt(Value);
}

inline FTestMessage& operator<<(FTestMessage& Message, int16 Value)
{
	return Message << FString::FromInt(Value);
}

inline FTestMessage& operator<<(FTestMessage& Message, int64 Value)
{
	return Message << TStringBuilder<32>().Appendf(TEXT("%" INT64_FMT), Value).ToString();
}

inline FTestMessage& operator<<(FTestMessage& Message, uint8 Value)
{
	return Message << FString::FromInt(int(Value));
}

inline FTestMessage& operator<<(FTestMessage& Message, uint16 Value)
{
	return Message << FString::FromInt(int(Value));
}

inline FTestMessage& operator<<(FTestMessage& Message, uint64 Value)
{
	return Message << TStringBuilder<32>().Appendf(TEXT("%" UINT64_FMT), Value).ToString();
}

inline FTestMessage& operator<<(FTestMessage& Message, const void* Value)
{
	return Message << TStringBuilder<32>().Appendf(TEXT("0x%" UPTRINT_X_FMT), UPTRINT(Value)).ToString();
}

// uintptr_t streaming
template<typename T, typename = typename TEnableIf<std::is_same_v<T, uintptr_t> && !std::is_same_v<uintptr_t, unsigned> && !std::is_same_v<uintptr_t, uint64>, uintptr_t>::Type>
inline FTestMessage& operator<<(FTestMessage& Message, T Value)
{
	return Message << TStringBuilder<32>().Appendf(TEXT("0x%" UPTRINT_X_FMT), UPTRINT(Value)).ToString();
}

// ptrdiff_t streaming
template<typename T, typename = typename TEnableIf<std::is_same_v<T, ptrdiff_t>, ptrdiff_t>::Type, typename = typename TEnableIf<std::is_same_v<T, ptrdiff_t> && !std::is_same_v<ptrdiff_t, int32> && !std::is_same_v<ptrdiff_t, int64>, ptrdiff_t>::Type>
inline FTestMessage& operator<<(FTestMessage& Message, T Value)
{
	return Message << TStringBuilder<32>().Appendf(TEXT("%" SSIZE_T_FMT), SSIZE_T(Value)).ToString();
}

inline FTestMessage& operator<<(FTestMessage& Message, float Value)
{
	return Message << TStringBuilder<384>().Appendf(TEXT("%f"), Value).ToString();
}

inline FTestMessage& operator<<(FTestMessage& Message, double Value)
{
	return Message << TStringBuilder<384>().Appendf(TEXT("%lf"), Value).ToString();
}

inline FTestMessage& operator<<(FTestMessage& Message, char Char)
{
	return Message << CharCast<TCHAR>(Char);
}

inline FTestMessage& operator<<(FTestMessage& Message, bool Value)
{
	return Message << (Value ? TEXT("true") : TEXT("false"));
}

inline FTestMessage& operator<<(FTestMessage& Message, TYPE_OF_NULLPTR)
{
	return Message << TEXT("nullptr");
}

inline FTestMessage& operator<<(FTestMessage& Message, const char* String)
{
	return Message << TStringBuilder<256>().Appendf(TEXT("%hs"), (String ? String : "(nullptr)"));
}

inline FTestMessage& operator<<(FTestMessage& Message, const FName& Name)
{
	return Message << Name.ToString();
}

inline FTestMessage& operator<<(FTestMessage& Message, const FVector& Vector)
{
	return Message << Vector.ToString();
}

inline FTestMessage& operator<<(FTestMessage& Message, const FVector3f& Vector)
{
	return Message << Vector.ToString();
}

FTestMessage& operator<<(FTestMessage& Message, const FNetHandle& NetHandle);

FTestMessage& operator<<(FTestMessage& Message, const FNetRefHandle& NetRefHandle);

FTestMessage& operator<<(FTestMessage& Message, const FNetToken& NetToken);

FTestMessage& operator<<(FTestMessage& Message, const FRemoteServerId& RemoteServerId);

FTestMessage& operator<<(FTestMessage& Message, const FRemoteObjectId& RemoteObjectId);

#if defined(_WIN32) && !defined(_WIN64)
inline FTestMessage& operator<<(FTestMessage& Message, SIZE_T Value)
{
	return Message << TStringBuilder<32>().Appendf(TEXT("%" SIZE_T_FMT), Value).ToString();
}
#endif

template<typename T>
inline FTestMessage& operator<<(FTestMessage& Message, const TObjectPtr<T>& ObjectPtr)
{
	return Message << TStringBuilder<512>().Appendf(TEXT("%s (%p)"), ToCStr(ObjectPtr.GetName()), ObjectPtr.Get());
}

}
