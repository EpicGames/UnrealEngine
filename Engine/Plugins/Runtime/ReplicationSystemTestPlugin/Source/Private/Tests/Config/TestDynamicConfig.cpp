// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Containers/Set.h"
#include "Iris/Config/IrisDynamicConfig.h"
#include "Iris/Config/ScopedIrisDynamicConfig.h"
#include "ReplicationSystemTestPlugin/HAL/UserSuppliedArchivePlatformFile.h"
#include "Serialization/MemoryReader.h"
#include "Tests/ReplicationSystem/ReplicationSystemTestFixture.h"
#include "Templates/UnrealTemplate.h"

namespace UE::Net::Private
{

class FTestDynamicConfigFixture : public FReplicationSystemTestFixture
{
protected:
	virtual void SetUp() override
	{
		FReplicationSystemTestFixture::SetUp();

		FIrisDynamicConfig::OnIrisDynamicConfigChange().AddRaw(this, &FTestDynamicConfigFixture::OnDynamicConfigChange);
	}

	virtual void TearDown() override
	{
		FIrisDynamicConfig::OnIrisDynamicConfigChange().RemoveAll(this);

		FReplicationSystemTestFixture::TearDown();

		OnDynamicConfigChangeModifiedSections.Reset();
	}


	void OnDynamicConfigChange(const TSet<FString>& ModifiedSections)
	{
		++DynamicConfigChangeCounter;
		OnDynamicConfigChangeModifiedSections = ModifiedSections;
	}

	const TSet<FString>& GetModifiedSections() const
	{
		return OnDynamicConfigChangeModifiedSections;
	}

	TSet<FString> GetAndClearModifiedSections() 
	{
		TSet<FString> ModifiedSections = MoveTemp(OnDynamicConfigChangeModifiedSections);
		OnDynamicConfigChangeModifiedSections = TSet<FString>();
		return ModifiedSections;
	}

	unsigned GetDynamicConfigChangeCounter() const
	{
		return DynamicConfigChangeCounter;
	}

private:
	// The set is cleared when the config system broadcasts a change
	TSet<FString> OnDynamicConfigChangeModifiedSections;
	unsigned DynamicConfigChangeCounter = 0;
};

UE_NET_TEST_FIXTURE(FTestDynamicConfigFixture, CanRegisterDynamicConfigBuffer)
{
	FScopedIrisDynamicConfig ScopedConfig;
	ScopedConfig.RegisterDynamicConfigBuffer(FName("TestDynamicConfig"), "TestDynamicConfig/Iris/Iris.ini", UTF8TEXTVIEW("[SectionName]"));

	const TSet<FString>& ModifiedSections = GetModifiedSections();
	UE_NET_ASSERT_EQ(ModifiedSections.Num(), 1);
	UE_NET_ASSERT_NE(ModifiedSections.Find("SectionName"), nullptr);
}

UE_NET_TEST_FIXTURE(FTestDynamicConfigFixture, RegisteringEmptyDynamicConfigIsNotBroadcasted)
{
	FScopedIrisDynamicConfig ScopedConfig;

	const unsigned OldCounter = GetDynamicConfigChangeCounter();
	ScopedConfig.RegisterDynamicConfigBuffer(FName("TestDynamicConfig1"), "TestDynamicConfig/Iris/Iris1.ini", UTF8TEXTVIEW(""));
	UE_NET_ASSERT_EQ(GetDynamicConfigChangeCounter(), OldCounter);

	ScopedConfig.RegisterDynamicConfigBuffer(FName("TestDynamicConfig2"), "TestDynamicConfig/Iris/Iris2.ini", FString());
	UE_NET_ASSERT_EQ(GetDynamicConfigChangeCounter(), OldCounter);
}

UE_NET_TEST_FIXTURE(FTestDynamicConfigFixture, CanUnregisterDynamicConfigBuffer)
{
	FScopedIrisDynamicConfig ScopedConfig;

	ScopedConfig.RegisterDynamicConfigBuffer(FName("TestDynamicConfig"), "TestDynamicConfig/Iris/Iris.ini", UTF8TEXTVIEW("[SectionName]"));

	GetAndClearModifiedSections();

	ScopedConfig.UnregisterDynamicConfig(FName("TestDynamicConfig"));
	const TSet<FString>& ModifiedSections = GetModifiedSections();
	UE_NET_ASSERT_EQ(ModifiedSections.Num(), 1);
	UE_NET_ASSERT_NE(ModifiedSections.Find("SectionName"), nullptr);
}

UE_NET_TEST_FIXTURE(FTestDynamicConfigFixture, CanRegisterAndUnregisterDynamicConfigsInDifferentOrder)
{
	FScopedIrisDynamicConfig ScopedConfig;

	ScopedConfig.RegisterDynamicConfigBuffer(FName("TestDynamicConfig1"), "TestDynamicConfig/Iris/Iris1.ini", UTF8TEXTVIEW("[SectionName1]"));
	ScopedConfig.RegisterDynamicConfigBuffer(FName("TestDynamicConfig2"), "TestDynamicConfig/Iris/Iris2.ini", UTF8TEXTVIEW("[SectionName2]"));

	GetAndClearModifiedSections();

	ScopedConfig.UnregisterDynamicConfig(FName("TestDynamicConfig2"));
	const TSet<FString>& ModifiedSections1 = GetModifiedSections();
	UE_NET_ASSERT_EQ(ModifiedSections1.Num(), 1);
	UE_NET_ASSERT_NE(ModifiedSections1.Find("SectionName2"), nullptr);

	ScopedConfig.UnregisterDynamicConfig(FName("TestDynamicConfig1"));
	const TSet<FString>& ModifiedSections2 = GetModifiedSections();
	UE_NET_ASSERT_EQ(ModifiedSections2.Num(), 1);
	UE_NET_ASSERT_NE(ModifiedSections2.Find("SectionName1"), nullptr);
}

UE_NET_TEST_FIXTURE(FTestDynamicConfigFixture, CanRegisterAndUnregisterConfigsWithOverlappingSectionsInDifferentOrder)
{
	FScopedIrisDynamicConfig ScopedConfig;

	ScopedConfig.RegisterDynamicConfigBuffer(FName("TestDynamicConfig1"), "TestDynamicConfig/Iris/Iris1.ini", UTF8TEXTVIEW("[SectionName1]\n; Hello\n[SectionName3]\n; World"));
	ScopedConfig.RegisterDynamicConfigBuffer(FName("TestDynamicConfig2"), "TestDynamicConfig/Iris/Iris2.ini", UTF8TEXTVIEW("[SectionName3]\n; Hello\n[SectionName2]\n; World"));

	GetAndClearModifiedSections();

	ScopedConfig.UnregisterDynamicConfig(FName("TestDynamicConfig2"));
	const TSet<FString>& ModifiedSections1 = GetModifiedSections();
	UE_NET_ASSERT_EQ(ModifiedSections1.Num(), 2);
	UE_NET_ASSERT_NE(ModifiedSections1.Find("SectionName2"), nullptr);
	UE_NET_ASSERT_NE(ModifiedSections1.Find("SectionName3"), nullptr);

	ScopedConfig.UnregisterDynamicConfig(FName("TestDynamicConfig1"));
	const TSet<FString>& ModifiedSections2 = GetModifiedSections();
	UE_NET_ASSERT_EQ(ModifiedSections2.Num(), 2);
	UE_NET_ASSERT_NE(ModifiedSections2.Find("SectionName1"), nullptr);
	UE_NET_ASSERT_NE(ModifiedSections2.Find("SectionName3"), nullptr);
}

UE_NET_TEST_FIXTURE(FTestDynamicConfigFixture, CanRetrieveSectionConfig)
{
	FScopedIrisDynamicConfig ScopedConfig;

	ScopedConfig.RegisterDynamicConfigBuffer(FName("TestDynamicConfig"), "TestDynamicConfig/Iris/Iris.ini", UTF8TEXTVIEW("[SectionName]\nKey1=Value\nKey2=Value"));

	TArray<FString> KeyValuePairsArray;
	FIrisDynamicConfig::GetSection(TEXT("SectionName"), KeyValuePairsArray);

	TSet<FString> KeyValuePairsSet(KeyValuePairsArray);
	UE_NET_ASSERT_EQ(KeyValuePairsSet.Num(), 2);

	UE_NET_ASSERT_NE(KeyValuePairsSet.Find("Key1=Value"), nullptr); 
	UE_NET_ASSERT_NE(KeyValuePairsSet.Find("Key2=Value"), nullptr); 
}

UE_NET_TEST_FIXTURE(FTestDynamicConfigFixture, CanRetrieveSectionConfigMulti)
{
	FScopedIrisDynamicConfig ScopedConfig;

	// Register multiple configs for the same section
	ScopedConfig.RegisterDynamicConfigBuffer(FName("TestDynamicConfig1"), "TestDynamicConfig/Iris/Iris1.ini", UTF8TEXTVIEW("[SectionName]\nKey1=Value\nKey2=Value"));
	ScopedConfig.RegisterDynamicConfigBuffer(FName("TestDynamicConfig2"), "TestDynamicConfig/Iris/Iris2.ini", UTF8TEXTVIEW("[SectionName]\nKey3=Value\nKey4=Value"));

	TArray<FString> KeyValuePairsArray;
	FIrisDynamicConfig::GetSection(TEXT("SectionName"), KeyValuePairsArray);

	TSet<FString> KeyValuePairsSet(KeyValuePairsArray);
	UE_NET_ASSERT_EQ(KeyValuePairsSet.Num(), 4);

	UE_NET_ASSERT_NE(KeyValuePairsSet.Find("Key1=Value"), nullptr); 
	UE_NET_ASSERT_NE(KeyValuePairsSet.Find("Key2=Value"), nullptr); 
	UE_NET_ASSERT_NE(KeyValuePairsSet.Find("Key3=Value"), nullptr); 
	UE_NET_ASSERT_NE(KeyValuePairsSet.Find("Key4=Value"), nullptr); 
}

// RegisterDynamicConfig (file-based) tests - these exercise FIrisDynamicConfig::RegisterDynamicConfig
// using FScopedUserSuppliedArchivePlatformFile to serve INI content from in-memory archives.

UE_NET_TEST_FIXTURE(FTestDynamicConfigFixture, CanRegisterDynamicConfig)
{
	const FUtf8StringView ConfigContent = UTF8TEXTVIEW("[SectionName]\nKey=Value");
	FMemoryReaderView Archive(FMemoryView(ConfigContent.GetData(), ConfigContent.Len()));

	FScopedUserSuppliedArchivePlatformFile StubFile;
	StubFile.AddArchive(TEXT("TestDynamicConfig/Iris/Iris.ini"), &Archive);

	FScopedIrisDynamicConfig ScopedConfig;
	ScopedConfig.RegisterDynamicConfig(FName("TestDynamicConfig"), "TestDynamicConfig/Iris/Iris.ini");

	const TSet<FString>& ModifiedSections = GetModifiedSections();
	UE_NET_ASSERT_EQ(ModifiedSections.Num(), 1);
	UE_NET_ASSERT_NE(ModifiedSections.Find("SectionName"), nullptr);
}

UE_NET_TEST_FIXTURE(FTestDynamicConfigFixture, RegisteringNonexistentFileIsNotBroadcasted)
{
	// No archive is registered for this path, so the config system will not find the file.
	FScopedUserSuppliedArchivePlatformFile StubFile;

	const unsigned OldCounter = GetDynamicConfigChangeCounter();

	FScopedIrisDynamicConfig ScopedConfig;
	ScopedConfig.RegisterDynamicConfig(FName("TestDynamicConfig"), "TestDynamicConfig/Iris/Missing.ini");

	UE_NET_ASSERT_EQ(GetDynamicConfigChangeCounter(), OldCounter);
}

UE_NET_TEST_FIXTURE(FTestDynamicConfigFixture, CanUnregisterDynamicConfig)
{
	const FUtf8StringView ConfigContent = UTF8TEXTVIEW("[SectionName]\nKey=Value");
	FMemoryReaderView Archive(FMemoryView(ConfigContent.GetData(), ConfigContent.Len()));

	FScopedUserSuppliedArchivePlatformFile StubFile;
	StubFile.AddArchive(TEXT("TestDynamicConfig/Iris/Iris.ini"), &Archive);

	FScopedIrisDynamicConfig ScopedConfig;
	ScopedConfig.RegisterDynamicConfig(FName("TestDynamicConfig"), "TestDynamicConfig/Iris/Iris.ini");

	GetAndClearModifiedSections();

	ScopedConfig.UnregisterDynamicConfig(FName("TestDynamicConfig"));
	const TSet<FString>& ModifiedSections = GetModifiedSections();
	UE_NET_ASSERT_EQ(ModifiedSections.Num(), 1);
	UE_NET_ASSERT_NE(ModifiedSections.Find("SectionName"), nullptr);
}

UE_NET_TEST_FIXTURE(FTestDynamicConfigFixture, CanRetrieveSectionConfigFromFile)
{
	const FUtf8StringView ConfigContent = UTF8TEXTVIEW("[SectionName]\nKey1=Value\nKey2=Value");
	FMemoryReaderView Archive(FMemoryView(ConfigContent.GetData(), ConfigContent.Len()));

	FScopedUserSuppliedArchivePlatformFile StubFile;
	StubFile.AddArchive(TEXT("TestDynamicConfig/Iris/Iris.ini"), &Archive);

	FScopedIrisDynamicConfig ScopedConfig;
	ScopedConfig.RegisterDynamicConfig(FName("TestDynamicConfig"), "TestDynamicConfig/Iris/Iris.ini");

	TArray<FString> KeyValuePairsArray;
	FIrisDynamicConfig::GetSection(TEXT("SectionName"), KeyValuePairsArray);

	TSet<FString> KeyValuePairsSet(KeyValuePairsArray);
	UE_NET_ASSERT_EQ(KeyValuePairsSet.Num(), 2);
	UE_NET_ASSERT_NE(KeyValuePairsSet.Find("Key1=Value"), nullptr);
	UE_NET_ASSERT_NE(KeyValuePairsSet.Find("Key2=Value"), nullptr);
}

UE_NET_TEST_FIXTURE(FTestDynamicConfigFixture, CanRegisterMultipleDynamicConfigsFromFiles)
{
	const FUtf8StringView ConfigContent1 = UTF8TEXTVIEW("[SectionName]\nKey1=Value\nKey2=Value");
	const FUtf8StringView ConfigContent2 = UTF8TEXTVIEW("[SectionName]\nKey3=Value\nKey4=Value");
	FMemoryReaderView Archive1(FMemoryView(ConfigContent1.GetData(), ConfigContent1.Len()));
	FMemoryReaderView Archive2(FMemoryView(ConfigContent2.GetData(), ConfigContent2.Len()));

	FScopedUserSuppliedArchivePlatformFile StubFile;
	StubFile.AddArchive(TEXT("TestDynamicConfig/Iris/Iris1.ini"), &Archive1);
	StubFile.AddArchive(TEXT("TestDynamicConfig/Iris/Iris2.ini"), &Archive2);

	FScopedIrisDynamicConfig ScopedConfig;
	ScopedConfig.RegisterDynamicConfig(FName("TestDynamicConfig1"), "TestDynamicConfig/Iris/Iris1.ini");
	ScopedConfig.RegisterDynamicConfig(FName("TestDynamicConfig2"), "TestDynamicConfig/Iris/Iris2.ini");

	TArray<FString> KeyValuePairsArray;
	FIrisDynamicConfig::GetSection(TEXT("SectionName"), KeyValuePairsArray);

	TSet<FString> KeyValuePairsSet(KeyValuePairsArray);
	UE_NET_ASSERT_EQ(KeyValuePairsSet.Num(), 4);
	UE_NET_ASSERT_NE(KeyValuePairsSet.Find("Key1=Value"), nullptr);
	UE_NET_ASSERT_NE(KeyValuePairsSet.Find("Key2=Value"), nullptr);
	UE_NET_ASSERT_NE(KeyValuePairsSet.Find("Key3=Value"), nullptr);
	UE_NET_ASSERT_NE(KeyValuePairsSet.Find("Key4=Value"), nullptr);
}

}
