// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Serialization/BitReader.h"
#include "Serialization/BitWriter.h"
#include <limits>

namespace UE::Net::Private
{
	UE_NET_TEST(FNetBitArchive, SerializeIntPacked32)
	{
		{
			const uint32 Values[] =
			{
				std::numeric_limits<uint32>::lowest(), std::numeric_limits<uint32>::max(), uint32(0), uint32(-1), (std::numeric_limits<uint32>::max() + std::numeric_limits<uint32>::lowest()) / uint32(2) - uint32(15), uint32(2048458 % std::numeric_limits<uint32>::max())
			};

			const SIZE_T ValueCount = sizeof(Values) / sizeof(Values[0]);

			for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
			{
				uint32 Value = Values[ValueIt];

				FBitWriter Writer(0, true);
				Writer.SerializeIntPacked(Value);

				UE_NET_ASSERT_FALSE(Writer.IsError());

				FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

				uint32 ResultValue = ~Value;
				Reader.SerializeIntPacked(ResultValue);

				UE_NET_ASSERT_FALSE(Reader.IsError());
				UE_NET_ASSERT_EQ(Value, ResultValue);
			}
		}
	}

	UE_NET_TEST(FNetBitArchive, SerializeIntPacked64)
	{
		{
			const uint64 Values[] =
			{
				std::numeric_limits<uint64>::lowest(), std::numeric_limits<uint64>::max(), uint64(0), uint64(-1), (std::numeric_limits<uint64>::max() + std::numeric_limits<uint64>::lowest()) / uint64(2) - uint64(15), uint64(2048458 % std::numeric_limits<uint64>::max())
			};

			const SIZE_T ValueCount = sizeof(Values) / sizeof(Values[0]);

			for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
			{
				uint64 Value = Values[ValueIt];

				FBitWriter Writer(0, true);
				Writer.SerializeIntPacked64(Value);

				UE_NET_ASSERT_FALSE(Writer.IsError());

				FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

				uint64 ResultValue = ~Value;
				Reader.SerializeIntPacked64(ResultValue);

				UE_NET_ASSERT_FALSE(Reader.IsError());
				UE_NET_ASSERT_EQ(Value, ResultValue);
			}
		}
	}

	// Pulled from NetBitsTest.cpp
	UE_NET_TEST(FNetBitArchive, SerializeInt)
	{
		// SerializeInt, Zero Value, 2 range
		{
			FBitWriter Writer(0, true);

			uint32 WriteValue = 0;
			Writer.SerializeInt(WriteValue, 2);

			UE_NET_ASSERT_FALSE(Writer.IsError());
			UE_NET_ASSERT_EQ(Writer.GetNumBits(), (int64)1);

			FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

			uint32 ReadValue = ~WriteValue;
			Reader.SerializeInt(ReadValue, 2);

			UE_NET_ASSERT_FALSE(Reader.IsError());
			UE_NET_ASSERT_EQ(ReadValue, WriteValue);
		}

		// SerializeInt, 1 Value, 2 range
		{
			FBitWriter Writer(0, true);

			uint32 WriteValue = 1;
			Writer.SerializeInt(WriteValue, 2);

			UE_NET_ASSERT_FALSE(Writer.IsError());
			UE_NET_ASSERT_EQ(Writer.GetNumBits(), (int64)1);

			FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

			uint32 ReadValue = ~WriteValue;
			Reader.SerializeInt(ReadValue, 2);

			UE_NET_ASSERT_FALSE(Reader.IsError());
			UE_NET_ASSERT_EQ(ReadValue, WriteValue);
		}

		// SerializeInt, 0 Value, 3 range
		{
			FBitWriter Writer(0, true);

			uint32 WriteValue = 0;
			Writer.SerializeInt(WriteValue, 3);

			UE_NET_ASSERT_FALSE(Writer.IsError());
			UE_NET_ASSERT_EQ(Writer.GetNumBits(), (int64)2);

			FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

			uint32 ReadValue = ~WriteValue;
			Reader.SerializeInt(ReadValue, 3);

			UE_NET_ASSERT_FALSE(Reader.IsError());
			UE_NET_ASSERT_EQ(ReadValue, WriteValue);
		}

		// SerializeInt, 1 Value, 3 range
		{
			FBitWriter Writer(0, true);
			uint32 WriteValue = 1;

			Writer.SerializeInt(WriteValue, 3);

			UE_NET_ASSERT_FALSE(Writer.IsError());
			UE_NET_ASSERT_EQ(Writer.GetNumBits(), (int64)1);

			FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

			uint32 ReadValue = ~WriteValue;
			Reader.SerializeInt(ReadValue, 3);

			UE_NET_ASSERT_FALSE(Reader.IsError());
			UE_NET_ASSERT_EQ(ReadValue, WriteValue);
		}

		// SerializeInt, 2 Value, 3 range
		{
			FBitWriter Writer(0, true);
			uint32 WriteValue = 2;

			Writer.SerializeInt(WriteValue, 3);

			UE_NET_ASSERT_FALSE(Writer.IsError());
			UE_NET_ASSERT_EQ(Writer.GetNumBits(), (int64)2);

			FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

			uint32 ReadValue = ~WriteValue;
			Reader.SerializeInt(ReadValue, 3);

			UE_NET_ASSERT_FALSE(Reader.IsError());
			UE_NET_ASSERT_EQ(ReadValue, WriteValue);
		}

		// SerializeInt, 0 Value, 4294967295U range
		{
			FBitWriter Writer(0, true);
			uint32 WriteValue = 0;

			Writer.SerializeInt(WriteValue, 4294967295U);

			UE_NET_ASSERT_FALSE(Writer.IsError());
			UE_NET_ASSERT_EQ(Writer.GetNumBits(), (int64)32);

			FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

			uint32 ReadValue = ~WriteValue;
			Reader.SerializeInt(ReadValue, 4294967295U);

			UE_NET_ASSERT_FALSE(Reader.IsError());
			UE_NET_ASSERT_EQ(ReadValue, WriteValue);
		}

		// SerializeInt, 4294967294U Value, 4294967295U range
		{
			FBitWriter Writer(0, true);
			uint32 WriteValue = 4294967294U;

			Writer.SerializeInt(WriteValue, 4294967295U);

			UE_NET_ASSERT_FALSE(Writer.IsError());
			UE_NET_ASSERT_EQ(Writer.GetNumBits(), (int64)32);

			FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

			uint32 ReadValue = ~WriteValue;
			Reader.SerializeInt(ReadValue, 4294967295U);

			UE_NET_ASSERT_FALSE(Reader.IsError());
			UE_NET_ASSERT_EQ(ReadValue, WriteValue);
		}
	}
}
