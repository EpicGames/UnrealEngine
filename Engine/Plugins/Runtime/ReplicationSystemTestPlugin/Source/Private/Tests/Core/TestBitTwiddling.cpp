// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/Core/BitTwiddling.h"
#include <limits>

namespace UE::Net::Private
{

// GetBitsNeeded
UE_NET_TEST(TestGetBitsNeeded, ReturnsExpectedValuesForInt8)
{
	UE_NET_ASSERT_EQ(GetBitsNeeded(std::numeric_limits<int8>::min()), 8U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(int8(-64)), 7U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(int8(-1)), 1U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(int8(0)), 1U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(int8(1)), 2U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(std::numeric_limits<int8>::max()), 8U);
}

UE_NET_TEST(TestGetBitsNeeded, ReturnsExpectedValuesForInt32)
{
	UE_NET_ASSERT_EQ(GetBitsNeeded(std::numeric_limits<int32>::min()), 32U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(int32(-64)), 7U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(int32(-1)), 1U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(int32(0)), 1U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(int32(1)), 2U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(std::numeric_limits<int32>::max()), 32U);
}

UE_NET_TEST(TestGetBitsNeeded, ReturnsExpectedValuesForInt64)
{
	UE_NET_ASSERT_EQ(GetBitsNeeded(std::numeric_limits<int64>::min()), 64U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(int64(-64)), 7U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(int64(-1)), 1U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(int64(0)), 1U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(int64(1)), 2U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(std::numeric_limits<int64>::max()), 64U);
}

UE_NET_TEST(TestGetBitsNeeded, ReturnsExpectedValuesForUint8)
{
	UE_NET_ASSERT_EQ(GetBitsNeeded(uint8(0)), 0U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(uint8(1)), 1U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(std::numeric_limits<uint8>::max()), 8U);
}

UE_NET_TEST(TestGetBitsNeeded, ReturnsExpectedValuesForUint32)
{
	UE_NET_ASSERT_EQ(GetBitsNeeded(uint32(0)), 0U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(uint32(1)), 1U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(std::numeric_limits<uint32>::max()), 32U);
}

UE_NET_TEST(TestGetBitsNeeded, ReturnsExpectedValuesForUint64)
{
	UE_NET_ASSERT_EQ(GetBitsNeeded(uint64(0)), 0U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(uint64(1)), 1U);
	UE_NET_ASSERT_EQ(GetBitsNeeded(std::numeric_limits<uint64>::max()), 64U);
}

// GetBitsNeededForRange
UE_NET_TEST(TestGetBitsNeededForRange, ReturnsExpectedValuesForInt8)
{
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(int8(47), int8(47)), 0U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(std::numeric_limits<int8>::min(), int8(-1)), 7U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(int8(std::numeric_limits<int8>::min() + 5), int8(4)), 7U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(std::numeric_limits<int8>::min(), int8(0)), 8U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(int8(11), int8(47)), 6U);
}

UE_NET_TEST(TestGetBitsNeededForRange, ReturnsExpectedValuesForInt16)
{
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(int16(4711), int16(4711)), 0U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(std::numeric_limits<int16>::min(), int16(-1)), 15U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(int16(std::numeric_limits<int16>::min() + 5), int16(4)), 15U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(std::numeric_limits<int16>::min(), int16(0)), 16U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(int16(11), int16(47)), 6U);
}

UE_NET_TEST(TestGetBitsNeededForRange, ReturnsExpectedValuesForInt32)
{
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(int32(4711), int32(4711)), 0U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(std::numeric_limits<int32>::min(), int32(-1)), 31U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(int32(std::numeric_limits<int32>::min() + 5), int32(4)), 31U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(std::numeric_limits<int32>::min(), int32(0)), 32U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(int32(11), int32(47)), 6U);
}

UE_NET_TEST(TestGetBitsNeededForRange, ReturnsExpectedValuesForInt64)
{
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(int64(4711), int64(4711)), 0U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(std::numeric_limits<int64>::min(), int64(-1)), 63U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(int64(std::numeric_limits<int64>::min() + 5), int64(4)), 63U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(std::numeric_limits<int64>::min(), int64(0)), 64U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(int64(11), int64(47)), 6U);
}

UE_NET_TEST(TestGetBitsNeededForRange, ReturnsExpectedValuesForUint8)
{
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(uint8(47), uint8(47)), 0U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(uint8(std::numeric_limits<uint8>::max()/2U + 1U), std::numeric_limits<uint8>::max()), 7U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(std::numeric_limits<uint8>::min(), std::numeric_limits<uint8>::max()), 8U);
}

UE_NET_TEST(TestGetBitsNeededForRange, ReturnsExpectedValuesForUint16)
{
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(uint16(4711), uint16(4711)), 0U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(uint16(std::numeric_limits<uint16>::max()/2U + 1U), std::numeric_limits<uint16>::max()), 15U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(std::numeric_limits<uint16>::min(), std::numeric_limits<uint16>::max()), 16U);
}

UE_NET_TEST(TestGetBitsNeededForRange, ReturnsExpectedValuesForUint32)
{
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(uint32(4711), uint32(4711)), 0U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(uint32(std::numeric_limits<uint32>::max() /2U + 1U), std::numeric_limits<uint32>::max()), 31U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(std::numeric_limits<uint32>::min(), std::numeric_limits<uint32>::max()), 32U);
}

UE_NET_TEST(TestGetBitsNeededForRange, ReturnsExpectedValuesForUint64)
{
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(uint64(4711), uint64(4711)), 0U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(uint64(std::numeric_limits<uint64>::max() /2U + 1U), std::numeric_limits<uint64>::max()), 63U);
	UE_NET_ASSERT_EQ(GetBitsNeededForRange(std::numeric_limits<uint64>::min(), std::numeric_limits<uint64>::max()), 64U);
}

}