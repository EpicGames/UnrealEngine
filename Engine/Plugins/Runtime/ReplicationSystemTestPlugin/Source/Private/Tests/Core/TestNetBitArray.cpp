// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Net/Core/NetBitArray.h"
#include "Net/Core/NetBitArrayPrinter.h"

namespace UE::Net::Private
{

UE_NET_TEST(FNetBitArrayView, Construct)
{	
	{
		const uint32 ExpectedSingleWordBuffer = 0xfe;
		uint32 SingleWordBuffer = ExpectedSingleWordBuffer;

		FNetBitArrayView BitArray(&SingleWordBuffer, 8);
		UE_NET_ASSERT_EQ(ExpectedSingleWordBuffer, SingleWordBuffer);
		UE_NET_ASSERT_EQ(1u, BitArray.GetNumWords());
	}
	{
		const uint32 ExpectedSingleWordBuffer = 0xfefefefe;
		uint32 SingleWordBuffer = ExpectedSingleWordBuffer;

		FNetBitArrayView BitArray(&SingleWordBuffer, 32);
		UE_NET_ASSERT_EQ(ExpectedSingleWordBuffer, SingleWordBuffer);
		UE_NET_ASSERT_EQ(1u, BitArray.GetNumWords());
	}
}

UE_NET_TEST(FNetBitArrayView, ClearAllBits)
{	
	{
		const uint32 ExpectedSingleWordBuffer = 0u;
		uint32 SingleWordBuffer = 0xfe;
		FNetBitArrayView BitArray(&SingleWordBuffer, 8);
		BitArray.ClearAllBits();

		UE_NET_ASSERT_EQ(ExpectedSingleWordBuffer, SingleWordBuffer);
	}

	{
		const uint32 ExpectedSingleWordBuffer = 0u;
		uint32 SingleWordBuffer = 0xfefefefe;
		FNetBitArrayView BitArray(&SingleWordBuffer, 32);
		BitArray.ClearAllBits();

		UE_NET_ASSERT_EQ(ExpectedSingleWordBuffer, SingleWordBuffer);
	}

	{
		const uint32 ExpectedWordBuffer[] =  { 0u, 2u, 3u, 4u };
		uint32 WordBuffer[] = { 1u, 2u, 3u, 4 };

		FNetBitArrayView BitArray(&WordBuffer[0], 8);
		BitArray.ClearAllBits();

		for (uint32 it = 0; it < UE_ARRAY_COUNT(ExpectedWordBuffer); ++it)
		{
			UE_NET_ASSERT_EQ(ExpectedWordBuffer[it], WordBuffer[it]);
		}
	}

	{
		const uint32 ExpectedWordBuffer[] =  { 0u, 0u, 0u, 0u };
		uint32 WordBuffer[] = { 1, 2, 3, 4 };

		FNetBitArrayView BitArray(&WordBuffer[0], 128);
		BitArray.ClearAllBits();

		for (uint32 it = 0; it < UE_ARRAY_COUNT(ExpectedWordBuffer); ++it)
		{
			UE_NET_ASSERT_EQ(ExpectedWordBuffer[it], WordBuffer[it]);
		}
	}
}

UE_NET_TEST(FNetBitArrayView, IsAnyBitSetIsFalseAfterClearAllBits)
{	
	{
		uint32 SingleWordBuffer = 0xfe;
		FNetBitArrayView BitArray(&SingleWordBuffer, 8);
		BitArray.ClearAllBits();

		UE_NET_ASSERT_FALSE(BitArray.IsAnyBitSet());
		UE_NET_ASSERT_FALSE(BitArray.IsAnyBitSet(0, ~0U));
	}

	{
		uint32 WordBuffer[] = { 1, 2, 3, 4 };

		FNetBitArrayView BitArray(&WordBuffer[0], 128);
		BitArray.ClearAllBits();

		UE_NET_ASSERT_FALSE(BitArray.IsAnyBitSet());
		UE_NET_ASSERT_FALSE(BitArray.IsAnyBitSet(0, ~0U));
	}
}

UE_NET_TEST(FNetBitArrayView, IsAllBitsSetAfterReset)
{	
	{
		uint32 SingleWordBuffer = 0xfe;
		const uint32 ExpectedSingleWordBuffer = 0xff;
		FNetBitArrayView BitArray(&SingleWordBuffer, 8);
		BitArray.SetAllBits();

		UE_NET_ASSERT_EQ(ExpectedSingleWordBuffer, SingleWordBuffer);
	}

	{
		uint32 WordBuffer[] = { 1, 2, 3, 4 };
		const uint32 ExpectedWordBuffer[] = { ~0U, ~0U, ~0U, ~0U };

		FNetBitArrayView BitArray(&WordBuffer[0], 128);
		BitArray.SetAllBits();

		UE_NET_ASSERT_EQ(ExpectedWordBuffer[0], WordBuffer[0]);
		UE_NET_ASSERT_EQ(ExpectedWordBuffer[1], WordBuffer[1]);
		UE_NET_ASSERT_EQ(ExpectedWordBuffer[2], WordBuffer[2]);
		UE_NET_ASSERT_EQ(ExpectedWordBuffer[3], WordBuffer[3]);
	}
}


UE_NET_TEST(FNetBitArrayView, IsAnyBitSetAndIsNoBitSet)
{	
	{
		uint32 SingleWordBuffer = 0xfe;
		FNetBitArrayView BitArray(&SingleWordBuffer, 8);

		UE_NET_ASSERT_TRUE(BitArray.IsAnyBitSet());
		UE_NET_ASSERT_FALSE(BitArray.IsNoBitSet());
	}

	{
		uint32 SingleWordBuffer = 0x00;
		FNetBitArrayView BitArray(&SingleWordBuffer, 8);

		UE_NET_ASSERT_FALSE(BitArray.IsAnyBitSet());
		UE_NET_ASSERT_TRUE(BitArray.IsNoBitSet());
	}

	{
		uint32 SingleWordBuffer = 0x01;
		FNetBitArrayView BitArray(&SingleWordBuffer, 8);

		UE_NET_ASSERT_TRUE(BitArray.IsAnyBitSet());
		UE_NET_ASSERT_FALSE(BitArray.IsNoBitSet());
	}

	{
		uint32 WordBuffer[] = { 1, 2, 3, 4 };

		FNetBitArrayView BitArray(&WordBuffer[0], 128);

		UE_NET_ASSERT_TRUE(BitArray.IsAnyBitSet());
		UE_NET_ASSERT_FALSE(BitArray.IsNoBitSet());
	}

	{
		uint32 WordBuffer[] = { 0, 0, 0, 1 };

		FNetBitArrayView BitArray(&WordBuffer[0], 128);

		UE_NET_ASSERT_TRUE(BitArray.IsAnyBitSet());
	}

	{
		uint32 WordBuffer[] = { 1, 0, 0, 0 };

		FNetBitArrayView BitArray(&WordBuffer[0], 128);

		UE_NET_ASSERT_TRUE(BitArray.IsAnyBitSet());
		UE_NET_ASSERT_FALSE(BitArray.IsNoBitSet());
	}

	{
		uint32 WordBuffer[] = { 0, 0, 0, 0 };

		FNetBitArrayView BitArray(&WordBuffer[0], 128);

		UE_NET_ASSERT_FALSE(BitArray.IsAnyBitSet());
		UE_NET_ASSERT_TRUE(BitArray.IsNoBitSet());
	}

}

UE_NET_TEST(FNetBitArrayView, IsAnyBitSetInRange)
{
	// Test zero bits set
	{
		uint32 WordBuffer[] = { 0, 0, 0, 0, 0, 0, 0, };
		FNetBitArrayView BitArray(&WordBuffer[0], 8*sizeof(WordBuffer));
		UE_NET_ASSERT_FALSE(BitArray.IsAnyBitSet(0U, ~0U));
	}

	// Test no bits are set when checking zero bits
	{
		uint32 WordBuffer[] = { ~0U, ~0U, ~0U, ~0U, ~0U, ~0U, ~0U, };
		FNetBitArrayView BitArray(&WordBuffer[0], 8*sizeof(WordBuffer));
		UE_NET_ASSERT_FALSE(BitArray.IsAnyBitSet(0U, 0U));
		UE_NET_ASSERT_FALSE(BitArray.IsAnyBitSet(BitArray.GetNumBits() - 1U, 0U));
	}

	// Test no bits are set when checking out of bounds
	{
		uint32 WordBuffer[] = { ~0U, ~0U, ~0U, ~0U, ~0U, ~0U, ~0U, };
		FNetBitArrayView BitArray(&WordBuffer[0], 8*sizeof(WordBuffer) - 3U, FNetBitArrayView::NoResetNoValidate);
		UE_NET_ASSERT_FALSE(BitArray.IsAnyBitSet(BitArray.GetNumBits(), 0U));
		UE_NET_ASSERT_FALSE(BitArray.IsAnyBitSet(BitArray.GetNumBits(), ~0U));
	}

	// Test bits are set in whole words
	{
		uint32 WordBuffer[] = { 0x00000041U, 0x00004100U, 0x00410000U, 0x41000000U, };
		FNetBitArrayView BitArray(&WordBuffer[0], 8*sizeof(WordBuffer));
		UE_NET_ASSERT_TRUE(BitArray.IsAnyBitSet(0U, 32U));
		UE_NET_ASSERT_TRUE(BitArray.IsAnyBitSet(32U, 32U));
		UE_NET_ASSERT_TRUE(BitArray.IsAnyBitSet(64U, 32U));
		UE_NET_ASSERT_TRUE(BitArray.IsAnyBitSet(96U, 32U));
	}

	// Test single bit is set
	{
		uint32 WordBuffer[] = { 0, 0, 0, 0, 0, 0, 0, };
		FNetBitArrayView BitArray(&WordBuffer[0], 8*sizeof(WordBuffer));
		constexpr uint32 TestBitIndex = 75;
		BitArray.SetBit(TestBitIndex);

		// No bit before test index is set
		UE_NET_ASSERT_FALSE(BitArray.IsAnyBitSet(0U, TestBitIndex));
		// Exact bit is set
		UE_NET_ASSERT_TRUE(BitArray.IsAnyBitSet(TestBitIndex, 1U));
		// Range including the test index at the end
		UE_NET_ASSERT_TRUE(BitArray.IsAnyBitSet(0, TestBitIndex +  1U));
		// Range including test index in between
		UE_NET_ASSERT_TRUE(BitArray.IsAnyBitSet(TestBitIndex - 32U, 40U));
		// Range including the test index at the start
		UE_NET_ASSERT_TRUE(BitArray.IsAnyBitSet(TestBitIndex, ~0U));
		// No bit after test index is set
		UE_NET_ASSERT_FALSE(BitArray.IsAnyBitSet(TestBitIndex + 1U, ~0U));
	}
}

UE_NET_TEST(FNetBitArrayView, ClearBits)
{
	// Test single bit
	{
		uint32 WordBuffer[] = { 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, };
		FNetBitArrayView BitArray(&WordBuffer[0], 8*sizeof(WordBuffer));

		BitArray.ClearBits(0,1);
		UE_NET_ASSERT_FALSE(BitArray.GetBit(0));
		UE_NET_ASSERT_TRUE(BitArray.FindFirstZero(1) == BitArray.InvalidIndex);
	}

	// Test clear multiple bits at beginning
	{
		uint32 WordBuffer[] = { 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, };
		FNetBitArrayView BitArray(&WordBuffer[0], 8*sizeof(WordBuffer));

		const uint32 BitsToSet = 4U;
		
		BitArray.ClearBits(0, BitsToSet);
		for (uint32 BitIt = 0U; BitIt < BitsToSet; ++BitIt)
		{
			UE_NET_ASSERT_FALSE(BitArray.GetBit(BitIt));
		}
		UE_NET_ASSERT_TRUE(BitArray.FindFirstZero(BitsToSet) == BitArray.InvalidIndex);
	}

	// Test clear multiple bits spanning word boundary
	{
		uint32 WordBuffer[] = { 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, };
		FNetBitArrayView BitArray(&WordBuffer[0], 8*sizeof(WordBuffer));

		const uint32 BitsToSet = 35U;
		const uint32 BitOffset = 0U;
		
		BitArray.ClearBits(0, BitsToSet);
		for (uint32 BitIt = BitOffset; BitIt < (BitOffset + BitsToSet); ++BitIt)
		{
			UE_NET_ASSERT_FALSE(BitArray.GetBit(BitIt));
		}
		UE_NET_ASSERT_TRUE(BitArray.FindFirstZero(BitOffset + BitsToSet) == BitArray.InvalidIndex);
	}

	// Test preserves surrounding bits at beginning
	{
		uint32 WordBuffer[] = { 0xffffff00, 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, 0x00ffffff };
		FNetBitArrayView BitArray(&WordBuffer[0], 8*sizeof(WordBuffer));

		const uint32 PreservedBits = 8U;

		const uint32 BitsToSet = 4U;
		const uint32 BitOffset = PreservedBits;
		
		BitArray.ClearBits(BitOffset, BitsToSet);
		for (uint32 BitIt = 0U; BitIt < (BitOffset + BitsToSet); ++BitIt)
		{
			UE_NET_ASSERT_FALSE(BitArray.GetBit(BitIt));
		}
		UE_NET_ASSERT_TRUE(BitArray.FindFirstZero(BitOffset + BitsToSet) == (BitArray.GetNumBits() - PreservedBits));
	}

	// Test preserves surrounding bits at end
	{
		uint32 WordBuffer[] = { 0xffffff00, 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff, 0x00ffffff };
		FNetBitArrayView BitArray(&WordBuffer[0], 8*sizeof(WordBuffer));

		const uint32 PreservedBits = 8U;

		const uint32 BitsToSet = 4U;
		const uint32 BitOffset = BitArray.GetNumBits() - (PreservedBits + BitsToSet);
		
		BitArray.ClearBits(BitOffset, BitsToSet);
		for (uint32 BitIt = BitOffset; BitIt < (BitOffset + BitsToSet + PreservedBits); ++BitIt)
		{
			UE_NET_ASSERT_FALSE(BitArray.GetBit(BitIt));
		}
		UE_NET_ASSERT_TRUE(BitArray.FindFirstZero(PreservedBits) == BitOffset);
	}
}

UE_NET_TEST(FNetBitArrayView, SetBits)
{
	// Test single bit
	{
		uint32 WordBuffer[] = { 0, 0, 0, 0, 0, 0, 0, };
		FNetBitArrayView BitArray(&WordBuffer[0], 8*sizeof(WordBuffer));

		BitArray.SetBits(0,1);
		UE_NET_ASSERT_TRUE(BitArray.GetBit(0));
		UE_NET_ASSERT_FALSE(BitArray.GetBit(1));
	}

	// Test set multiple bits at beginning
	{
		uint32 WordBuffer[] = { 0, 0, 0, 0, 0, 0, 0, };
		FNetBitArrayView BitArray(&WordBuffer[0], 8*sizeof(WordBuffer));

		const uint32 BitsToSet = 4U;
		
		BitArray.SetBits(0, BitsToSet);
		for (uint32 BitIt = 0U; BitIt < BitsToSet; ++BitIt)
		{
			UE_NET_ASSERT_TRUE(BitArray.GetBit(BitIt));
		}
		UE_NET_ASSERT_TRUE(BitArray.FindFirstOne(BitsToSet) == BitArray.InvalidIndex);
	}

	// Test set multiple bits spanning word boundary
	{
		uint32 WordBuffer[] = { 0, 0, 0, 0, 0, 0, 0, };
		FNetBitArrayView BitArray(&WordBuffer[0], 8*sizeof(WordBuffer));

		const uint32 BitsToSet = 35U;
		const uint32 BitOffset = 0U;
		
		BitArray.SetBits(0, BitsToSet);
		for (uint32 BitIt = BitOffset; BitIt < (BitOffset + BitsToSet); ++BitIt)
		{
			UE_NET_ASSERT_TRUE(BitArray.GetBit(BitIt));
		}
		UE_NET_ASSERT_TRUE(BitArray.FindFirstOne(BitOffset + BitsToSet) == BitArray.InvalidIndex);
	}

	// Test preserves surrounding bits at beginning
	{
		uint32 WordBuffer[] = { 0x000000ff, 0, 0, 0, 0, 0, 0xff000000 };
		FNetBitArrayView BitArray(&WordBuffer[0], 8*sizeof(WordBuffer));

		const uint32 PreservedBits = 8U;

		const uint32 BitsToSet = 4U;
		const uint32 BitOffset = PreservedBits;
		
		BitArray.SetBits(BitOffset, BitsToSet);
		for (uint32 BitIt = 0U; BitIt < (BitOffset + BitsToSet); ++BitIt)
		{
			UE_NET_ASSERT_TRUE(BitArray.GetBit(BitIt));
		}
		UE_NET_ASSERT_TRUE(BitArray.FindFirstOne(BitOffset + BitsToSet) == (BitArray.GetNumBits() - PreservedBits));
	}

	// Test preserves surrounding bits at end
	{
		uint32 WordBuffer[] = { 0x000000ff, 0, 0, 0, 0, 0, 0xff000000 };
		FNetBitArrayView BitArray(&WordBuffer[0], 8*sizeof(WordBuffer));

		const uint32 PreservedBits = 8U;

		const uint32 BitsToSet = 4U;
		const uint32 BitOffset = BitArray.GetNumBits() - (PreservedBits + BitsToSet);
		
		BitArray.SetBits(BitOffset, BitsToSet);
		for (uint32 BitIt = BitOffset; BitIt < (BitOffset + BitsToSet + PreservedBits); ++BitIt)
		{
			UE_NET_ASSERT_TRUE(BitArray.GetBit(BitIt));
		}
		UE_NET_ASSERT_TRUE(BitArray.FindFirstOne(PreservedBits) == BitOffset);
	}
}


UE_NET_TEST(FNetBitArrayView, SetBitValue)
{
	uint32 WordBuffer[] = { 0, 0, 0, 0, 0, 0, 0, };
	FNetBitArrayView BitArray(&WordBuffer[0], 8 * sizeof(WordBuffer));
	constexpr uint32 TestBitIndex = 75;
	BitArray.SetBitValue(TestBitIndex, true);
	UE_NET_ASSERT_TRUE(BitArray.GetBit(TestBitIndex));
	BitArray.SetBitValue(TestBitIndex, false);
	UE_NET_ASSERT_FALSE(BitArray.GetBit(TestBitIndex));
}

UE_NET_TEST(FNetBitArrayView, ClearBit)
{
	uint32 WordBuffer[] = { 0, 0, 0, 0, 0, 0, 0, };
	FNetBitArrayView BitArray(&WordBuffer[0], 8 * sizeof(WordBuffer));
	constexpr uint32 TestBitIndex = 75;
	BitArray.SetBit(TestBitIndex);
	BitArray.ClearBit(TestBitIndex);
	UE_NET_ASSERT_FALSE(BitArray.GetBit(TestBitIndex));
}

UE_NET_TEST(FNetBitArrayView, OrBitValue)
{
	uint32 WordBuffer[] = { ~(0U), 0U, 0xAAAA };
	FNetBitArrayView BitArray(&WordBuffer[0], 8 * sizeof(WordBuffer));
	
	// Test first uint32 (~0)
	{
		UE_NET_ASSERT_EQ(BitArray.CountSetBits(0, 32), 32U);

		UE_NET_ASSERT_TRUE(BitArray.GetBit(1));
		BitArray.OrBitValue(1, true);
		UE_NET_ASSERT_TRUE(BitArray.GetBit(1));

		UE_NET_ASSERT_EQ(BitArray.CountSetBits(0, 32), 32U);

		UE_NET_ASSERT_TRUE(BitArray.GetBit(2));
		BitArray.OrBitValue(2, false);
		UE_NET_ASSERT_TRUE(BitArray.GetBit(2));

		UE_NET_ASSERT_EQ(BitArray.CountSetBits(0, 32), 32U);
	}

	// Test second uint32 (0)
	{
		UE_NET_ASSERT_EQ(BitArray.CountSetBits(32, 32), 0U);

		UE_NET_ASSERT_FALSE(BitArray.GetBit(33));
		BitArray.OrBitValue(33, true);
		UE_NET_ASSERT_TRUE(BitArray.GetBit(33));

		UE_NET_ASSERT_EQ(BitArray.CountSetBits(32, 32), 1U);

		UE_NET_ASSERT_FALSE(BitArray.GetBit(34));
		BitArray.OrBitValue(34, false);
		UE_NET_ASSERT_FALSE(BitArray.GetBit(34));

		UE_NET_ASSERT_EQ(BitArray.CountSetBits(32, 32), 1U);
	}

	// Test third uint32 (0xAAAA)
	{
		UE_NET_ASSERT_EQ(BitArray.CountSetBits(64, 32), 8U);

		UE_NET_ASSERT_TRUE(BitArray.GetBit(65));
		BitArray.OrBitValue(65, true);
		UE_NET_ASSERT_TRUE(BitArray.GetBit(65));

		UE_NET_ASSERT_EQ(BitArray.CountSetBits(64, 32), 8U);

		UE_NET_ASSERT_FALSE(BitArray.GetBit(66));
		BitArray.OrBitValue(66, true);
		UE_NET_ASSERT_TRUE(BitArray.GetBit(66));

		UE_NET_ASSERT_EQ(BitArray.CountSetBits(64, 32), 9U);
	}	
}

UE_NET_TEST(FNetBitArrayView, AndBitValue)
{
	uint32 WordBuffer[] = { ~(0U), 0U, 0xAAAA };
	FNetBitArrayView BitArray(&WordBuffer[0], 8 * sizeof(WordBuffer));

	// Test first uint32 (~0)
	{
		UE_NET_ASSERT_EQ(BitArray.CountSetBits(0, 32), 32U);

		UE_NET_ASSERT_TRUE(BitArray.GetBit(1));
		BitArray.AndBitValue(1, true);
		UE_NET_ASSERT_TRUE(BitArray.GetBit(1));

		UE_NET_ASSERT_EQ(BitArray.CountSetBits(0, 32), 32U);

		UE_NET_ASSERT_TRUE(BitArray.GetBit(2));
		BitArray.AndBitValue(2, false);
		UE_NET_ASSERT_FALSE(BitArray.GetBit(2));

		UE_NET_ASSERT_EQ(BitArray.CountSetBits(0, 32), 31U);
	}

	// Test second uint32 (0)
	{
		UE_NET_ASSERT_EQ(BitArray.CountSetBits(32, 32), 0U);

		UE_NET_ASSERT_FALSE(BitArray.GetBit(33));
		BitArray.AndBitValue(33, true);
		UE_NET_ASSERT_FALSE(BitArray.GetBit(33));

		UE_NET_ASSERT_EQ(BitArray.CountSetBits(32, 32), 0U);

		UE_NET_ASSERT_FALSE(BitArray.GetBit(34));
		BitArray.AndBitValue(34, false);
		UE_NET_ASSERT_FALSE(BitArray.GetBit(34));

		UE_NET_ASSERT_EQ(BitArray.CountSetBits(32, 32), 0U);
	}

	// Test third uint32 (0xAAAA)
	{
		UE_NET_ASSERT_EQ(BitArray.CountSetBits(64, 32), 8U);

		UE_NET_ASSERT_FALSE(BitArray.GetBit(64));
		BitArray.AndBitValue(64, true);
		UE_NET_ASSERT_FALSE(BitArray.GetBit(64));

		UE_NET_ASSERT_EQ(BitArray.CountSetBits(64, 32), 8U);

		UE_NET_ASSERT_TRUE(BitArray.GetBit(65));
		BitArray.AndBitValue(65, true);
		UE_NET_ASSERT_TRUE(BitArray.GetBit(65));

		UE_NET_ASSERT_EQ(BitArray.CountSetBits(64, 32), 8U);

		UE_NET_ASSERT_TRUE(BitArray.GetBit(65));
		BitArray.AndBitValue(65, false);
		UE_NET_ASSERT_FALSE(BitArray.GetBit(65));

		UE_NET_ASSERT_EQ(BitArray.CountSetBits(64, 32), 7U);
	}
}
UE_NET_TEST(FNetBitArrayView, GetSetBit)
{	
	{
		uint32 ExpectedWordBuffer = 0xaebecede;
		uint32 DstWordBuffer = ExpectedWordBuffer;

		const FNetBitArrayView SrcBitArray(&ExpectedWordBuffer, 32);
		FNetBitArrayView DstBitArray(&DstWordBuffer, 32);
		DstBitArray.ClearAllBits();

		UE_NET_ASSERT_FALSE(DstBitArray.IsAnyBitSet());

		for (uint32 It=0; It < 32; ++It)
		{
			if (SrcBitArray.GetBit(It))
			{
				DstBitArray.SetBit(It);
			}
		}

		UE_NET_ASSERT_EQ(ExpectedWordBuffer, DstWordBuffer);
	}

	{
		uint32 ExpectedWordBuffer[] = { 0xaebecede, 0xa0000001, 0x10101010, 0x1 };
		uint32 DstWordBuffer[]= { 0xaebecede, 0xa0000001, 0x10101010, 0x1 };

		const FNetBitArrayView SrcBitArray(ExpectedWordBuffer, 128);
		FNetBitArrayView DstBitArray(DstWordBuffer, 128);
		DstBitArray.ClearAllBits();

		UE_NET_ASSERT_FALSE(DstBitArray.IsAnyBitSet());

		for (uint32 It=0; It < 128; ++It)
		{
			if (SrcBitArray.GetBit(It))
			{
				DstBitArray.SetBit(It);
			}
		}

		UE_NET_ASSERT_TRUE(FMemory::Memcmp(ExpectedWordBuffer, DstWordBuffer, sizeof(ExpectedWordBuffer)) == 0);
	}

}

UE_NET_TEST(FNetBitArrayView, TestFindFirstZero)
{
	// No bits set
	{
		uint32 WordBuffer[] = { 0x00, 0x00, 0x00, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(), 0U);
	}

	// All bits set, except padding bits
	{
		uint32 WordBuffer[] = { 0x03, 0x00, 0x00, };
		FNetBitArrayView BitArray(WordBuffer, 2U);

		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(), BitArray.InvalidIndex);
	}

	// Some bits set, but not at position zero
	{
		uint32 WordBuffer[] = { 0xFE, 0xEF, 0xEE, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(), 0U);
	}

	// Many bits set starting from offset 0
	{
		uint32 WordBuffer[] = { 0xFFFFFFFFU, 0xFFFFFFFFU, 0xFF0FFFFFU, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(), 84U);
	}
}

UE_NET_TEST(FNetBitArrayView, TestFindFirstOne)
{
	// No bits set
	{
		uint32 WordBuffer[] = { 0x00, 0x00, 0x00, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(), BitArray.InvalidIndex);
	}

	// No bits set, except for padding bits
	{
		uint32 WordBuffer[] = { 0xFFFFFFF0U, 0xFFFFFFFFU, 0xFFFFFFFFU, };
		FNetBitArrayView BitArray(WordBuffer, 4U, FNetBitArrayView::NoResetNoValidate);

		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(), BitArray.InvalidIndex);
	}

	// All bits set, including padding bits
	{
		uint32 WordBuffer[] = { 0xFFFFFFFFU, 0xFFFFFFFFU, 0xFFFFFFFFU, };
		FNetBitArrayView BitArray(WordBuffer, 9U, FNetBitArrayView::NoResetNoValidate);

		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(), 0U);
	}

	// Some bits set, but not at position zero
	{
		uint32 WordBuffer[] = { 0xF0, 0xEF, 0x0F, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(), 4U);
	}

	// A couple of bits set in the middle of the buffer
	{
		uint32 WordBuffer[] = { 0x00000000, 0x00300000U, 0x00000000, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(), 52U);
	}
}

UE_NET_TEST(FNetBitArrayView, TestFindFirstZeroFromIndex)
{
	// No bits set
	{
		uint32 WordBuffer[] = { 0x00, 0x00, 0x00, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(0), 0U);
		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(BitArray.GetNumBits() - 1U), BitArray.GetNumBits() - 1U);

		// Trying to find a zero bit outside the array should always fail
		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(~0U), BitArray.InvalidIndex);
	}

	// All bits set, except padding bits
	{
		uint32 WordBuffer[] = { 0x03, 0x00, 0x00, };
		FNetBitArrayView BitArray(WordBuffer, 2U);

		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(0), BitArray.InvalidIndex);
		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(1), BitArray.InvalidIndex);
		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(2), BitArray.InvalidIndex);
	}

	// Some bits set, but not at position zero
	{
		uint32 WordBuffer[] = { 0xFE, 0xEF, 0xEE, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(0), 0U);
		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(1), 8U);
		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(32), 36U);
	}

	// Many bits set starting from offset 0
	{
		uint32 WordBuffer[] = { 0xFFFFFFFFU, 0xFFFFFFFFU, 0xFF0FFFFFU, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(0), 84U);
		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(84), 84U);
		UE_NET_ASSERT_EQ(BitArray.FindFirstZero(88), BitArray.InvalidIndex);
	}
}

UE_NET_TEST(FNetBitArrayView, TestFindFirstOneFromIndex)
{
	// No bits set
	{
		uint32 WordBuffer[] = { 0x00, 0x00, 0x00, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(0), BitArray.InvalidIndex);
		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(64), BitArray.InvalidIndex);
		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(~0U), BitArray.InvalidIndex);
	}

	// No bits set, except for padding bits
	{
		uint32 WordBuffer[] = { 0xFFFFFFF0U, 0xFFFFFFFFU, 0xFFFFFFFFU, };
		FNetBitArrayView BitArray(WordBuffer, 4U, FNetBitArrayView::NoResetNoValidate);

		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(0), BitArray.InvalidIndex);
		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(BitArray.GetNumBits()), BitArray.InvalidIndex);
	}

	// All bits set, including padding bits
	{
		uint32 WordBuffer[] = { 0xFFFFFFFFU, 0xFFFFFFFFU, 0xFFFFFFFFU, };
		FNetBitArrayView BitArray(WordBuffer, 9U, FNetBitArrayView::NoResetNoValidate);

		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(0), 0U);
		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(8), 8U);
		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(9), BitArray.InvalidIndex);
	}

	// Some bits set, but not at position zero
	{
		uint32 WordBuffer[] = { 0xF0, 0xEF, 0x0F, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(0), 4U);
		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(36), 37U);
	}

	// A couple of bits set in the middle of the buffer
	{
		uint32 WordBuffer[] = { 0x00000000, 0x00300000U, 0x00000000, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(0), 52U);
		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(32), 52U);
		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(52), 52U);
		UE_NET_ASSERT_EQ(BitArray.FindFirstOne(100), BitArray.InvalidIndex);
	}
}

UE_NET_TEST(FNetBitArrayView, TestFindLastZero)
{
	// No bits set
	{
		uint32 WordBuffer[] = { 0x00, 0x00, 0x00, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindLastZero(), BitArray.GetNumBits() - 1U);
	}

	// All bits set, except padding bits
	{
		uint32 WordBuffer[] = { 0x03 };
		FNetBitArrayView BitArray(WordBuffer, 2U);

		UE_NET_ASSERT_EQ(BitArray.FindLastZero(), BitArray.InvalidIndex);
	}

	// Arbitrary zero in the middle of the array
	{
		uint32 WordBuffer[] = { 0xFFFFFFFFU, 0xFFFCCFFFU, 0xFFFFFFFFU, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindLastZero(), 49U);
	}

	// All bits zero, with unaligned bitcount
	{
		uint32 WordBuffer[] = { 0x0 };
		FNetBitArrayView BitArray(WordBuffer, 31);

		// Verify that we do not get an out of bounds value
		UE_NET_ASSERT_LT(BitArray.FindLastZero(), 31U);
		UE_NET_ASSERT_EQ(BitArray.FindLastZero(), 30U);
	}

	// Set last bit, unaligned bitcount
	{
		uint32 WordBuffer[] = { 0x40000000 };
		FNetBitArrayView BitArray(WordBuffer, 31);

		UE_NET_ASSERT_EQ(BitArray.FindLastZero(), 29U);
	}

	// All bits set, unaligned bitcount
	{
		uint32 WordBuffer[] = { 0x7fffffff };
		FNetBitArrayView BitArray(WordBuffer, 31);

		UE_NET_ASSERT_EQ(BitArray.FindLastZero(), BitArray.InvalidIndex);
	}

}

UE_NET_TEST(FNetBitArrayView, TestFindLastOne)
{
	// No bits set
	{
		uint32 WordBuffer[] = { 0x00, 0x00, 0x00, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindLastOne(), BitArray.InvalidIndex);
	}

	// All bits set, except for padding bits
	{
		uint32 WordBuffer[] = { (1U << 9U) - 1U, 0x00, 0x00, };
		FNetBitArrayView BitArray(WordBuffer, 9U, FNetBitArrayView::NoResetNoValidate);

		UE_NET_ASSERT_EQ(BitArray.FindLastOne(), BitArray.GetNumBits() - 1U);
	}

	// Some bits set
	{
		uint32 WordBuffer[] = { 0xF0, 0xEF, 0x0FFFFFFFU, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindLastOne(), 91U);
	}

	// A couple of bits set in the middle of the buffer
	{
		uint32 WordBuffer[] = { 0x00000000, 0x00030000U, 0x00000000, };
		FNetBitArrayView BitArray(WordBuffer, sizeof(WordBuffer)*8);

		UE_NET_ASSERT_EQ(BitArray.FindLastOne(), 49U);
	}
}

UE_NET_TEST(FNetBitArrayView, TestForEachSet)
{	
	struct FCollectSetBitsFunctor
	{
		void operator()(uint32 Index) { Invoked.Add(Index); }
		TArray<uint32> Invoked;
	};


	// Test with first two bits set
	{
		uint32 SingleWordBuffer = 0xffffff00;
		FNetBitArrayView BitArray(&SingleWordBuffer, 8, FNetBitArrayView::NoResetNoValidate);
		
		FCollectSetBitsFunctor Functor;

		BitArray.ForAllSetBits(Functor);
		UE_NET_ASSERT_EQ(0, Functor.Invoked.Num());

		Functor.Invoked.Reset();
		BitArray.ForAllUnsetBits(Functor);
		UE_NET_ASSERT_EQ(8, Functor.Invoked.Num());
	}

	// Test with two last bits set
	{
		uint32 SingleWordBuffer = 0xa0000000;
		FNetBitArrayView BitArray(&SingleWordBuffer, 32);
		
		FCollectSetBitsFunctor Functor;

		BitArray.ForAllSetBits(Functor);
		UE_NET_ASSERT_EQ(2, Functor.Invoked.Num());

		Functor.Invoked.Reset();
		BitArray.ForAllUnsetBits(Functor);
		UE_NET_ASSERT_EQ(30, Functor.Invoked.Num());
	}

	// Test ForAllSetBits with first and last bits set
	{
		uint32 SingleWordBuffer = 0x80000001;
		FNetBitArrayView BitArray(&SingleWordBuffer, 32);
		
		FCollectSetBitsFunctor Functor;

		BitArray.ForAllSetBits(Functor);
		UE_NET_ASSERT_EQ(2, Functor.Invoked.Num());
		UE_NET_ASSERT_EQ(0u, Functor.Invoked[0]);
		UE_NET_ASSERT_EQ(31u, Functor.Invoked[1]);
	}

	// Test ForAllUnsetBits with first and last bits not set
	{
		uint32 SingleWordBuffer = 0x7FFFFFFE;
		FNetBitArrayView BitArray(&SingleWordBuffer, 32);
		
		FCollectSetBitsFunctor Functor;
		
		BitArray.ForAllUnsetBits(Functor);
		UE_NET_ASSERT_EQ(2, Functor.Invoked.Num());
		UE_NET_ASSERT_EQ(0u, Functor.Invoked[0]);
		UE_NET_ASSERT_EQ(31u, Functor.Invoked[1]);
	}

	// Test ForAllSetBits with ALL bits set
	{
		uint32 SingleWordBuffer = 0xffffffff;
		FNetBitArrayView BitArray(&SingleWordBuffer, 32);
		
		FCollectSetBitsFunctor Functor;
		BitArray.ForAllSetBits(Functor);

		UE_NET_ASSERT_EQ(32, Functor.Invoked.Num());
	}

	// Test ForAllUnsetBits with ZERO bits set
	{
		uint32 SingleWordBuffer = 0x00000000;
		FNetBitArrayView BitArray(&SingleWordBuffer, 32);
		
		FCollectSetBitsFunctor Functor;
		BitArray.ForAllUnsetBits(Functor);

		UE_NET_ASSERT_EQ(32, Functor.Invoked.Num());
	}

	// Test ForAllSetBit with random bits
	{
		const uint32 IndicesToTest[] = { 1, 31, 32, 34, 35, 36, 37, 64, 65, 66, 126 };
		const uint32 ExpectedNumBitsSet = UE_ARRAY_COUNT(IndicesToTest);

		uint32 WordBuffer[32] = { 0 };

		FNetBitArrayView BitArray(&WordBuffer[0], 127);

		for (uint32 It = 0; It < ExpectedNumBitsSet; ++It)
		{
			BitArray.SetBit(IndicesToTest[It]);
		}

		FCollectSetBitsFunctor Functor;
		BitArray.ForAllSetBits(Functor);

		UE_NET_ASSERT_EQ(ExpectedNumBitsSet, (uint32)Functor.Invoked.Num());

		for (uint32 It = 0; It < ExpectedNumBitsSet; ++It)
		{
			UE_NET_ASSERT_EQ(IndicesToTest[It], Functor.Invoked[It]);
		}
	}

	// Test ForAllUnsetBits with random bits
	{
		const uint32 IndicesToTest[] = { 1, 31, 32, 34, 35, 36, 37, 64, 65, 66, 126 };
		const uint32 ExpectedNumBitsSet = UE_ARRAY_COUNT(IndicesToTest);

		uint32 WordBuffer[32] = { 0 };
		
		FNetBitArrayView BitArray(&WordBuffer[0], 127);
		BitArray.SetAllBits();

		for (uint32 It = 0; It < ExpectedNumBitsSet; ++It)
		{
			BitArray.SetBitValue(IndicesToTest[It], false);
		}

		FCollectSetBitsFunctor Functor;
		BitArray.ForAllUnsetBits(Functor);

		UE_NET_ASSERT_EQ(ExpectedNumBitsSet, (uint32)Functor.Invoked.Num());

		for (uint32 It = 0; It < ExpectedNumBitsSet; ++It)
		{
			UE_NET_ASSERT_EQ(IndicesToTest[It], Functor.Invoked[It]);
		}
	}

	// Test ForAllSetBitsInRange with ALL bits set
	{
		uint32 SingleWordBuffer = 0xffffffff;
		FNetBitArrayView BitArray(&SingleWordBuffer, 32);

		FCollectSetBitsFunctor FunctorOne;
		BitArray.ForAllSetBitsInRange(0, 31, FunctorOne);

		UE_NET_ASSERT_EQ(32, FunctorOne.Invoked.Num());

		FCollectSetBitsFunctor FunctorTwo;
		BitArray.ForAllSetBitsInRange(0, 15, FunctorTwo);

		UE_NET_ASSERT_EQ(16, FunctorTwo.Invoked.Num());

		FCollectSetBitsFunctor FunctorThree;
		BitArray.ForAllSetBitsInRange(16, 31, FunctorThree);

		UE_NET_ASSERT_EQ(16, FunctorThree.Invoked.Num());
	}

	// Test ForAllSetBitsInRange with NO bits set
	{
		uint32 SingleWordBuffer = 0x00000000;
		FNetBitArrayView BitArray(&SingleWordBuffer, 32);

		FCollectSetBitsFunctor Functor;
		BitArray.ForAllSetBitsInRange(0, 31, Functor);

		UE_NET_ASSERT_EQ(0, Functor.Invoked.Num());
	}

	// Test ForAllSetBitsInRange with HALF bits set
	{
		uint32 SingleWordBuffer = 0xffff0000;
		FNetBitArrayView BitArray(&SingleWordBuffer, 32);

		FCollectSetBitsFunctor Functor;
		BitArray.ForAllSetBitsInRange(0, 31, Functor);

		UE_NET_ASSERT_EQ(16, Functor.Invoked.Num());
	}

	// Test ForAllSetBitsInRange with HALF bits set (interleaved)
	{
		uint32 SingleWordBuffer = 0xaaaaaaaa;
		FNetBitArrayView BitArray(&SingleWordBuffer, 32);

		FCollectSetBitsFunctor Functor;
		BitArray.ForAllSetBitsInRange(0, 31, Functor);

		UE_NET_ASSERT_EQ(16, Functor.Invoked.Num());
	}

	// Test ForAllSetBitsInRange double word boundary test (HALF bits set, interleaved)
	{
		uint64 DoubleWordBuffer = 0xaaaaaaaaaaaaaaaa;
		FNetBitArrayView BitArray(reinterpret_cast<uint32*>(&DoubleWordBuffer), 64);

		FCollectSetBitsFunctor Functor;
		BitArray.ForAllSetBitsInRange(20, 39, Functor);

		UE_NET_ASSERT_EQ(10, Functor.Invoked.Num());
	}
}

UE_NET_TEST(FNetBitArrayView, TestForAllSetBitsMultiple)
{	
	struct FCollectSetBitsFunctor
	{
		void operator()(uint32 Index) { Invoked.Add(Index); }
		TArray<uint32> Invoked;
	};

	auto TestOp = []<typename WordOp1, typename WordOp2>(WordOp1 Op1, WordOp2 Op2) -> TTuple<int32, int32>
	{
		uint32 BufferA = 0xFFFFFFFF;
		FNetBitArrayView BitArrayA(&BufferA, 32, FNetBitArrayView::NoResetNoValidate);

		uint32 BufferB = 0xAAAAAAAA;
		FNetBitArrayView BitArrayB(&BufferB, 32, FNetBitArrayView::NoResetNoValidate);

		uint32 BufferC = 0x00000000;
		FNetBitArrayView BitArrayC(&BufferC, 32, FNetBitArrayView::NoResetNoValidate);

		FCollectSetBitsFunctor Functor;
		FNetBitArrayView::ForAllSetBitsMultiple(BitArrayA, Op1, BitArrayB, Op2, BitArrayC, Functor);

		const uint32 FinalWord = Op1(BufferA, Op2(BufferB, BufferC));
		const int32 ExpectedBits = FMath::CountBits(FinalWord);
		
		return TTuple<int32, int32>(ExpectedBits, Functor.Invoked.Num());
	};

	{
		auto Result = TestOp(FNetBitArrayView::AndOp, FNetBitArrayView::OrOp);
		UE_NET_ASSERT_EQ(Result.Get<0>(), Result.Get<1>());
	}

	{
		auto Result = TestOp(FNetBitArrayView::AndNotOp, FNetBitArrayView::OrOp);
		UE_NET_ASSERT_EQ(Result.Get<0>(), Result.Get<1>());
	}

	{
		auto Result = TestOp(FNetBitArrayView::XorOp, FNetBitArrayView::OrOp);
		UE_NET_ASSERT_EQ(Result.Get<0>(), Result.Get<1>());
	}

	{
		auto Result = TestOp(FNetBitArrayView::OrOp, FNetBitArrayView::AndOp);
		UE_NET_ASSERT_EQ(Result.Get<0>(), Result.Get<1>());
	}
}

UE_NET_TEST(FNetBitArrayView, TestGetSetBitIndices)
{	
	constexpr uint32 OOBValue = ~0U;

	// No bits set
	{
		uint32 SingleWordBuffer = 0xffffff00;
		FNetBitArrayView BitArray(&SingleWordBuffer, 8, FNetBitArrayView::NoResetNoValidate);
		
		uint32 Indices[1];
		for (uint32 StartOffset = 0; StartOffset < BitArray.GetNumBits(); ++StartOffset)
		{
			const uint32 IndexCount = BitArray.GetSetBitIndices(StartOffset, ~0U, Indices, 1);
			UE_NET_ASSERT_EQ(IndexCount, 0U);
		}
	}

	// Single word with bits set
	{
		uint32 SingleWordBuffer = 0x40000001;
		FNetBitArrayView BitArray(&SingleWordBuffer, 32);
		
		uint32 Indices[3] = {OOBValue, OOBValue, OOBValue};
		
		{
			const uint32 IndexCount = BitArray.GetSetBitIndices(0U, 0U, Indices, 1);
			UE_NET_ASSERT_EQ(IndexCount, 0U);

			UE_NET_ASSERT_EQ(Indices[0], OOBValue);
		}

		{
			const uint32 IndexCount = BitArray.GetSetBitIndices(BitArray.GetNumBits() - 1U, 0U, Indices, 1);
			UE_NET_ASSERT_EQ(IndexCount, 0U);

			UE_NET_ASSERT_EQ(Indices[0], OOBValue);
		}
		{
			const uint32 IndexCount = BitArray.GetSetBitIndices(0U, 1U, Indices, 1);
			UE_NET_ASSERT_EQ(IndexCount, 1U);
			UE_NET_ASSERT_EQ(Indices[0], 0U);

			UE_NET_ASSERT_EQ(Indices[1], OOBValue);
		}

		{
			const uint32 IndexCount = BitArray.GetSetBitIndices(0U, 30U, Indices, 2);
			UE_NET_ASSERT_EQ(IndexCount, 1U);
			UE_NET_ASSERT_EQ(Indices[0], 0U);

			UE_NET_ASSERT_EQ(Indices[2], OOBValue);
		}

		{
			const uint32 IndexCount = BitArray.GetSetBitIndices(1U, ~0U, Indices, 2);
			UE_NET_ASSERT_EQ(IndexCount, 1U);
			UE_NET_ASSERT_EQ(Indices[0], 30U);

			UE_NET_ASSERT_EQ(Indices[2], OOBValue);
		}

		{
			const uint32 IndexCount = BitArray.GetSetBitIndices(0U, ~0U, Indices, 2);
			UE_NET_ASSERT_EQ(IndexCount, 2U);
			UE_NET_ASSERT_EQ(Indices[0], 0U);
			UE_NET_ASSERT_EQ(Indices[1], 30U);

			UE_NET_ASSERT_EQ(Indices[2], OOBValue);
		}
	}

	// Many words with arbitrary bits set
	{
		const uint32 IndicesToTest[] = { 1, 31, 32, 34, 35, 36, 37, 64, 65, 66, 162 };
		const uint32 ExpectedNumBitsSet = UE_ARRAY_COUNT(IndicesToTest);
		uint32 Indices[UE_ARRAY_COUNT(IndicesToTest) + 2];

		uint32 WordBuffer[32] = {};
		FNetBitArrayView BitArray(&WordBuffer[0], IndicesToTest[ExpectedNumBitsSet - 1U] + 1U);
		for (uint32 It = 0; It < ExpectedNumBitsSet; ++It)
		{
			BitArray.SetBit(IndicesToTest[It]);
			Indices[It] = OOBValue;
		}
		Indices[UE_ARRAY_COUNT(Indices) - 2] = OOBValue;
		Indices[UE_ARRAY_COUNT(Indices) - 1] = OOBValue;

		// Test range spanning words without set bits
		{
			const uint32 IndexCount = BitArray.GetSetBitIndices(96U, 64U, Indices, 1);
			UE_NET_ASSERT_EQ(IndexCount, 0U);

			UE_NET_ASSERT_EQ(Indices[1], OOBValue);
		}

		// Test range spanning three words with bits set
		{
			const uint32 IndexCount = BitArray.GetSetBitIndices(8U, 57U, Indices, 9);
			UE_NET_ASSERT_EQ(IndexCount, 7U);
			UE_NET_ASSERT_EQ(Indices[0], 31U);
			UE_NET_ASSERT_EQ(Indices[1], 32U);
			UE_NET_ASSERT_EQ(Indices[2], 34U);
			UE_NET_ASSERT_EQ(Indices[3], 35U);
			UE_NET_ASSERT_EQ(Indices[4], 36U);
			UE_NET_ASSERT_EQ(Indices[5], 37U);
			UE_NET_ASSERT_EQ(Indices[6], 64U);

			UE_NET_ASSERT_EQ(Indices[9], OOBValue);
		}

		// Get all set bits
		{
			const uint32 IndexCount = BitArray.GetSetBitIndices(0U, ~0U, Indices, ExpectedNumBitsSet + 1);
			UE_NET_ASSERT_EQ(IndexCount, ExpectedNumBitsSet);
			UE_NET_ASSERT_EQ(0, FMemory::Memcmp(IndicesToTest, Indices, ExpectedNumBitsSet*sizeof(*Indices)));

			UE_NET_ASSERT_EQ(Indices[ExpectedNumBitsSet + 1], OOBValue);
		}
	}
}

UE_NET_TEST(FNetBitArrayView, TestCountSetBits)
{	
	constexpr uint32 OOBValue = ~0U;

	// No bits set
	{
		uint32 SingleWordBuffer = 0xffffff00;
		FNetBitArrayView BitArray(&SingleWordBuffer, 8, FNetBitArrayView::NoResetNoValidate);
		
		for (uint32 StartOffset = 0, EndOffset = BitArray.GetNumBits(); StartOffset < EndOffset; ++StartOffset)
		{
			const uint32 SetBitCount = BitArray.CountSetBits(StartOffset);
			UE_NET_ASSERT_EQ(SetBitCount, 0U);
		}
	}

	// Single word with bits set
	{
		uint32 SingleWordBuffer = 0x40000001;
		FNetBitArrayView BitArray(&SingleWordBuffer, 32);
		
		uint32 Indices[3] = {OOBValue, OOBValue, OOBValue};
		
		{
			const uint32 SetBitCount = BitArray.CountSetBits(0U, 0U);
			UE_NET_ASSERT_EQ(SetBitCount, 0U);

			UE_NET_ASSERT_EQ(Indices[0], OOBValue);
		}

		{
			const uint32 SetBitCount = BitArray.CountSetBits(BitArray.GetNumBits() - 1U, 0U);
			UE_NET_ASSERT_EQ(SetBitCount, 0U);

			UE_NET_ASSERT_EQ(Indices[0], OOBValue);
		}
		{
			const uint32 SetBitCount = BitArray.CountSetBits(0U, 1U);
			UE_NET_ASSERT_EQ(SetBitCount, 1U);
		}

		{
			const uint32 SetBitCount = BitArray.CountSetBits(0U, 30U);
			UE_NET_ASSERT_EQ(SetBitCount, 1U);
		}

		{
			const uint32 SetBitCount = BitArray.CountSetBits(1U, ~0U);
			UE_NET_ASSERT_EQ(SetBitCount, 1U);
		}

		{
			const uint32 SetBitCount = BitArray.CountSetBits(0U, ~0U);
			UE_NET_ASSERT_EQ(SetBitCount, 2U);
		}
	}

	// Many words with arbitrary bits set
	{
		const uint32 IndicesToTest[] = { 1, 31, 32, 34, 35, 36, 37, 64, 65, 66, 162, };
		const uint32 ExpectedNumBitsSet = UE_ARRAY_COUNT(IndicesToTest);

		uint32 WordBuffer[32] = {};
		FNetBitArrayView BitArray(&WordBuffer[0], IndicesToTest[ExpectedNumBitsSet - 1U] + 1U);
		for (uint32 It = 0; It < ExpectedNumBitsSet; ++It)
		{
			BitArray.SetBit(IndicesToTest[It]);
		}

		// Test range spanning words without set bits
		{
			const uint32 SetBitCount = BitArray.CountSetBits(96U, 64U);
			UE_NET_ASSERT_EQ(SetBitCount, 0U);
		}

		// Test range spanning three words with bits set
		{
			const uint32 SetBitCount = BitArray.CountSetBits(8U, 57U);
			UE_NET_ASSERT_EQ(SetBitCount, 7U);
		}

		// Count all set bits
		{
			const uint32 SetBitCount = BitArray.CountSetBits();
			UE_NET_ASSERT_EQ(SetBitCount, ExpectedNumBitsSet);
		}
	}
}

UE_NET_TEST(FNetBitArrayView, Or)
{	
	auto&& WordOp = FNetBitArrayView::OrOp;

	// Test partial word
	{
		const uint32 ExpectedSingleWordBuffer = 0x0000ffff;
		uint32 SingleWordBufferA = 0xfefe;
		uint32 SingleWordBufferB = 0x1111;

		FNetBitArrayView BitArrayA(&SingleWordBufferA, 16);
		const FNetBitArrayView BitArrayB(&SingleWordBufferB, 16);

		BitArrayA.Combine(BitArrayB, WordOp);

		UE_NET_ASSERT_EQ(ExpectedSingleWordBuffer, SingleWordBufferA);
	}

	// Test full word
	{
		const uint32 ExpectedSingleWordBuffer = 0xffffffff;
		uint32 SingleWordBufferA = 0xfefefefe;
		uint32 SingleWordBufferB = 0x11111111;

		FNetBitArrayView BitArrayA(&SingleWordBufferA, 32);
		const FNetBitArrayView BitArrayB(&SingleWordBufferB, 32);

		BitArrayA.Combine(BitArrayB, WordOp);

		UE_NET_ASSERT_EQ(ExpectedSingleWordBuffer, SingleWordBufferA);
	}

	// Test Multiple words
	{
		const uint32 ExpectedWordBuffer[] =  { 1u, 2u, 3u, 4u };
		uint32 WordBufferA[] = { 0, 0, 0, 0 };
		uint32 WordBufferB[] = { 1, 2, 3, 4 };

		FNetBitArrayView BitArrayA(&WordBufferA[0], 128);
		const FNetBitArrayView BitArrayB(&WordBufferB[0], 128);

		BitArrayA.Combine(BitArrayB, WordOp);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(ExpectedWordBuffer); ++it)
		{
			UE_NET_ASSERT_EQ(ExpectedWordBuffer[it], WordBufferA[it]);
		}
	}

	// Test Multiple arrays
	{
		const uint32 ExpectedWordBuffer[] = { 0x0011, 0x0110, 0x1100, 0x1001 };
		uint32 WordBufferA[] = { 0x0000, 0x0000, 0x0000, 0x0000 };
		uint32 WordBufferB[] = { 0x0001, 0x0010, 0x0100, 0x1000 };
		uint32 WordBufferC[] = { 0x0010, 0x0100, 0x1000, 0x0001 };

		FNetBitArrayView BitArrayA(&WordBufferA[0], 128);
		const FNetBitArrayView BitArrayB(&WordBufferB[0], 128);
		const FNetBitArrayView BitArrayC(&WordBufferC[0], 128);

		BitArrayA.CombineMultiple(WordOp, BitArrayB, WordOp, BitArrayC);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(ExpectedWordBuffer); ++it)
		{
			UE_NET_ASSERT_EQ(ExpectedWordBuffer[it], WordBufferA[it]);
		}
	}
}

UE_NET_TEST(FNetBitArrayView, And)
{
	auto&& WordOp = FNetBitArrayView::AndOp;

	// Test partial word
	{
		const uint32 ExpectedSingleWordBuffer = 0x00001111;
		uint32 SingleWordBufferA = 0xffff;
		uint32 SingleWordBufferB = 0x1111;

		FNetBitArrayView BitArrayA(&SingleWordBufferA, 16);
		const FNetBitArrayView BitArrayB(&SingleWordBufferB, 16);

		BitArrayA.Combine(BitArrayB, WordOp);

		UE_NET_ASSERT_EQ(ExpectedSingleWordBuffer, SingleWordBufferA);
	}

	// Test full word
	{
		const uint32 ExpectedSingleWordBuffer = 0x11111111;
		uint32 SingleWordBufferA = 0xffffffff;
		uint32 SingleWordBufferB = 0x11111111;

		FNetBitArrayView BitArrayA(&SingleWordBufferA, 32);
		const FNetBitArrayView BitArrayB(&SingleWordBufferB, 32);

		BitArrayA.Combine(BitArrayB, WordOp);

		UE_NET_ASSERT_EQ(ExpectedSingleWordBuffer, SingleWordBufferA);
	}

	// Test Multiple words
	{
		const uint32 ExpectedWordBuffer[] =  { 1u, 0u, 3u, 0u };
		uint32 WordBufferA[] = { 1, 2, 3, 4 };
		uint32 WordBufferB[] = { 1, 0, 3, 0 };

		FNetBitArrayView BitArrayA(&WordBufferA[0], 128);
		const FNetBitArrayView BitArrayB(&WordBufferB[0], 128);

		BitArrayA.Combine(BitArrayB, WordOp);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(ExpectedWordBuffer); ++it)
		{
			UE_NET_ASSERT_EQ(ExpectedWordBuffer[it], WordBufferA[it]);
		}
	}

	// Test Multiple arrays
	{
		const uint32 ExpectedWordBuffer[] = { 0x0001, 0x0010, 0x0100, 0x1000 };
		uint32 WordBufferA[] = { 0x0001, 0x0010, 0x0100, 0x1000 };
		uint32 WordBufferB[] = { 0x0001, 0x0010, 0x0100, 0x1000 };
		uint32 WordBufferC[] = { 0x0001, 0x0010, 0x0100, 0x1000 };

		FNetBitArrayView BitArrayA(&WordBufferA[0], 128);
		const FNetBitArrayView BitArrayB(&WordBufferB[0], 128);
		const FNetBitArrayView BitArrayC(&WordBufferC[0], 128);

		BitArrayA.CombineMultiple(WordOp, BitArrayB, WordOp, BitArrayC);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(ExpectedWordBuffer); ++it)
		{
			UE_NET_ASSERT_EQ(ExpectedWordBuffer[it], WordBufferA[it]);
		}
	}
}

UE_NET_TEST(FNetBitArrayView, AndNot)
{	
	auto&& WordOp = FNetBitArrayView::AndNotOp;

	// Test partial word
	{
		const uint32 ExpectedSingleWordBuffer = 0x0000eeee;
		uint32 SingleWordBufferA = 0xffff;
		uint32 SingleWordBufferB = 0x1111;

		FNetBitArrayView BitArrayA(&SingleWordBufferA, 16);
		const FNetBitArrayView BitArrayB(&SingleWordBufferB, 16);

		BitArrayA.Combine(BitArrayB, WordOp);

		UE_NET_ASSERT_EQ(ExpectedSingleWordBuffer, SingleWordBufferA);
	}

	// Test full word
	{
		const uint32 ExpectedSingleWordBuffer = 0xeeeeeeee;
		uint32 SingleWordBufferA = 0xffffffff;
		uint32 SingleWordBufferB = 0x11111111;

		FNetBitArrayView BitArrayA(&SingleWordBufferA, 32);
		const FNetBitArrayView BitArrayB(&SingleWordBufferB, 32);

		BitArrayA.Combine(BitArrayB, WordOp);

		UE_NET_ASSERT_EQ(ExpectedSingleWordBuffer, SingleWordBufferA);
	}

	// Test Multiple words
	{
		const uint32 ExpectedWordBuffer[] =  { 0, 2, 0, 4 };
		uint32 WordBufferA[] = { 1, 2, 3, 4 };
		uint32 WordBufferB[] = { 1, 0, 3, 0 };

		FNetBitArrayView BitArrayA(&WordBufferA[0], 128);
		const FNetBitArrayView BitArrayB(&WordBufferB[0], 128);

		BitArrayA.Combine(BitArrayB, WordOp);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(ExpectedWordBuffer); ++it)
		{
			UE_NET_ASSERT_EQ(ExpectedWordBuffer[it], WordBufferA[it]);
		}
	}

	// Test Multiple arrays
	{
		const uint32 ExpectedWordBuffer[] = { 0xFFFe, 0xFFeF, 0xFeFF, 0xeFFF };
		uint32 WordBufferA[] = { 0xFFFF, 0xFFFF, 0xFFFF, 0xFFFF };
		uint32 WordBufferB[] = { 0x0001, 0x0010, 0x0100, 0x1000 };
		uint32 WordBufferC[] = { 0x0000, 0x0000, 0x0000, 0x0000 };

		FNetBitArrayView BitArrayA(&WordBufferA[0], 128);
		const FNetBitArrayView BitArrayB(&WordBufferB[0], 128);
		const FNetBitArrayView BitArrayC(&WordBufferC[0], 128);

		BitArrayA.CombineMultiple(WordOp, BitArrayB, WordOp, BitArrayC);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(ExpectedWordBuffer); ++it)
		{
			UE_NET_ASSERT_EQ(ExpectedWordBuffer[it], WordBufferA[it]);
		}
	}
}

UE_NET_TEST(FNetBitArrayView, Xor)
{	
	auto&& WordOp = FNetBitArrayView::XorOp;

	// Test partial word
	{
		const uint32 ExpectedSingleWordBuffer = 0x00006666U;
		uint32 SingleWordBufferA = 0x3030U;
		uint32 SingleWordBufferB = 0x5656U;

		FNetBitArrayView BitArrayA(&SingleWordBufferA, 16);
		const FNetBitArrayView BitArrayB(&SingleWordBufferB, 16);

		BitArrayA.Combine(BitArrayB, WordOp);

		UE_NET_ASSERT_EQ(ExpectedSingleWordBuffer, SingleWordBufferA);
	}

	// Test full word
	{
		const uint32 ExpectedSingleWordBuffer = 0x66666666U;
		uint32 SingleWordBufferA = 0x30303030U;
		uint32 SingleWordBufferB = 0x56565656U;

		FNetBitArrayView BitArrayA(&SingleWordBufferA, 32);
		const FNetBitArrayView BitArrayB(&SingleWordBufferB, 32);

		BitArrayA.Combine(BitArrayB, WordOp);

		UE_NET_ASSERT_EQ(ExpectedSingleWordBuffer, SingleWordBufferA);
	}

	// Test Multiple words
	{
		const uint32 ExpectedWordBuffer[] =  { ~0U, 0, ~0U, };
		uint32 WordBufferA[] = { 0x00303000U, 0x56565656U, ~0U, };
		uint32 WordBufferB[] = { ~0x00303000U, 0x56565656U, 0U, };

		FNetBitArrayView BitArrayA(&WordBufferA[0], 96);
		const FNetBitArrayView BitArrayB(&WordBufferB[0], 96);

		BitArrayA.Combine(BitArrayB, WordOp);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(ExpectedWordBuffer); ++it)
		{
			UE_NET_ASSERT_EQ(ExpectedWordBuffer[it], WordBufferA[it]);
		}
	}

	// Test Multiple arrays
	{
		const uint32 ExpectedWordBuffer[] = { 0xFFFF, 0xFFFF, 0xFFFF, 0xFFFF };
		uint32 WordBufferA[] = { 0xFFee, 0xFeeF, 0xeeFF, 0xeFFe };
		uint32 WordBufferB[] = { 0x0001, 0x0010, 0x0100, 0x1000 };
		uint32 WordBufferC[] = { 0x0010, 0x0100, 0x1000, 0x0001 };

		FNetBitArrayView BitArrayA(&WordBufferA[0], 128);
		const FNetBitArrayView BitArrayB(&WordBufferB[0], 128);
		const FNetBitArrayView BitArrayC(&WordBufferC[0], 128);

		BitArrayA.CombineMultiple(WordOp, BitArrayB, WordOp, BitArrayC);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(ExpectedWordBuffer); ++it)
		{
			UE_NET_ASSERT_EQ(ExpectedWordBuffer[it], WordBufferA[it]);
		}
	}
}

UE_NET_TEST(FNetBitArrayView, TestCombineMultiple)
{
	auto&& FirstWordOp = FNetBitArrayView::OrOp;
	auto&& SecondWordOp = FNetBitArrayView::AndOp;

	// Test Multiple arrays
	{
		uint32 WordBufferA[] = { 0xFF00, 0xF00F, 0x00FF, 0x0FF0 };
		uint32 WordBufferB[] = { 0x0011, 0x0110, 0x1100, 0x1001 };
		uint32 WordBufferC[] = { 0x0010, 0x0100, 0x1000, 0x0001 };
		const uint32 ExpectedWordBuffer[] = { 0xFF10, 0xF10F, 0x10FF, 0x0FF1 };

		FNetBitArrayView BitArrayA(&WordBufferA[0], 128);
		const FNetBitArrayView BitArrayB(&WordBufferB[0], 128);
		const FNetBitArrayView BitArrayC(&WordBufferC[0], 128);

		BitArrayA.CombineMultiple(FirstWordOp, BitArrayB, SecondWordOp, BitArrayC);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(ExpectedWordBuffer); ++it)
		{
			UE_NET_ASSERT_EQ(ExpectedWordBuffer[it], WordBufferA[it]);
		}
	}

	// Test Multiple arrays
	{
		uint32 WordBufferA[] = { 0xFF00, 0xF00F, 0x00FF, 0x0FF0 };
		uint32 WordBufferB[] = { 0x0011, 0x0110, 0x1100, 0x1001 };
		uint32 WordBufferC[] = { 0x0010, 0x0100, 0x1000, 0x0001 };
		const uint32 ExpectedWordBuffer[] = { 0xFF10, 0xF10F, 0x10FF, 0x0FF1 };

		FNetBitArray BitArrayA, BitArrayB, BitArrayC;
		BitArrayA.Init(128);
		BitArrayB.Init(128);
		BitArrayC.Init(128);
		MakeNetBitArrayView(BitArrayA).Copy(FNetBitArrayView(&WordBufferA[0], 128));
		MakeNetBitArrayView(BitArrayB).Copy(FNetBitArrayView(&WordBufferB[0], 128));
		MakeNetBitArrayView(BitArrayC).Copy(FNetBitArrayView(&WordBufferC[0], 128));

		BitArrayA.CombineMultiple(FirstWordOp, BitArrayB, SecondWordOp, BitArrayC);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(ExpectedWordBuffer); ++it)
		{
			UE_NET_ASSERT_EQ(ExpectedWordBuffer[it], BitArrayA.GetData()[it]);
		}
	}
}

UE_NET_TEST(FNetBitArrayView, TestSet)
{
	// Test Set function on BitArrayView
	{
		const uint32 ExpectedWordBuffer[] = { 0x0011, 0x0110, 0x1100, 0x1001 };
		uint32 WordBufferA[] = { 0xFFFF, 0xFFFF, 0xFFFF, 0xFFFF };
		uint32 WordBufferB[] = { 0x0001, 0x0010, 0x0100, 0x1000 };
		uint32 WordBufferC[] = { 0x0010, 0x0100, 0x1000, 0x0001 };

		FNetBitArrayView BitArrayA(&WordBufferA[0], 128);
		const FNetBitArrayView BitArrayB(&WordBufferB[0], 128);
		const FNetBitArrayView BitArrayC(&WordBufferC[0], 128);

		BitArrayA.Set(BitArrayB, FNetBitArrayView::OrOp, BitArrayC);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(ExpectedWordBuffer); ++it)
		{
			UE_NET_ASSERT_EQ(ExpectedWordBuffer[it], BitArrayA.GetData()[it]);
		}
	}

	// Test Set function on BitArray
	{
		const uint32 ExpectedWordBuffer[] = { 0x0011, 0x0110, 0x1100, 0x1001 };
		uint32 WordBufferA[] = { 0xFFFF, 0xFFFF, 0xFFFF, 0xFFFF };
		uint32 WordBufferB[] = { 0x0001, 0x0010, 0x0100, 0x1000 };
		uint32 WordBufferC[] = { 0x0010, 0x0100, 0x1000, 0x0001 };

		FNetBitArray BitArrayA, BitArrayB, BitArrayC;
		BitArrayA.InitAndCopy(FNetBitArrayView(&WordBufferA[0], 128));
		BitArrayB.InitAndCopy(FNetBitArrayView(&WordBufferB[0], 128));
		BitArrayC.InitAndCopy(FNetBitArrayView(&WordBufferC[0], 128));

		BitArrayA.Set(BitArrayB, FNetBitArrayView::OrOp, BitArrayC);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(ExpectedWordBuffer); ++it)
		{
			UE_NET_ASSERT_EQ(ExpectedWordBuffer[it], BitArrayA.GetData()[it]);
		}
	}
}

UE_NET_TEST(FNetBitArrayView, Copy)
{	
	// Test Partial word
	{
		const uint32 ExpectedSingleWordBuffer = 0x0000eeee;
		uint32 SingleWordBufferA = 0xffff;
		uint32 SingleWordBufferB = 0x1111;

		FNetBitArrayView BitArrayA(&SingleWordBufferA, 16);
		const FNetBitArrayView BitArrayB(&SingleWordBufferB, 16);

		UE_NET_ASSERT_NE(SingleWordBufferA, SingleWordBufferB);

		BitArrayA.Copy(BitArrayB);

		UE_NET_ASSERT_EQ(SingleWordBufferA, SingleWordBufferB);
	}

	// Test Multiple words
	{
		uint32 WordBufferA[] = { 1, 2, 3, 4 };
		uint32 WordBufferB[] = { 1, 0, 3, 0 };

		FNetBitArrayView BitArrayA(&WordBufferA[0], 128);
		const FNetBitArrayView BitArrayB(&WordBufferB[0], 128);

		BitArrayA.Copy(BitArrayB);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(WordBufferA); ++it)
		{
			UE_NET_ASSERT_EQ(WordBufferB[it], WordBufferA[it]);
		}
	}

	// Test View copying BitArray
	{
		uint32 WordBufferA[] = { 1, 2, 3, 4 };
		FNetBitArrayView BitArrayA(&WordBufferA[0], 128);
		
		const uint32 WordBufferB[] = { 5, 6, 7, 8 };
		FNetBitArray BitArrayB;
		BitArrayB.Init(128);
		BitArrayB.GetData()[0] = WordBufferB[0];
		BitArrayB.GetData()[1] = WordBufferB[1];
		BitArrayB.GetData()[2] = WordBufferB[2];
		BitArrayB.GetData()[3] = WordBufferB[3];

		BitArrayA.Copy(BitArrayB);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(WordBufferA); ++it)
		{
			UE_NET_ASSERT_EQ(BitArrayB.GetData()[it], WordBufferA[it]);
		}
	}

	// Test BitArray copying View
	{
		uint32 WordBufferA[] = { 1, 2, 3, 4 };
		FNetBitArrayView BitArrayA(&WordBufferA[0], 128);

		const uint32 WordBufferB[] = { 5, 6, 7, 8 };
		FNetBitArray BitArrayB;
		BitArrayB.Init(128);
		BitArrayB.GetData()[0] = WordBufferB[0];
		BitArrayB.GetData()[1] = WordBufferB[1];
		BitArrayB.GetData()[2] = WordBufferB[2];
		BitArrayB.GetData()[3] = WordBufferB[3];

		BitArrayB.Copy(BitArrayA);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(WordBufferA); ++it)
		{
			UE_NET_ASSERT_EQ(BitArrayB.GetData()[it], WordBufferA[it]);
		}
	}
}

UE_NET_TEST(FNetBitArrayView, InitAndCopy)
{
	{
		uint32 WordBufferA[] = { 1, 2, 3, 4 };
		const FNetBitArrayView BitArrayA(&WordBufferA[0], 128);

		FNetBitArray TestBitArrayA;
		TestBitArrayA.InitAndCopy(BitArrayA);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(WordBufferA); ++it)
		{
			UE_NET_ASSERT_EQ(TestBitArrayA.GetData()[it], WordBufferA[it]);
		}
	}

	{
		uint32 WordBufferB[] = { 5, 6, 7, 8 };
		FNetBitArray BitArrayB;
		BitArrayB.Init(128);
		BitArrayB.GetData()[0] = WordBufferB[0];
		BitArrayB.GetData()[1] = WordBufferB[1];
		BitArrayB.GetData()[2] = WordBufferB[2];
		BitArrayB.GetData()[3] = WordBufferB[3];

		FNetBitArray TestBitArrayB;
		TestBitArrayB.InitAndCopy(BitArrayB);

		for (uint32 it = 0; it < UE_ARRAY_COUNT(WordBufferB); ++it)
		{
			UE_NET_ASSERT_EQ(TestBitArrayB.GetData()[it], WordBufferB[it]);
		}
	}
}

UE_NET_TEST(FNetBitArrayView, PrintHelper)
{
	{
		uint32 WordBufferA[] = { 0xFF00, 0xF00F, 0x00FF, 0x0FF0 };

		FNetBitArray BitArrayB;
		BitArrayB.Init(128);
		BitArrayB.GetData()[0] = WordBufferA[0];
		BitArrayB.GetData()[1] = WordBufferA[1];
		BitArrayB.GetData()[2] = WordBufferA[2];
		BitArrayB.GetData()[3] = WordBufferA[3];
		
		const FNetBitArrayView BitArrayA(&WordBufferA[0], 128);
		FString PrintBufferA;
		FString PrintBufferB;

		PrintBufferA = FNetBitArrayPrinter::PrintSetBits(BitArrayA);
		PrintBufferB = FNetBitArrayPrinter::PrintSetBits(BitArrayB);
		UE_NET_ASSERT_EQ(PrintBufferA, PrintBufferB);

		PrintBufferA = FNetBitArrayPrinter::PrintSetSummary(BitArrayA);
		PrintBufferB = FNetBitArrayPrinter::PrintSetSummary(BitArrayB);
		UE_NET_ASSERT_EQ(PrintBufferA, PrintBufferB);

		PrintBufferA = BitArrayA.ToString();
		PrintBufferB = BitArrayB.ToString();
		UE_NET_ASSERT_EQ(PrintBufferA, PrintBufferB);

		PrintBufferA = FNetBitArrayPrinter::PrintZeroBits(BitArrayA);
		PrintBufferB = FNetBitArrayPrinter::PrintZeroBits(BitArrayB);
		UE_NET_ASSERT_EQ(PrintBufferA, PrintBufferB);
		
		PrintBufferA = FNetBitArrayPrinter::PrintZeroSummary(BitArrayA);
		PrintBufferB = FNetBitArrayPrinter::PrintZeroSummary(BitArrayB);
		UE_NET_ASSERT_EQ(PrintBufferA, PrintBufferB);

		uint32 WordBufferDelta[] = { 0x0011, 0xF00F, 0x1100, 0x0FF0 };
		const FNetBitArrayView BitArrayDelta(&WordBufferDelta[0], 128);

		PrintBufferA = FNetBitArrayPrinter::PrintDeltaSummary(BitArrayA, BitArrayDelta);
		PrintBufferB = FNetBitArrayPrinter::PrintDeltaSummary(BitArrayDelta, BitArrayB);
		UE_NET_ASSERT_EQ(PrintBufferA, PrintBufferB);

		PrintBufferA = FNetBitArrayPrinter::PrintDeltaBits(BitArrayA, BitArrayDelta);
		PrintBufferB = FNetBitArrayPrinter::PrintDeltaBits(BitArrayB, BitArrayDelta);
		UE_NET_ASSERT_EQ(PrintBufferA, PrintBufferB);
	}
}

UE_NET_TEST(FNetBitArrayView, TestRoundUpToMaxWord)
{
	{
		const uint32 NumBits = 0;
		uint32 RoundUpZero = FNetBitArray::RoundUpToMaxWordBitCount(NumBits);
		UE_NET_ASSERT_TRUE(RoundUpZero > NumBits);
		UE_NET_ASSERT_TRUE((RoundUpZero % FNetBitArray::WordBitCount) == 0);

		FNetBitArray BitArray(RoundUpZero);
		UE_NET_ASSERT_TRUE(BitArray.GetNumWords() == 1);
		UE_NET_ASSERT_TRUE(BitArray.GetNumBits() == RoundUpZero);
	}

	{
		const uint32 NumBits = 32;
		uint32 RoundUpWord = FNetBitArray::RoundUpToMaxWordBitCount(NumBits);
		UE_NET_ASSERT_TRUE(RoundUpWord == NumBits);
		UE_NET_ASSERT_TRUE((RoundUpWord % FNetBitArray::WordBitCount) == 0);

		FNetBitArray BitArray(RoundUpWord);
		UE_NET_ASSERT_TRUE(BitArray.GetNumWords() == 1);
		UE_NET_ASSERT_TRUE(BitArray.GetNumBits() == RoundUpWord);
	}

	{
		const uint32 NumBits = 48;
		uint32 RoundUpTwoWords = FNetBitArray::RoundUpToMaxWordBitCount(NumBits);
		UE_NET_ASSERT_TRUE(RoundUpTwoWords > NumBits);
		UE_NET_ASSERT_TRUE((RoundUpTwoWords % FNetBitArray::WordBitCount) == 0);

		FNetBitArray BitArray(RoundUpTwoWords);
		UE_NET_ASSERT_TRUE(BitArray.GetNumWords() == 2);
		UE_NET_ASSERT_TRUE(BitArray.GetNumBits() == RoundUpTwoWords);
	}
}

class FNetBitArrayViewFixture : public FNetworkAutomationTestSuiteFixture
{
public:
	virtual void SetUp() override
	{
	}

	virtual void TearDown() override
	{
	}

	struct FCollectSetBitsFunctor
	{
		void operator()(uint32 Index) { Invoked.Add(Index); }
		TArray<uint32> Invoked;
	};

	void VerifyResult(const FCollectSetBitsFunctor& Collector, const FNetBitArrayView& TrueForIndexBitArray, const FNetBitArrayView& FalseForIndexBitArray)
	{
		// verify A
		for (uint32 It = 0; It < (uint32)Collector.Invoked.Num(); ++It)
		{
			const uint32 Index = Collector.Invoked[It];
			UE_NET_ASSERT_TRUE(TrueForIndexBitArray.GetBit(Index));
			UE_NET_ASSERT_FALSE(FalseForIndexBitArray.GetBit(Index));
		}
	}
};

UE_NET_TEST_FIXTURE(FNetBitArrayViewFixture, TestForAllExclusiveBits)
{
	{
		uint32 ExpectedOnlyInABits = 0x0000aaaa;
		uint32 ExpectedOnlyInBBits = 0x00bb0000;
		uint32 ExpectedBitsSetInAB = ~(ExpectedOnlyInABits ^ ExpectedOnlyInBBits) & 0x00ffffff;

		FNetBitArrayView BitArrayA(&ExpectedOnlyInABits, 24);
		FNetBitArrayView BitArrayB(&ExpectedOnlyInBBits, 24);
		FNetBitArrayView BitArrayAB(&ExpectedBitsSetInAB, 24);

		BitArrayA.Combine(BitArrayAB, FNetBitArrayView::OrOp);
		BitArrayB.Combine(BitArrayAB, FNetBitArrayView::OrOp);

		FCollectSetBitsFunctor ACollector, BCollector;
		FNetBitArrayView::ForAllExclusiveBits(BitArrayA, BitArrayB, ACollector, BCollector);

		// Verify result
		VerifyResult(ACollector, BitArrayA, BitArrayB);
		VerifyResult(BCollector, BitArrayB, BitArrayA);
	}

	{
		uint32 ExpectedOnlyInABits = 0x0000aaaa;
		uint32 ExpectedOnlyInBBits = 0xbbbb0000;
		uint32 ExpectedBitsSetInAB = ~(ExpectedOnlyInABits ^ ExpectedOnlyInBBits);

		FNetBitArrayView BitArrayA(&ExpectedOnlyInABits, 32);
		FNetBitArrayView BitArrayB(&ExpectedOnlyInBBits, 32);
		FNetBitArrayView BitArrayAB(&ExpectedBitsSetInAB, 32);

		BitArrayA.Combine(BitArrayAB, FNetBitArrayView::OrOp);
		BitArrayB.Combine(BitArrayAB, FNetBitArrayView::OrOp);

		FCollectSetBitsFunctor ACollector, BCollector;
		FNetBitArrayView::ForAllExclusiveBits(BitArrayA, BitArrayB, ACollector, BCollector);

		// Verify result
		VerifyResult(ACollector, BitArrayA, BitArrayB);
		VerifyResult(BCollector, BitArrayB, BitArrayA);
	}

	{
		uint32 ExpectedOnlyInABits[] = {0x0000aaaa, 0x01010101, 0x2e2e2e2e };
		uint32 ExpectedOnlyInBBits[] = {0xbbbb0000, 0x10101010, 0x10101010 };
		uint32 ExpectedBitsSetInAB[] = {~(ExpectedOnlyInABits[0] ^ ExpectedOnlyInBBits[0]), ~(ExpectedOnlyInABits[1] ^ ExpectedOnlyInBBits[1]), 0 };

		FNetBitArrayView BitArrayA(ExpectedOnlyInABits, 96);
		FNetBitArrayView BitArrayB(ExpectedOnlyInBBits, 96);
		FNetBitArrayView BitArrayAB(ExpectedBitsSetInAB, 96);

		BitArrayA.Combine(BitArrayAB, FNetBitArrayView::OrOp);
		BitArrayB.Combine(BitArrayAB, FNetBitArrayView::OrOp);

		FCollectSetBitsFunctor ACollector, BCollector;
		FNetBitArrayView::ForAllExclusiveBits(BitArrayA, BitArrayB, ACollector, BCollector);

		// Verify result
		VerifyResult(ACollector, BitArrayA, BitArrayB);
		VerifyResult(BCollector, BitArrayB, BitArrayA);
	}

	{
		uint32 ExpectedOnlyInABits[] = {0x0000aaaa, 0x01010101, 0x002e2e2e };
		uint32 ExpectedOnlyInBBits[] = {0xbbbb0000, 0x10101010, 0x00101010 };
		uint32 ExpectedBitsSetInAB[] = {~(ExpectedOnlyInABits[0] ^ ExpectedOnlyInBBits[0]), ~(ExpectedOnlyInABits[1] ^ ExpectedOnlyInBBits[1]), 0 };

		FNetBitArrayView BitArrayA(ExpectedOnlyInABits, 88);
		FNetBitArrayView BitArrayB(ExpectedOnlyInBBits, 88);
		FNetBitArrayView BitArrayAB(ExpectedBitsSetInAB, 88);

		BitArrayA.Combine(BitArrayAB, FNetBitArrayView::OrOp);
		BitArrayB.Combine(BitArrayAB, FNetBitArrayView::OrOp);

		FCollectSetBitsFunctor ACollector, BCollector;
		FNetBitArrayView::ForAllExclusiveBits(BitArrayA, BitArrayB, ACollector, BCollector);

		// Verify result
		VerifyResult(ACollector, BitArrayA, BitArrayB);
		VerifyResult(BCollector, BitArrayB, BitArrayA);
	}
}

UE_NET_TEST_FIXTURE(FNetBitArrayViewFixture, TestForAllExclusiveBitsByPredicate)
{
	// Test predicate that returns true which should behave identically to not having a predicate.
	{
		uint32 ExpectedOnlyInABits = 0x0000aaaa;
		uint32 ExpectedOnlyInBBits = 0x00bb0000;
		uint32 ExpectedBitsSetInAB = ~(ExpectedOnlyInABits ^ ExpectedOnlyInBBits) & 0x00ffffff;

		FNetBitArrayView BitArrayA(&ExpectedOnlyInABits, 24);
		FNetBitArrayView BitArrayB(&ExpectedOnlyInBBits, 24);
		FNetBitArrayView BitArrayAB(&ExpectedBitsSetInAB, 24);

		BitArrayA.Combine(BitArrayAB, FNetBitArrayView::OrOp);
		BitArrayB.Combine(BitArrayAB, FNetBitArrayView::OrOp);

		FCollectSetBitsFunctor ACollector, BCollector;
		constexpr auto AllowAll = [](uint32)
			{
				return true;
			};
		FNetBitArrayView::ForAllExclusiveBitsByPredicate(BitArrayA, BitArrayB, ACollector, BCollector, AllowAll);

		// Verify result
		VerifyResult(ACollector, BitArrayA, BitArrayB);
		VerifyResult(BCollector, BitArrayB, BitArrayA);
	}

	// Test predicate that returns false which should cause no functor to be called.
	{
		uint32 ExpectedOnlyInABits = 0x0000aaaa;
		uint32 ExpectedOnlyInBBits = 0x00bb0000;
		uint32 ExpectedBitsSetInAB = ~(ExpectedOnlyInABits ^ ExpectedOnlyInBBits) & 0x00ffffff;

		FNetBitArrayView BitArrayA(&ExpectedOnlyInABits, 24);
		FNetBitArrayView BitArrayB(&ExpectedOnlyInBBits, 24);
		FNetBitArrayView BitArrayAB(&ExpectedBitsSetInAB, 24);

		BitArrayA.Combine(BitArrayAB, FNetBitArrayView::OrOp);
		BitArrayB.Combine(BitArrayAB, FNetBitArrayView::OrOp);

		FCollectSetBitsFunctor ACollector, BCollector;
		constexpr auto AllowNone = [](uint32)
			{
				return false;
			};
		FNetBitArrayView::ForAllExclusiveBitsByPredicate(BitArrayA, BitArrayB, ACollector, BCollector, AllowNone);

		UE_NET_ASSERT_TRUE(ACollector.Invoked.IsEmpty());
		UE_NET_ASSERT_TRUE(BCollector.Invoked.IsEmpty());
	}

	// Test predicate that only allows odd bit indices.
	{
		uint32 ExpectedOnlyInABits = 0x0000aaaa;
		uint32 ExpectedOnlyInBBits = 0x00bb0000;
		uint32 ExpectedBitsSetInAB = ~(ExpectedOnlyInABits ^ ExpectedOnlyInBBits) & 0x00ffffff;

		FNetBitArrayView BitArrayA(&ExpectedOnlyInABits, 24);
		FNetBitArrayView BitArrayB(&ExpectedOnlyInBBits, 24);
		FNetBitArrayView BitArrayAB(&ExpectedBitsSetInAB, 24);

		BitArrayA.Combine(BitArrayAB, FNetBitArrayView::OrOp);
		BitArrayB.Combine(BitArrayAB, FNetBitArrayView::OrOp);

		FCollectSetBitsFunctor ACollector, BCollector;
		auto AllowOdd = [](uint32 BitIndex)
			{
				return (BitIndex & 1U) == 1U;
			};
		FNetBitArrayView::ForAllExclusiveBitsByPredicate(BitArrayA, BitArrayB, ACollector, BCollector, AllowOdd);

		// Without odd bits being exclusive to one bit array this test is useless.
		UE_NET_ASSERT_GT(ACollector.Invoked.Num(), 0);
		UE_NET_ASSERT_GT(BCollector.Invoked.Num(), 0);

		for (const uint32 BitIndex : ACollector.Invoked)
		{
			UE_NET_ASSERT_EQ((BitIndex & 1U), 1U);
		}

		for (const uint32 BitIndex : BCollector.Invoked)
		{
			UE_NET_ASSERT_EQ((BitIndex & 1U), 1U);
		}
	}

	// Test predicate that only allows even bit indices.
	{
		uint32 ExpectedOnlyInABits = 0x0000aabb;
		uint32 ExpectedOnlyInBBits = 0x00bb0004;
		uint32 ExpectedBitsSetInAB = ~(ExpectedOnlyInABits ^ ExpectedOnlyInBBits) & 0x00ffffff;

		FNetBitArrayView BitArrayA(&ExpectedOnlyInABits, 24);
		FNetBitArrayView BitArrayB(&ExpectedOnlyInBBits, 24);
		FNetBitArrayView BitArrayAB(&ExpectedBitsSetInAB, 24);

		BitArrayA.Combine(BitArrayAB, FNetBitArrayView::OrOp);
		BitArrayB.Combine(BitArrayAB, FNetBitArrayView::OrOp);

		FCollectSetBitsFunctor ACollector, BCollector;
		auto AllowEven = [](uint32 BitIndex)
			{
				return (BitIndex & 1U) == 0U;
			};
		FNetBitArrayView::ForAllExclusiveBitsByPredicate(BitArrayA, BitArrayB, ACollector, BCollector, AllowEven);

		// Without even bits being exclusive to one bit array this test is useless.
		UE_NET_ASSERT_GT(ACollector.Invoked.Num(), 0);
		UE_NET_ASSERT_GT(BCollector.Invoked.Num(), 0);

		for (const uint32 BitIndex : ACollector.Invoked)
		{
			UE_NET_ASSERT_EQ((BitIndex & 1U), 0U);
		}

		for (const uint32 BitIndex : BCollector.Invoked)
		{
			UE_NET_ASSERT_EQ((BitIndex & 1U), 0U);
		}
	}
}

class FNetBitArrayViewTestForAllSetBitsFixture : public FNetworkAutomationTestSuiteFixture
{
public:
	virtual void SetUp() override
	{
	}

	virtual void TearDown() override
	{
	}

	struct FCollectSetBitsFunctor
	{
		void operator()(uint32 Index) { Invoked.Add(Index); }
		TArray<uint32> Invoked;
	};

	void VerifyResult(const FCollectSetBitsFunctor& Collector, const FCollectSetBitsFunctor& Expected)
	{
		UE_NET_ASSERT_EQ(Collector.Invoked.Num(), Expected.Invoked.Num());

		// verify A
		for (uint32 It = 0; It < (uint32)Collector.Invoked.Num(); ++It)
		{
			UE_NET_ASSERT_EQ(Collector.Invoked[It], Expected.Invoked[It]);
		}
	}

	template <typename Op>
	void TestOp(Op && Operand)
	{
		{
			uint32 ABits = 0x0011aaa1;
			uint32 BBits = 0x00ff0001;

			uint32 ExpectedBits = Operand(ABits, BBits);

			FCollectSetBitsFunctor ExpectedCollector;
			FNetBitArrayView BitArrayExpected(&ExpectedBits, 24, FNetBitArrayView::NoResetNoValidate);
			BitArrayExpected.ForAllSetBits(ExpectedCollector);

			FNetBitArrayView BitArrayA(&ABits, 24);
			FNetBitArrayView BitArrayB(&BBits, 24);

			FCollectSetBitsFunctor OpCollector;
	
			FNetBitArrayView::ForAllSetBits(BitArrayA, BitArrayB, Operand, OpCollector);

			// Verify result
			VerifyResult(OpCollector, ExpectedCollector);
		}

		{
			uint32 ABits = 0x1011aaa1;
			uint32 BBits = 0x10ff0001;

			uint32 ExpectedBits = Operand(ABits, BBits);

			FCollectSetBitsFunctor ExpectedCollector;
			FNetBitArrayView BitArrayExpected(&ExpectedBits, 32, FNetBitArrayView::NoResetNoValidate);
			BitArrayExpected.ForAllSetBits(ExpectedCollector);

			FNetBitArrayView BitArrayA(&ABits, 32);
			FNetBitArrayView BitArrayB(&BBits, 32);

			FCollectSetBitsFunctor OpCollector;
	
			FNetBitArrayView::ForAllSetBits(BitArrayA, BitArrayB, Operand, OpCollector);

			// Verify result
			VerifyResult(OpCollector, ExpectedCollector);
		}

		{
			uint32 ABits[] = { 0x0011aaa1, 0xffffffff, 0x0, 0x7fff0000 };
			uint32 BBits[] = { 0x00ff0001, 0xffffffff, 0x0, 0x0 };

			uint32 ExpectedBits[] = { Operand(ABits[0], BBits[0]), Operand(ABits[1], BBits[1]), Operand(ABits[2], BBits[2]), Operand(ABits[3], BBits[3]) };

			FCollectSetBitsFunctor ExpectedCollector;
			FNetBitArrayView BitArrayExpected(ExpectedBits, 127, FNetBitArrayView::NoResetNoValidate);
			BitArrayExpected.ForAllSetBits(ExpectedCollector);

			FNetBitArrayView BitArrayA(ABits, 127);
			FNetBitArrayView BitArrayB(BBits, 127);

			FCollectSetBitsFunctor OpCollector;
	
			FNetBitArrayView::ForAllSetBits(BitArrayA, BitArrayB, Operand, OpCollector);

			// Verify result
			VerifyResult(OpCollector, ExpectedCollector);
		}
	}
};

UE_NET_TEST_FIXTURE(FNetBitArrayViewTestForAllSetBitsFixture, TestOps)
{
	// And
	{
		uint32 ABits = 0x100ff001;
		uint32 BBits = 0x1ff00ff1;

		uint32 ExpectedBits = ABits & BBits;

		UE_NET_ASSERT_EQ(ExpectedBits, FNetBitArrayView::AndOp(ABits, BBits));
	}

	// AndNot
	{
		uint32 ABits = 0x100ff001;
		uint32 BBits = 0x1ff00ff1;

		uint32 ExpectedBits = ABits & ~BBits;

		UE_NET_ASSERT_EQ(ExpectedBits, FNetBitArrayView::AndNotOp(ABits, BBits));
	}

	// Or
	{
		uint32 ABits = 0xffff0000;
		uint32 BBits = 0x0000ffff;

		uint32 ExpectedBits = ABits | BBits;

		UE_NET_ASSERT_EQ(ExpectedBits, FNetBitArrayView::OrOp(ABits, BBits));
	}

	// Xor
	{
		uint32 ABits = 0x30303030;
		uint32 BBits = 0x56565656;

		uint32 ExpectedBits = ABits ^ BBits;

		UE_NET_ASSERT_EQ(ExpectedBits, FNetBitArrayView::XorOp(ABits, BBits));
	}
}

UE_NET_TEST_FIXTURE(FNetBitArrayViewTestForAllSetBitsFixture, TestForAllSetOpBits)
{
	TestOp(FNetBitArrayView::AndOp);
	TestOp(FNetBitArrayView::AndNotOp);
	TestOp(FNetBitArrayView::OrOp);
	TestOp(FNetBitArrayView::XorOp);
}

// NetBitArray specific tests. Doesn't test everything the BitArrayView tests as implementations are generally identical.
class FNetBitArrayFixture : public FNetworkAutomationTestSuiteFixture
{
protected:
	static bool VerifyZeroedStorage(const FNetBitArray& BitArray)
	{
		const FNetBitArray::StorageWordType* Storage = BitArray.GetData();
		const uint32 WordCount = BitArray.GetNumWords();
		FNetBitArray::StorageWordType Bits = 0U;
		for (SIZE_T WordIt = 0; WordIt != WordCount; ++WordIt)
		{
			Bits |= Storage[WordIt];
		}

		return Bits == 0U;
	}
};

UE_NET_TEST_FIXTURE(FNetBitArrayFixture, Construct)
{
	// Empty array
	{
		FNetBitArray BitArray;
		UE_NET_ASSERT_EQ(BitArray.GetNumBits(), 0U);
		UE_NET_ASSERT_EQ(BitArray.GetNumWords(), 0U);
		UE_NET_ASSERT_TRUE(VerifyZeroedStorage(BitArray));
	}

	// Single word array
	{
		constexpr uint32 BitCount = 3U;
		FNetBitArray BitArray(BitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumBits(), BitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumWords(), 1U);
		UE_NET_ASSERT_TRUE(VerifyZeroedStorage(BitArray));
	}

	// Multi word array
	{
		constexpr uint32 BitCount = 333U;
		FNetBitArray BitArray(BitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumBits(), BitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumWords(), (BitCount + FNetBitArray::WordBitCount - 1U)/FNetBitArray::WordBitCount);
		UE_NET_ASSERT_TRUE(VerifyZeroedStorage(BitArray));
	}
}

UE_NET_TEST_FIXTURE(FNetBitArrayFixture, SetNumBits)
{
	// Start with empty array
	{
		constexpr uint32 BitCount = 333U;
		FNetBitArray BitArray;
		BitArray.SetNumBits(BitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumBits(), BitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumWords(), (BitCount + FNetBitArray::WordBitCount - 1U)/FNetBitArray::WordBitCount);
		UE_NET_ASSERT_TRUE(VerifyZeroedStorage(BitArray));
	}

	// Start with non-empty array and grow
	{
		constexpr uint32 BitCount = 333U;
		FNetBitArray BitArray(3U);
		BitArray.SetNumBits(BitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumBits(), BitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumWords(), (BitCount + FNetBitArray::WordBitCount - 1U)/FNetBitArray::WordBitCount);
		UE_NET_ASSERT_TRUE(VerifyZeroedStorage(BitArray));
	}

	// Start with non-empty array and shrink
	{
		constexpr uint32 BitCount = 3U;
		FNetBitArray BitArray(333U);
		BitArray.SetNumBits(BitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumBits(), BitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumWords(), (BitCount + FNetBitArray::WordBitCount - 1U)/FNetBitArray::WordBitCount);
		UE_NET_ASSERT_TRUE(VerifyZeroedStorage(BitArray));
	}
}

UE_NET_TEST_FIXTURE(FNetBitArrayFixture, AddBits)
{
	// Start with empty array
	{
		constexpr uint32 AdditionalBitCount = 333U;
		FNetBitArray BitArray;
		BitArray.AddBits(AdditionalBitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumBits(), AdditionalBitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumWords(), (AdditionalBitCount + FNetBitArray::WordBitCount - 1U)/FNetBitArray::WordBitCount);
		UE_NET_ASSERT_TRUE(VerifyZeroedStorage(BitArray));
	}

	// Start with non-empty array and grow
	{
		constexpr uint32 OriginalBitCount = 3U;
		constexpr uint32 AdditionalBitCount = 333U;
		FNetBitArray BitArray(OriginalBitCount);
		BitArray.AddBits(AdditionalBitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumBits(), OriginalBitCount + AdditionalBitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumWords(), (OriginalBitCount + AdditionalBitCount + FNetBitArray::WordBitCount - 1U)/FNetBitArray::WordBitCount);
		UE_NET_ASSERT_TRUE(VerifyZeroedStorage(BitArray));
	}

	// Start with non-empty array and add nothing
	{
		constexpr uint32 OriginalBitCount = 333U;
		constexpr uint32 AdditionalBitCount = 0U;
		FNetBitArray BitArray(OriginalBitCount);
		BitArray.AddBits(AdditionalBitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumBits(), OriginalBitCount + AdditionalBitCount);
		UE_NET_ASSERT_EQ(BitArray.GetNumWords(), (OriginalBitCount + AdditionalBitCount + FNetBitArray::WordBitCount - 1U)/FNetBitArray::WordBitCount);
		UE_NET_ASSERT_TRUE(VerifyZeroedStorage(BitArray));
	}
}

UE_NET_TEST_FIXTURE(FNetBitArrayFixture, PaddingBitsAreCleared)
{
	{
		FNetBitArray BitArray(333U);
		BitArray.SetAllBits();
		BitArray.SetNumBits(1U);
		BitArray.ClearBit(0U);
		UE_NET_ASSERT_TRUE(VerifyZeroedStorage(BitArray));
	}
}

UE_NET_TEST_FIXTURE(FNetBitArrayFixture, MakeNetBitArrayView)
{
	{
		constexpr uint32 BitCount = 333U;
		FNetBitArray BitArray(333U);

		FNetBitArrayView BitArrayView = MakeNetBitArrayView(BitArray);
		UE_NET_ASSERT_EQ(BitArray.GetData(), BitArrayView.GetData());
		UE_NET_ASSERT_EQ(BitArray.GetNumBits(), BitArrayView.GetNumBits());
	}
}

UE_NET_TEST_FIXTURE(FNetBitArrayFixture, Test_NetBitArray_GetWord)
{
	constexpr uint32 WordCount = 16;
	FNetBitArray BitArray;
	BitArray.Init(WordCount * FNetBitArray::WordBitCount);

	for (uint32 WordIt=0; WordIt < WordCount; ++WordIt)
	{
		UE_NET_ASSERT_TRUE(BitArray.GetWord(WordIt) == 0x00);
		BitArray.GetWord(WordIt) = 0xAA;
		UE_NET_ASSERT_TRUE(BitArray.GetWord(WordIt) == 0xAA);
	}

	FNetBitArrayView BitArrayView = MakeNetBitArrayView(BitArray);
	for (uint32 WordIt = 0; WordIt < WordCount; ++WordIt)
	{
		UE_NET_ASSERT_TRUE(BitArrayView.GetWord(WordIt) == 0xAA);
		BitArrayView.GetWord(WordIt) = 0xBB;
		UE_NET_ASSERT_TRUE(BitArrayView.GetWord(WordIt) == 0xBB);
	}
}

UE_NET_TEST_FIXTURE(FNetBitArrayFixture, Test_NetBitArray_GetDataChecked)
{
	constexpr uint32 WordCount = 16;
	FNetBitArray BitArray;
	BitArray.Init(WordCount * FNetBitArray::WordBitCount);

	{
		FNetBitArray::StorageWordType* RawData = BitArray.GetDataChecked(WordCount);
		for (uint32 WordIt = 0; WordIt < WordCount; ++WordIt)
		{
			UE_NET_ASSERT_TRUE(RawData[WordIt] == 0x00);
			RawData[WordIt] = 0xAA;
			UE_NET_ASSERT_TRUE(BitArray.GetWord(WordIt) == 0xAA);
		}
	}

	FNetBitArrayView BitArrayView = MakeNetBitArrayView(BitArray);
	{
		FNetBitArray::StorageWordType* RawData = BitArrayView.GetDataChecked(WordCount);
		for (uint32 WordIt = 0; WordIt < WordCount; ++WordIt)
		{
			UE_NET_ASSERT_TRUE(RawData[WordIt] == 0xAA);
			RawData[WordIt] = 0xBB;
			UE_NET_ASSERT_TRUE(BitArrayView.GetWord(WordIt) == 0xBB);
		}
	}
}

UE_NET_TEST(FNetBitArrayConstRangeIterator, CanIterateOverEmptyBitArray)
{
	bool bIteratedOverEmptyArray = false;
	FNetBitArrayView Empty;
	for (const uint32 Index : Empty)
	{
		bIteratedOverEmptyArray = true;
	}

	UE_NET_ASSERT_FALSE(bIteratedOverEmptyArray);
}

UE_NET_TEST(FNetBitArrayConstRangeIterator, CanIterateOverArbitrailySizedBitArrayWithAllBitsSet)
{
	FNetBitArrayBase::StorageWordType WordBuffer[] = { ~0U, ~0U, ~0U, ~0U, ~0U, ~0U, ~0U, };

	constexpr uint32 ArraySize = 126U;
	FNetBitArrayView BitArray(WordBuffer, ArraySize, FNetBitArrayView::NoResetNoValidate);
	uint32 It = 0;
	for (const uint32 Index : BitArray)
	{
		// Make sure we're hitting the right indices.
		UE_NET_ASSERT_EQ(Index, It);
		++It;
	}

	UE_NET_ASSERT_EQ(It, ArraySize);
}

UE_NET_TEST(FNetBitArrayConstRangeIterator, CanIterateOverArbitrailySizedBitArrayWithArbitraryBitsSet)
{
	constexpr uint32 ArraySize = 128U;
	FNetBitArray BitArray(ArraySize);

	const uint32 IndicesToSet[] = { 65, 68 };
	for (uint32 IndexToSet : IndicesToSet)
	{
		BitArray.SetBit(IndexToSet);
	}

	uint32 It = 0;
	for (const uint32 Index : BitArray)
	{
		// Make sure we're hitting the right indices.
		UE_NET_ASSERT_EQ(Index, IndicesToSet[It]);
		++It;
	}

	UE_NET_ASSERT_EQ(It, (uint32)UE_ARRAY_COUNT(IndicesToSet));
}

} // end namespace UE::Net::Private
