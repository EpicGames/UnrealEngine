// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/Core/NetChunkedArray.h"

namespace UE::Net::Private
{
	void InitializeTestArrayValue(TNetChunkedArray<int32>& Arr)
	{
		for (int32 Index = 0; Index < Arr.Num(); Index++)
		{
			Arr[Index] = Index;
		}
	}

	UE_NET_TEST(TNetChunkedArray, MemoryInitializationTypes)
	{
		struct TestStruct
		{
			TestStruct() : VarA(10), VarB(20) {};
			int32 VarA;
			int32 VarB;
		};

		// Test that the default constructor is called for the structure.
		TNetChunkedArray<TestStruct> ArrStruct1(100, EInitMemory::Constructor);
		for (int32 Index = 0; Index < ArrStruct1.Num(); Index++)
		{
			UE_NET_ASSERT_EQ(ArrStruct1[Index].VarA, 10);
			UE_NET_ASSERT_EQ(ArrStruct1[Index].VarB, 20);
		}

		// Test that memory is zeroed out.
		TNetChunkedArray<TestStruct> ArrStruct2(100, EInitMemory::Zero);
		for (int32 Index = 0; Index < ArrStruct2.Num(); Index++)
		{
			UE_NET_ASSERT_EQ(ArrStruct2[Index].VarA, 0);
			UE_NET_ASSERT_EQ(ArrStruct2[Index].VarB, 0);
		}
	}

	UE_NET_TEST(TNetChunkedArray, CopySemantics)
	{
		// An empty test array.
		TNetChunkedArray<int32> ArrEmpty(0, EInitMemory::Zero);

		// A test array with pre-allocated and dynamically allocated chunks.
		TNetChunkedArray<int32> ArrChunked(200);
		ArrChunked.AddToIndexZeroed(310);
		InitializeTestArrayValue(ArrChunked);
			
		// Copy constructor.
		{
			TNetChunkedArray<int32> CopyEmpty(ArrEmpty);
			UE_NET_ASSERT_EQ(0, CopyEmpty.Num());
			UE_NET_ASSERT_EQ(ArrEmpty.Capacity(), CopyEmpty.Capacity());

			TNetChunkedArray<int32> CopyChunked(ArrChunked);
			UE_NET_ASSERT_EQ(ArrChunked.Num(), CopyChunked.Num());
			UE_NET_ASSERT_EQ(ArrChunked.Capacity(), CopyChunked.Capacity());
			for (int32 Index = 0; Index < ArrChunked.Num(); Index++)
			{
				UE_NET_ASSERT_EQ(ArrChunked[Index], CopyChunked[Index]);
			}
		}

		// Copy assignment operator.
		{
			TNetChunkedArray<int32> CopyEmpty;
			CopyEmpty = ArrEmpty;
			UE_NET_ASSERT_EQ(0, CopyEmpty.Num());
			UE_NET_ASSERT_EQ(ArrEmpty.Capacity(), CopyEmpty.Capacity());

			TNetChunkedArray<int32> CopyChunked;
			CopyChunked = ArrChunked;
			UE_NET_ASSERT_EQ(ArrChunked.Num(), CopyChunked.Num());
			UE_NET_ASSERT_EQ(ArrChunked.Capacity(), CopyChunked.Capacity());
			for (int32 Index = 0; Index < ArrChunked.Num(); Index++)
			{
				UE_NET_ASSERT_EQ(ArrChunked[Index], CopyChunked[Index]);
			}
		}
	}

	UE_NET_TEST(TNetChunkedArray, MoveSemantics)
	{
		// Move constructor.
		{
			TNetChunkedArray<int32> ArrEmpty(0, EInitMemory::Zero);
			TNetChunkedArray<int32> MovedEmpty(MoveTemp(ArrEmpty));
			UE_NET_ASSERT_EQ(0, MovedEmpty.Num());
			UE_NET_ASSERT_EQ(0, MovedEmpty.Capacity());
			UE_NET_ASSERT_EQ(0, ArrEmpty.Num());
			UE_NET_ASSERT_EQ(0, ArrEmpty.Capacity());

			TNetChunkedArray<int32> ArrChunked(200);
			ArrChunked.AddToIndexZeroed(310);
			InitializeTestArrayValue(ArrChunked);
			TNetChunkedArray<int32> MovedChunked(MoveTemp(ArrChunked));
			UE_NET_ASSERT_EQ(311, MovedChunked.Num());
			UE_NET_ASSERT_EQ(400, MovedChunked.Capacity());
			UE_NET_ASSERT_EQ(0, ArrChunked.Num());
			UE_NET_ASSERT_EQ(0, ArrChunked.Capacity());
			for (int32 Index = 0; Index < MovedChunked.Num(); Index++)
			{
				UE_NET_ASSERT_EQ(MovedChunked[Index], Index);
			}
		}

		// Move assignment operator.
		{
			TNetChunkedArray<int32> MovedEmpty;
			MovedEmpty = TNetChunkedArray<int32>(0, EInitMemory::Zero);
			UE_NET_ASSERT_EQ(0, MovedEmpty.Num());
			UE_NET_ASSERT_EQ(0, MovedEmpty.Capacity());

			TNetChunkedArray<int32> ArrChunked(200);
			ArrChunked.AddToIndexZeroed(310);
			InitializeTestArrayValue(ArrChunked);
			TNetChunkedArray<int32> MovedChunked;
			MovedChunked = MoveTemp(ArrChunked);
			UE_NET_ASSERT_EQ(311, MovedChunked.Num());
			UE_NET_ASSERT_EQ(400, MovedChunked.Capacity());
			UE_NET_ASSERT_EQ(0, ArrChunked.Num());
			UE_NET_ASSERT_EQ(0, ArrChunked.Capacity());
			for (int32 Index = 0; Index < MovedChunked.Num(); Index++)
			{
				UE_NET_ASSERT_EQ(MovedChunked[Index], Index);
			}
		}
	}

	// Test moving and copying an array of structs that have default constructors.
	UE_NET_TEST(TNetChunkedArray, CopyAndMoveStruct)
	{
		struct TestStruct
		{
			TestStruct() : VarA(10), VarB(20) {};
			int32 VarA;
			int32 VarB;
		};

		// Copy constructor.
		{
			TNetChunkedArray<TestStruct> ArrStruct(200, EInitMemory::Constructor);
			ArrStruct.AddToIndexZeroed(310);
			for (int32 Index = 0; Index < ArrStruct.Num(); Index++)
			{
				ArrStruct[Index].VarA = 100;
				ArrStruct[Index].VarB = 200;
			}

			TNetChunkedArray<TestStruct> CopyStruct(ArrStruct);
			for (int32 Index = 0; Index < ArrStruct.Num(); Index++)
			{
				UE_NET_ASSERT_EQ(ArrStruct[Index].VarA, CopyStruct[Index].VarA);
				UE_NET_ASSERT_EQ(ArrStruct[Index].VarB, CopyStruct[Index].VarB);
			}
		}

		// Copy assignment operator.
		{
			TNetChunkedArray<TestStruct> ArrStruct(200, EInitMemory::Constructor);
			ArrStruct.AddToIndexZeroed(310);
			for (int32 Index = 0; Index < ArrStruct.Num(); Index++)
			{
				ArrStruct[Index].VarA = 100;
				ArrStruct[Index].VarB = 200;
			}

			TNetChunkedArray<TestStruct> CopyStruct;
			CopyStruct = ArrStruct;
			for (int32 Index = 0; Index < ArrStruct.Num(); Index++)
			{
				UE_NET_ASSERT_EQ(ArrStruct[Index].VarA, CopyStruct[Index].VarA);
				UE_NET_ASSERT_EQ(ArrStruct[Index].VarB, CopyStruct[Index].VarB);
			}
		}

		// Move constructor.
		{
			TNetChunkedArray<TestStruct> ArrStruct(200, EInitMemory::Constructor);
			ArrStruct.AddToIndexZeroed(310);
			for (int32 Index = 0; Index < ArrStruct.Num(); Index++)
			{
				ArrStruct[Index].VarA = 100;
				ArrStruct[Index].VarB = 200;
			}

			TNetChunkedArray<TestStruct> MoveStruct(MoveTemp(ArrStruct));
			for (int32 Index = 0; Index < MoveStruct.Num(); Index++)
			{
				UE_NET_ASSERT_EQ(MoveStruct[Index].VarA, 100);
				UE_NET_ASSERT_EQ(MoveStruct[Index].VarB, 200);
			}
		}

		// Move assignment operator.
		{
			TNetChunkedArray<TestStruct> MoveStruct;
			MoveStruct = TNetChunkedArray<TestStruct>(200, EInitMemory::Constructor);
			for (int32 Index = 0; Index < MoveStruct.Num(); Index++)
			{
				UE_NET_ASSERT_EQ(MoveStruct[Index].VarA, 10);
				UE_NET_ASSERT_EQ(MoveStruct[Index].VarB, 20);
			}
		}
	}
}