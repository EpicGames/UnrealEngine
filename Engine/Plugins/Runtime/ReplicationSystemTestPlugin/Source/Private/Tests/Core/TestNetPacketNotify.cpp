// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Net/NetPacketNotify.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Net/Util/SequenceHistory.h"
#include "Net/Util/SequenceNumber.h"
#include "Serialization/BitWriter.h"
#include "Serialization/BitReader.h"

namespace UE::Net
{

FTestMessage& operator<<(FTestMessage& Message, const FNetPacketNotify::SequenceNumberT& SequenceNumber)
{
	return Message << SequenceNumber.Get();
}

FTestMessage& operator<<(FTestMessage& Message, const FNetPacketNotify::SequenceHistoryT& History)
{
	for (uint32 WordIt=0U; WordIt<FNetPacketNotify::SequenceHistoryT::WordCount; ++WordIt)
	{
		Message << History.Data()[WordIt];
	}
	return Message;
}

namespace Private
{

struct FNetPacketNotifyTestUtil : public FNetworkAutomationTestSuiteFixture
{
	enum
	{
		LastValidSequenceHistoryIndex = FNetPacketNotify::MaxSequenceHistoryLength - 1
	};

	FNetPacketNotify DefaultNotify;
	FNetPacketNotifyTestUtil()
	{
		DefaultNotify.Init(FNetPacketNotify::SequenceNumberT(-1),  FNetPacketNotify::SequenceNumberT(0));
	}

	// Helper to fill in SequenceHistory with expected result
	template <typename T>
	static void InitHistory(FNetPacketNotify::SequenceHistoryT& History, const T& DataToSet)
	{
		const SIZE_T Count = sizeof(T) / sizeof(DataToSet[0]);
		static_assert(Count <= FNetPacketNotify::SequenceHistoryT::WordCount, "DataToSet array cannot be larger HistoryBuffer");

		for (SIZE_T It=0; It < Count; ++It)
		{
			History.Data()[It] = DataToSet[It];
		}
	}

	// Pretend to receive and acknowledge incoming packet to generate ackdata
	static int32 PretendReceiveSeq(FNetPacketNotify& PacketNotify, FNetPacketNotify::SequenceNumberT Seq, bool Ack = true)
	{
		FNetPacketNotify::FNotificationHeader Data;
		Data.Seq = Seq;
		Data.AckedSeq = PacketNotify.GetOutAckSeq();
		Data.History = FNetPacketNotify::SequenceHistoryT(0);
		Data.HistoryWordCount = 1;
		
		FNetPacketNotify::SequenceNumberT::DifferenceT SeqDelta = PacketNotify.Update(Data, [](FNetPacketNotify::SequenceNumberT AckedSequence, bool delivered) {});		
		if (SeqDelta > 0 && !PacketNotify.IsWaitingForSequenceHistoryFlush())
		{
			if (Ack)
			{
				PacketNotify.AckSeq(Seq);
			}
		}

		return SeqDelta;
	}

	// Pretend to send packet
	static void PretendSendSeq(FNetPacketNotify& PacketNotify, FNetPacketNotify::SequenceNumberT LastAckSeq)
	{
		// set last InAcqSeq that we know that the remote end knows that we know (AckAck)
		PacketNotify.WrittenHistoryWordCount = 1;
		PacketNotify.WrittenInAckSeq = LastAckSeq;

		// Store data
		PacketNotify.CommitAndIncrementOutSeq();
	}

	// pretend to ack array of sequence numbers
	template<typename T>
	static void PretendAckSequenceNumbers(FNetPacketNotify& PacketNotify, const T& InSequenceNumbers)
	{
		SIZE_T SequenceNumberCount = sizeof(InSequenceNumbers) / sizeof(InSequenceNumbers[0]);

		for (SIZE_T I=0; I<SequenceNumberCount; ++I)
		{
			FNetPacketNotifyTestUtil::PretendReceiveSeq(PacketNotify, InSequenceNumbers[I]);
		}
	}
	
	// Pretend that we received a packet
	template<typename T>
	static SIZE_T PretendReceivedPacket(FNetPacketNotify& PacketNotify, const FNetPacketNotify::FNotificationHeader Data, T& OutSequenceNumbers)
	{
		SIZE_T NotificationCount = 0;

		auto HandleAck = [&OutSequenceNumbers, &NotificationCount](FNetPacketNotify::SequenceNumberT Seq, bool delivered)
		{
			const SIZE_T MaxSequenceNumberCount = sizeof(OutSequenceNumbers) / sizeof(OutSequenceNumbers[0]);

			if (delivered)
			{
				if (NotificationCount < MaxSequenceNumberCount)
				{
					OutSequenceNumbers[NotificationCount] = Seq;
				}
				++NotificationCount;		
			}
		};
		return PacketNotify.Update(Data, HandleAck);
	}

	struct FTestNode
	{
		FNetPacketNotify PacketNotify;

		uint32 InPacketId = -1;
		uint32 OutPacketId = 0;
		uint32 LastNotifiedPacketId = -1;

		TArray<uint32> AcceptedInPackets;
		TArray<uint32> AcknowledgedPackets;
		TArray<uint32> ExpectedAcceptedInPackets;

		FBitWriter Writer{FNetPacketNotify::MaxSequenceHistoryLength, true};

		FTestNode(FNetPacketNotify::SequenceNumberT FirstSequence = 0)
		{
			InPacketId = FirstSequence.Get() - 1U;
			OutPacketId = FirstSequence.Get();
			LastNotifiedPacketId = OutPacketId - 1;

			PacketNotify.Init(FNetPacketNotify::SequenceNumberT::SequenceT(InPacketId), FNetPacketNotify::SequenceNumberT::SequenceT(OutPacketId));
		}

		FTestNode(FNetPacketNotify::SequenceNumberT InSequnce, FNetPacketNotify::SequenceNumberT OutSequnce)
		{
			InPacketId = InSequnce.Get();
			OutPacketId = OutSequnce.Get();
			LastNotifiedPacketId = OutPacketId - 1;

			PacketNotify.Init(FNetPacketNotify::SequenceNumberT::SequenceT(InPacketId), FNetPacketNotify::SequenceNumberT::SequenceT(OutPacketId));
		}

		const FBitWriter& Send(bool ResetWriter = true)
		{
			const SIZE_T CurrentHistoryWordCount = FMath::Clamp<SIZE_T>((PacketNotify.GetCurrentSequenceHistoryLength() + FNetPacketNotify::SequenceHistoryT::BitsPerWord - 1u) / FNetPacketNotify::SequenceHistoryT::BitsPerWord, 1u, FNetPacketNotify::SequenceHistoryT::WordCount);

			FNetPacketNotify::FNotificationHeader Data;

			if (ResetWriter)
			{
				Writer.Reset();
			}

			PacketNotify.WriteHeader(Writer);
			PacketNotify.CommitAndIncrementOutSeq();

			++OutPacketId;

			return Writer;
		}

		const FBitWriter& SendBurst(int32 NumPackets, bool bResetWriter = true)
		{
			if (bResetWriter)
			{
				Writer.Reset();
			}
			while (NumPackets > 0)
			{
				Send(false);
				--NumPackets;
			}

			return Writer;
		}

		// Read IncomingData and (optional) compare received acks with state from the from node.
		// Returns false if an error is detected
		bool Receive(const FBitWriter& IncomingData, const FTestNode* From = nullptr)
		{
			bool bSuccess = true;

			auto HandlePacketNotification = [&](FNetPacketNotify::SequenceNumberT Seq, bool bDelivered)
			{
				++LastNotifiedPacketId;

				// Sanity check
				if (!ensure(FNetPacketNotify::SequenceNumberT(FNetPacketNotify::SequenceNumberT::SequenceT(LastNotifiedPacketId)) == Seq))
				{
					bSuccess = false;
				}

				if (bDelivered)
				{
					AcknowledgedPackets.Add(LastNotifiedPacketId);
					if (!ensure(!From || From->AcceptedInPackets.Contains(LastNotifiedPacketId)))
					{
						bSuccess = false;
					}
				}
				else
				{
					if (!ensure(!From || !From->AcceptedInPackets.Contains(LastNotifiedPacketId)))
					{
						bSuccess = false;
					}
				}
			};

			FBitReader Reader(IncomingData.GetData(), IncomingData.GetNumBits());

			FNetPacketNotify::FNotificationHeader Header;
			PacketNotify.ReadHeader(Header, Reader);
			
			const uint32 SequenceDelta = PacketNotify.Update(Header, HandlePacketNotification);
			if (SequenceDelta > 0U)
			{
				InPacketId += SequenceDelta;

				// Sanity check
				check(FNetPacketNotify::SequenceNumberT(FNetPacketNotify::SequenceNumberT::SequenceT(InPacketId)) == PacketNotify.InSeq);

				if (!PacketNotify.IsWaitingForSequenceHistoryFlush())
				{
					PacketNotify.AckSeq(FNetPacketNotify::SequenceNumberT::SequenceT(InPacketId));
					AcceptedInPackets.Add(InPacketId);
				}
			}

			return bSuccess;
		}

		// Read IncomingData and (optional) compare received acks with state from the from node.
		// Returns false if an error is detected
		bool ReceiveBurst(const FBitWriter& IncomingData, const FTestNode* From = nullptr)
		{
			bool bSuccess = true;

			auto HandlePacketNotification = [&](FNetPacketNotify::SequenceNumberT Seq, bool bDelivered)
			{
				++LastNotifiedPacketId;

				// Sanity check
				if (!ensure(FNetPacketNotify::SequenceNumberT(FNetPacketNotify::SequenceNumberT::SequenceT(LastNotifiedPacketId)) == Seq))
				{
					bSuccess = false;
				}

				if (bDelivered)
				{
					AcknowledgedPackets.Add(LastNotifiedPacketId);
					if (!ensure(!From || From->AcceptedInPackets.Contains(LastNotifiedPacketId)))
					{
						bSuccess = false;
					}
				}
				else
				{
					if (!ensure(!From || !From->AcceptedInPackets.Contains(LastNotifiedPacketId)))
					{
						bSuccess = false;
					}
				}
			};

			FBitReader Reader(IncomingData.GetData(), IncomingData.GetNumBits());

			while (Reader.GetBytesLeft() > 0)
			{
				FNetPacketNotify::FNotificationHeader Header;
				PacketNotify.ReadHeader(Header, Reader);

				const uint32 SequenceDelta = PacketNotify.Update(Header, HandlePacketNotification);
				if (SequenceDelta > 0U)
				{
					InPacketId += SequenceDelta;

					// Sanity check
					check(FNetPacketNotify::SequenceNumberT(FNetPacketNotify::SequenceNumberT::SequenceT(InPacketId)) == PacketNotify.InSeq);
				
					if (!PacketNotify.IsWaitingForSequenceHistoryFlush())
					{
						PacketNotify.AckSeq(FNetPacketNotify::SequenceNumberT::SequenceT(InPacketId));
						AcceptedInPackets.Add(InPacketId);
					}
				}
			}

			return bSuccess;
		}

		// Read IncomingData and (optional) compare received acks with state from the from node.
		// Returns false if an error is detected
		template<typename T>
		bool ReceiveWithCallback(const FBitWriter& IncomingData, T&& Functor, const FTestNode* From = nullptr)
		{
			bool bSuccess = true;

			auto HandlePacketNotification = [&](FNetPacketNotify::SequenceNumberT Seq, bool bDelivered)
			{
				++LastNotifiedPacketId;

				// Sanity check
				if (!ensure(FNetPacketNotify::SequenceNumberT(FNetPacketNotify::SequenceNumberT::SequenceT(LastNotifiedPacketId)) == Seq))
				{
					bSuccess = false;
				}

				if (bDelivered)
				{
					AcknowledgedPackets.Add(LastNotifiedPacketId);
					if (!ensure(!From || From->AcceptedInPackets.Contains(LastNotifiedPacketId)))
					{
						bSuccess = false;
					}
				}
				else
				{
					if (!ensure(!From || !From->AcceptedInPackets.Contains(LastNotifiedPacketId)))
					{
						bSuccess = false;
					}
				}
			};

			FBitReader Reader(IncomingData.GetData(), IncomingData.GetNumBits());

			FNetPacketNotify::FNotificationHeader Header;
			PacketNotify.ReadHeader(Header, Reader);
			
			const uint32 SequenceDelta = PacketNotify.Update(Header, HandlePacketNotification);
			if (SequenceDelta > 0U)
			{
				InPacketId += SequenceDelta;

				// Sanity check
				check(FNetPacketNotify::SequenceNumberT(FNetPacketNotify::SequenceNumberT::SequenceT(InPacketId)) == PacketNotify.InSeq);

				if (!PacketNotify.IsWaitingForSequenceHistoryFlush())
				{
					// Do some things before accepting the packet.
					Functor();

					PacketNotify.AckSeq(FNetPacketNotify::SequenceNumberT::SequenceT(InPacketId));
					AcceptedInPackets.Add(InPacketId);
				}
			}

			return bSuccess;
		}

		void SendAndDeliverTo(FTestNode& Target)
		{
			Target.Receive(Send());
		}
	
		void SendRange(TArrayView<const uint32> PacketIds, bool bResetWriter = true)
		{
			if (bResetWriter)
			{
				Writer.Reset();
			}
			for (const uint32 PacketId : PacketIds)
			{
				// Skip missing seqs
				FBitWriterMark Mark(Writer);
				while (PacketId != OutPacketId)
				{					
					Send(false);
				}
				Mark.Pop(Writer);
				Send(false);
			}
		}

		void SendAndDeliverTo(FTestNode& Target, TArrayView<const uint32> PacketIds)
		{
			SendRange(PacketIds);
			Target.ReceiveBurst(Writer);
		}

		bool ValidateAcknowledgedPackets(TArrayView<const uint32> ExpectedPacketIds) const
		{
			if (AcknowledgedPackets.Num() != ExpectedPacketIds.Num())
			{
				return false;
			}
			const int32 MaxCount = FMath::Min(AcknowledgedPackets.Num(), ExpectedPacketIds.Num());
			for (int32 I = 0; I < MaxCount; ++I)
			{
				if (AcknowledgedPackets[I] != ExpectedPacketIds[I])
				{
					UE_LOGF(LogNetTraffic, Error, "Ack %u != Exp %u index %u", AcknowledgedPackets[I], ExpectedPacketIds[I], I);
					return false;
				}
			}
			return true;
		}

		bool ValidateAcceptedPackets(TArrayView<const uint32> ExpectedPacketIds) const
		{
			return AcceptedInPackets == ExpectedPacketIds;
		}

		bool ValidateExpectedAcceptedPackets()
		{
			return ValidateAcceptedPackets(MakeArrayView(ExpectedAcceptedInPackets));
		}
	};
};

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, Fill)
{
	FNetPacketNotify Acks = DefaultNotify;

	FNetPacketNotify::SequenceNumberT ExpectedInSeq(30);
	FNetPacketNotify::SequenceHistoryT ExpectedInSeqHistory(0x55555555u, 1);
				
	for (FNetPacketNotify::SequenceNumberT::SequenceT I = 0U; I < 16U; ++I)
	{
		FNetPacketNotifyTestUtil::PretendReceiveSeq(Acks, I * 2U);
	}
		
	// Verify InSeq
	UE_NET_ASSERT_EQ(Acks.GetInSeq(), ExpectedInSeq);

	// Verify History
	UE_NET_ASSERT_EQ(Acks.GetInSeqHistory(), ExpectedInSeqHistory);
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, DropEveryOther)
{
	FNetPacketNotify Acks = DefaultNotify;

	FNetPacketNotify::SequenceNumberT ExpectedInSeq(31U);
	FNetPacketNotify::SequenceHistoryT ExpectedInSeqHistory(0xffffffffu, 1U);
				
	for (FNetPacketNotify::SequenceNumberT::SequenceT I=0U; I<32U; ++I)
	{
		FNetPacketNotifyTestUtil::PretendReceiveSeq(Acks, I, true);
	}

	// Verify InSeq
	UE_NET_ASSERT_EQ(Acks.GetInSeq(), ExpectedInSeq);

	// Verify History
	UE_NET_ASSERT_EQ(Acks.GetInSeqHistory(), ExpectedInSeqHistory);
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, SequenceNumberOverflow)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetTraffic, ELogVerbosity::Error);

	FNetPacketNotify Acks = DefaultNotify;

	const FNetPacketNotify::SequenceNumberT MaxWindowSeq(FNetPacketNotify::SequenceNumberT::SeqNumberHalf);
	const FNetPacketNotify::SequenceNumberT ExpectedInSeq(0);

	// Verify default
	UE_NET_ASSERT_NE(Acks.GetInSeq(), ExpectedInSeq);

	// Pretend receive first seq
	FNetPacketNotifyTestUtil::PretendReceiveSeq(Acks, ExpectedInSeq, true);

	// Expects that we have accepted and acked the this sequence
	UE_NET_ASSERT_EQ(Acks.GetInSeq(), ExpectedInSeq);
	UE_NET_ASSERT_EQ(Acks.GetInAckSeq(), ExpectedInSeq);

	// Verify that we did not accept invalid sequence
	FNetPacketNotifyTestUtil::PretendReceiveSeq(Acks, MaxWindowSeq, true);
	UE_NET_ASSERT_EQ(Acks.GetInSeq(), ExpectedInSeq);
	UE_NET_ASSERT_EQ(Acks.GetInAckSeq(), ExpectedInSeq);

	// Verify that we can receive next valid sequence
	const FNetPacketNotify::SequenceNumberT NextExpectedInSeq(1);
	FNetPacketNotifyTestUtil::PretendReceiveSeq(Acks, NextExpectedInSeq, true);
	UE_NET_ASSERT_EQ(Acks.GetInSeq(), NextExpectedInSeq);
	UE_NET_ASSERT_EQ(Acks.GetInAckSeq(), NextExpectedInSeq);

	// Verify that we accepted a new sequence even if it is out and bounds and will trigger a SequencyHistoryFlush
	FNetPacketNotifyTestUtil::PretendReceiveSeq(Acks, MaxWindowSeq, true);
	UE_NET_ASSERT_GT(Acks.GetInSeq(), NextExpectedInSeq);
	UE_NET_ASSERT_GT(Acks.GetInAckSeq(), NextExpectedInSeq);
	UE_NET_ASSERT_TRUE(Acks.IsWaitingForSequenceHistoryFlush());
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, BurstDrop)
{
	FNetPacketNotify Acks = DefaultNotify;

	FNetPacketNotify::SequenceNumberT ExpectedInSeq(128);
	FNetPacketNotify::SequenceHistoryT ExpectedInSeqHistory;
	uint32 ExpectedArray[] = {0x1, 0, 0, 0x20000000 };
	FNetPacketNotifyTestUtil::InitHistory(ExpectedInSeqHistory, ExpectedArray );

	// Drop early
	FNetPacketNotifyTestUtil::PretendReceiveSeq(Acks, 3);

	// Large gap until next seq
	FNetPacketNotifyTestUtil::PretendReceiveSeq(Acks, 128);

	UE_NET_ASSERT_EQ(Acks.GetInSeq(), ExpectedInSeq);
	UE_NET_ASSERT_EQ(Acks.GetInSeqHistory(), ExpectedInSeqHistory);
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, CreateHistory)
{
	FNetPacketNotify Acks = DefaultNotify;

	const FNetPacketNotify::SequenceNumberT ExpectedInSeq(18);
	const FNetPacketNotify::SequenceHistoryT ExpectedInSeqHistory(0x8853u);

	const FNetPacketNotify::SequenceNumberT AckdPacketIds[] = {3, 7, 12, 14, 17, 18};
	const SIZE_T ExpectedCount = sizeof(AckdPacketIds)/sizeof((AckdPacketIds)[0]);		

	FNetPacketNotifyTestUtil::PretendAckSequenceNumbers(Acks, AckdPacketIds);
 
	UE_NET_ASSERT_EQ(Acks.GetInSeq(), ExpectedInSeq);
	UE_NET_ASSERT_EQ(Acks.GetInSeqHistory(), ExpectedInSeqHistory);
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, Notifications)
{
	FNetPacketNotify Acks = DefaultNotify;

	static const FNetPacketNotify::SequenceNumberT ExpectedAckdPacketIds[] = {3, 7, 12, 14, 17, 18};
	static const SIZE_T ExpectedCount = sizeof(ExpectedAckdPacketIds)/sizeof((ExpectedAckdPacketIds)[0]);		

	FNetPacketNotify::SequenceNumberT RcvdAcks[ExpectedCount] = { 0 };
	SIZE_T RcvdCount = 0;

	// Fill in some data
	FNetPacketNotify::FNotificationHeader Data;
	Data.Seq = FNetPacketNotify::SequenceNumberT(0);
	Data.AckedSeq = FNetPacketNotify::SequenceNumberT(18);
	Data.History = FNetPacketNotify::SequenceHistoryT(0x8853u);
	Data.HistoryWordCount = 1;

	// Need to fake ack record as well.
	for (SIZE_T It=0; It <= 18; ++It)
	{
		FNetPacketNotifyTestUtil::PretendSendSeq(Acks, 0);
	}
	
	SIZE_T DeltaSeq = FNetPacketNotifyTestUtil::PretendReceivedPacket(Acks, Data, RcvdAcks);

	UE_NET_ASSERT_EQ(DeltaSeq, SIZE_T(1));
	UE_NET_ASSERT_EQ(FPlatformMemory::Memcmp(ExpectedAckdPacketIds, RcvdAcks, sizeof(ExpectedAckdPacketIds)), 0);
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, ReceiveInvalidAck)
{
	FNetPacketNotify Acks = DefaultNotify;

	static const FNetPacketNotify::SequenceNumberT ExpectedAckdPacketIds[] = {3, 7, 12, 14, 17, 18};
	static const SIZE_T ExpectedCount = sizeof(ExpectedAckdPacketIds)/sizeof((ExpectedAckdPacketIds)[0]);		

	FNetPacketNotify::SequenceNumberT RcvdAcks[ExpectedCount] = { 0 };
	SIZE_T RcvdCount = 0;

	// Fill in some data
	FNetPacketNotify::FNotificationHeader Data;
	Data.Seq = FNetPacketNotify::SequenceNumberT(0);
	Data.AckedSeq = FNetPacketNotify::SequenceNumberT(19);
	Data.History = FNetPacketNotify::SequenceHistoryT(0x8853u);
	Data.HistoryWordCount = 1;

	// Need to fake ack record as well.
	for (SIZE_T It=0; It <= 18; ++It)
	{
		FNetPacketNotifyTestUtil::PretendSendSeq(Acks, FNetPacketNotify::SequenceNumberT(0U));
	}
	
	SIZE_T DeltaSeq = FNetPacketNotifyTestUtil::PretendReceivedPacket(Acks, Data, RcvdAcks);

	UE_NET_ASSERT_EQ(DeltaSeq, SIZE_T(0));
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, FillSeqWindowWitNoMargin)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetTraffic, ELogVerbosity::Error);

	// Test without safety margin
	for (FNetPacketNotify::SequenceNumberT SeqOffset : {FNetPacketNotify::SequenceNumberT(0), FNetPacketNotify::SequenceNumberT(3)})
	{
		FNetPacketNotify PacketNotify = DefaultNotify;
		for (FNetPacketNotify::SequenceNumberT It = 0; It < FNetPacketNotify::MaxSequenceHistoryLength - 1; ++It)
		{
			PretendSendSeq(PacketNotify, SeqOffset);
		}
		UE_NET_ASSERT_FALSE_MSG(PacketNotify.IsSequenceWindowFull(), "Test SeqWindowFull {0, MaxSequenceHistoryLength - 2}");

		// Fill sequence window
		FNetPacketNotifyTestUtil::PretendSendSeq(PacketNotify, SeqOffset);
		UE_NET_ASSERT_TRUE_MSG(PacketNotify.IsSequenceWindowFull(), "Test SeqWindowFull {0, MaxSequenceHistoryLength - 1}");

		// Fake all acked. We expect acks on all sequence numbers.
		{
			FNetPacketNotify::FNotificationHeader NotificationData;
			NotificationData.Seq = 0;
			NotificationData.AckedSeq = PacketNotify.GetOutSeq() - 1;
			NotificationData.History = FNetPacketNotify::SequenceHistoryT(0xFFFFFFFFU, FNetPacketNotify::SequenceHistoryT::WordCount);
			NotificationData.HistoryWordCount = FNetPacketNotify::SequenceHistoryT::WordCount;

			PacketNotify.Update(NotificationData, [this](FNetPacketNotify::SequenceNumberT, bool bWasDelivered)
				{
					UE_NET_ASSERT_TRUE_MSG(bWasDelivered, "Test SeqWindowFull all delivered");
				});
		}
	}
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, FillSeqWindowWitSafetyMargin)
{
	// Test with safety margin
	constexpr unsigned SafetyMargin = 3;
	FNetPacketNotify PacketNotify = DefaultNotify;
	for (SIZE_T It = 0; It < FNetPacketNotify::MaxSequenceHistoryLength - 1 - SafetyMargin; ++It)
	{
		FNetPacketNotifyTestUtil::PretendSendSeq(PacketNotify, 0);
	}
	UE_NET_ASSERT_FALSE_MSG(PacketNotify.IsSequenceWindowFull(SafetyMargin), "Test SeqWindowFull Margin 3 {0, MaxSequenceHistoryLength - 2}");

	FNetPacketNotifyTestUtil::PretendSendSeq(PacketNotify, 0);
	UE_NET_ASSERT_TRUE_MSG(PacketNotify.IsSequenceWindowFull(SafetyMargin), "Test SeqWindowFull Margin 3 {0, MaxSequenceHistoryLength - 1}");
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, TestSendAndReceive)
{
	FTestNode Src;
	FTestNode Dst;

	// Pretend to send and accept packet in both directions
	Src.SendAndDeliverTo(Dst);
	Dst.SendAndDeliverTo(Src);

	// Validate expected state
	UE_NET_ASSERT_TRUE(Src.ValidateAcknowledgedPackets({0}));
	UE_NET_ASSERT_TRUE(Dst.ValidateAcceptedPackets({0}));
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, TestIgnoresOutOfBound)
{
	// Initialize with an offset that wraps around the sequence space
	FTestNode Src(FNetPacketNotify::SequenceNumberT::SeqNumberHalf);
	FTestNode Dst;

	// Pretend to send packet in both directions, which should be ignored by dst
	Src.SendAndDeliverTo(Dst);
	Dst.SendAndDeliverTo(Src);

	// Validate expected state
	UE_NET_ASSERT_EQ(Dst.AcceptedInPackets.Num(), 0);
	UE_NET_ASSERT_EQ(Src.AcknowledgedPackets.Num(), 0);	
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, TestOutOfBoundDoesNotTriggerFlush)
{
	FTestNode Src;
	FTestNode Dst;

	// Pretend to send and accept packet in both directions
	Src.SendAndDeliverTo(Dst , {FNetPacketNotify::SequenceNumberT::SeqNumberMax, FNetPacketNotify::SequenceNumberT::SeqNumberMax + 1U});
	Dst.SendAndDeliverTo(Src);

	// Validate expected state
	UE_NET_ASSERT_FALSE(Dst.PacketNotify.IsWaitingForSequenceHistoryFlush());
	UE_NET_ASSERT_TRUE(Src.ValidateAcknowledgedPackets({0}));
	UE_NET_ASSERT_TRUE(Dst.ValidateAcceptedPackets({0}));
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, TestBurstReceiveValidRangeWrapAround)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetTraffic, ELogVerbosity::Error);

	// Initialize with an offset that wraps around the sequence space
	FTestNode Src(FNetPacketNotify::SequenceNumberT::SeqNumberHalf + 2);
	FTestNode Dst(FNetPacketNotify::SequenceNumberT::SeqNumberHalf + 2);

	// Pretend to send and accept packet in both directions
	Dst.ExpectedAcceptedInPackets.Add(FNetPacketNotify::SequenceNumberT::SeqNumberMax);
	Dst.ExpectedAcceptedInPackets.Add(FNetPacketNotify::SequenceNumberT::SeqNumberMax + 1U);

	Src.SendAndDeliverTo(Dst, MakeArrayView(Dst.ExpectedAcceptedInPackets));
	Dst.SendAndDeliverTo(Src);

	// Validate expected state
	UE_NET_ASSERT_FALSE(Dst.PacketNotify.IsWaitingForSequenceHistoryFlush());
	UE_NET_ASSERT_TRUE(Dst.ValidateExpectedAcceptedPackets());
	UE_NET_ASSERT_TRUE(Src.ValidateAcknowledgedPackets(MakeArrayView(Dst.AcceptedInPackets)));
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, TestAcceptsHugeDeltaWithNoPreviousAcks)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetTraffic, ELogVerbosity::Error);

	// Initialize with an offset that is at the end of the sequence space
	const FNetPacketNotify::SequenceNumberT FirstOutSeq(FNetPacketNotify::SequenceNumberT::SeqNumberHalf - 2);

	FTestNode Src(-1, FirstOutSeq);
	FTestNode Dst;

	// Pretend to send and accept packet in both directions
	Src.SendAndDeliverTo(Dst);
	Dst.SendAndDeliverTo(Src);

	// Validate expected state
	UE_NET_ASSERT_TRUE(Src.ValidateAcknowledgedPackets({FirstOutSeq.Get()}));
	UE_NET_ASSERT_TRUE(Dst.ValidateAcceptedPackets({FirstOutSeq.Get()}));
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, TestBurstReceiveValidRange)
{
	FTestNode Src;
	FTestNode Dst;

	// Send a range burst of packets
	Src.SendAndDeliverTo(Dst, {0U, LastValidSequenceHistoryIndex});

	// Respond
	Dst.SendAndDeliverTo(Src);

	// Validate expected state
	UE_NET_ASSERT_TRUE(Src.ValidateAcknowledgedPackets({0U, LastValidSequenceHistoryIndex}));
	UE_NET_ASSERT_TRUE(Dst.ValidateAcceptedPackets({0U, LastValidSequenceHistoryIndex}));	
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, TestBurstReceiveTriggersWaitForSequenceWindowFlush)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetTraffic, ELogVerbosity::Error);

	FTestNode Src;
	FTestNode Dst;

	// Send a range burst of packets
	Src.SendAndDeliverTo(Dst, {0U, FNetPacketNotify::MaxSequenceHistoryLength});

	// Respond
	Dst.SendAndDeliverTo(Src);

	// Verify that we triggered overshoot
	UE_NET_ASSERT_TRUE(Dst.PacketNotify.IsWaitingForSequenceHistoryFlush());

	// Validate expected state
	UE_NET_ASSERT_TRUE(Src.ValidateAcknowledgedPackets({0U}));
	UE_NET_ASSERT_TRUE(Dst.ValidateAcceptedPackets({0U}));	
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, TestReceiveDoesNotTriggersWaitForSequenceWindowFlushIfNoAcks)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetTraffic, ELogVerbosity::Error);

	FTestNode Src;
	FTestNode Dst;

	// Send a specific sequence that would overflow window if we had already acked data
	// we have no previous acks we can accept the sequence and allow the other end to inject any Fing nacks
	Src.SendAndDeliverTo(Dst, {FNetPacketNotify::MaxSequenceHistoryLength + 1, FNetPacketNotify::MaxSequenceHistoryLength + 2});

	// Respond
	Dst.SendAndDeliverTo(Src);

	// Verify that we triggered overshoot
	UE_NET_ASSERT_FALSE(Dst.PacketNotify.IsWaitingForSequenceHistoryFlush());

	// Validate expected state
	UE_NET_ASSERT_TRUE(Src.ValidateAcknowledgedPackets({FNetPacketNotify::MaxSequenceHistoryLength + 1, FNetPacketNotify::MaxSequenceHistoryLength + 2}));
	UE_NET_ASSERT_TRUE(Dst.ValidateAcceptedPackets({FNetPacketNotify::MaxSequenceHistoryLength + 1, FNetPacketNotify::MaxSequenceHistoryLength + 2}));
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, TestCanRecoverFromsWaitForSequenceWindowFlush)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetTraffic, ELogVerbosity::Error);

	FTestNode Src;
	FTestNode Dst;

	// Send a range burst of packets
	Src.SendAndDeliverTo(Dst, {0U, FNetPacketNotify::MaxSequenceHistoryLength});

	// Respond
	Dst.SendAndDeliverTo(Src);

	// Verify that we triggered overshoot
	UE_NET_ASSERT_TRUE(Dst.PacketNotify.IsWaitingForSequenceHistoryFlush());

	// Validate expected state
	UE_NET_ASSERT_TRUE(Src.ValidateAcknowledgedPackets({0U}));
	UE_NET_ASSERT_TRUE(Dst.ValidateAcceptedPackets({0U}));

	// Send a range burst of packets
	Src.SendAndDeliverTo(Dst, {FNetPacketNotify::MaxSequenceHistoryLength + 1, FNetPacketNotify::MaxSequenceHistoryLength + 2});

	// Verify that we recovered from overshoot
	UE_NET_ASSERT_FALSE(Dst.PacketNotify.IsWaitingForSequenceHistoryFlush());

	// Respond
	Dst.SendAndDeliverTo(Src);

	// Validate expected state
	UE_NET_ASSERT_TRUE(Src.ValidateAcknowledgedPackets({0U, FNetPacketNotify::MaxSequenceHistoryLength + 1, FNetPacketNotify::MaxSequenceHistoryLength + 2}));
	UE_NET_ASSERT_TRUE(Dst.ValidateAcceptedPackets({0U, FNetPacketNotify::MaxSequenceHistoryLength + 1, FNetPacketNotify::MaxSequenceHistoryLength + 2}));
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, TestGetBothEndsWaitForSequenceWindowFlush)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetTraffic, ELogVerbosity::Error);

	FTestNode Src;
	FTestNode Dst;

	// Send a range burst of packets
	Src.SendAndDeliverTo(Dst, {0U, FNetPacketNotify::MaxSequenceHistoryLength});
	Dst.SendAndDeliverTo(Src, {0U, FNetPacketNotify::MaxSequenceHistoryLength});

	// Verify that we triggered overshoot on both ends
	UE_NET_ASSERT_TRUE(Dst.PacketNotify.IsWaitingForSequenceHistoryFlush());
	UE_NET_ASSERT_TRUE(Src.PacketNotify.IsWaitingForSequenceHistoryFlush());

	// Send a range burst of from source to src, the packets should be accepted as we have cleared the sequence history
	Src.SendAndDeliverTo(Dst, {FNetPacketNotify::MaxSequenceHistoryLength + 1, FNetPacketNotify::MaxSequenceHistoryLength + 2});

	// Verify that we recovered from overshoot
	UE_NET_ASSERT_FALSE(Dst.PacketNotify.IsWaitingForSequenceHistoryFlush());

	// Src should still be waiting on flush
	UE_NET_ASSERT_TRUE(Src.PacketNotify.IsWaitingForSequenceHistoryFlush());

	// Send a range burst of from dst to src, the packets should be accepted as we have cleared the sequence history
	Dst.SendAndDeliverTo(Src, {FNetPacketNotify::MaxSequenceHistoryLength + 1, FNetPacketNotify::MaxSequenceHistoryLength + 2});

	// Verify that both recovered from overshoot
	UE_NET_ASSERT_FALSE(Src.PacketNotify.IsWaitingForSequenceHistoryFlush());
	UE_NET_ASSERT_FALSE(Dst.PacketNotify.IsWaitingForSequenceHistoryFlush());

	// Respond
	Src.SendAndDeliverTo(Src);
	Dst.SendAndDeliverTo(Dst);

	// Validate expected state
	UE_NET_ASSERT_TRUE(Src.ValidateAcknowledgedPackets({0, FNetPacketNotify::MaxSequenceHistoryLength + 1, FNetPacketNotify::MaxSequenceHistoryLength + 2}));
	UE_NET_ASSERT_TRUE(Src.ValidateAcceptedPackets({0, FNetPacketNotify::MaxSequenceHistoryLength + 1, FNetPacketNotify::MaxSequenceHistoryLength + 2, FNetPacketNotify::MaxSequenceHistoryLength + 3}));
	UE_NET_ASSERT_TRUE(Dst.ValidateAcknowledgedPackets({0, FNetPacketNotify::MaxSequenceHistoryLength + 1, FNetPacketNotify::MaxSequenceHistoryLength + 2}));
	UE_NET_ASSERT_TRUE(Dst.ValidateAcceptedPackets({0, FNetPacketNotify::MaxSequenceHistoryLength + 1, FNetPacketNotify::MaxSequenceHistoryLength + 2, FNetPacketNotify::MaxSequenceHistoryLength + 3}));
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, TestGetBothEndsWaitForSequenceWindowFlushDueToSendWindowOverflow)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetTraffic, ELogVerbosity::Error);

	FNetPacketNotify::SequenceNumberT SrcStartingSeq(1024);
	FNetPacketNotify::SequenceNumberT DstStartingSeq(0);

	FTestNode Src(DstStartingSeq, SrcStartingSeq + 1);
	FTestNode Dst(SrcStartingSeq, DstStartingSeq + 1);

	// Send off bursts on both ends, over committing send window which will trigger SequenceHistoryFlush.
	Src.SendBurst(FNetPacketNotify::MaxSequenceHistoryLength + 1);
	Dst.SendBurst(FNetPacketNotify::MaxSequenceHistoryLength + 1);

	// Receive the data
	Dst.ReceiveBurst(Src.Writer, &Src);
	Src.ReceiveBurst(Dst.Writer, &Dst);

	// Verify that we triggered SequenceHistoryFlush on both ends
	UE_NET_ASSERT_TRUE(Src.PacketNotify.IsWaitingForSequenceHistoryFlush());
	UE_NET_ASSERT_TRUE(Dst.PacketNotify.IsWaitingForSequenceHistoryFlush());

	// We a full round trip to reset SequenceHistory
	Src.SendBurst(1);
	Dst.SendBurst(1);
	Src.ReceiveBurst(Dst.Writer, &Dst);
	Dst.ReceiveBurst(Src.Writer, &Src);

	Src.SendBurst(1);
	Dst.SendBurst(1);
	Src.ReceiveBurst(Dst.Writer, &Dst);
	Dst.ReceiveBurst(Src.Writer, &Src);

	// Verify that we recovered on both ends
	UE_NET_ASSERT_FALSE(Src.PacketNotify.IsWaitingForSequenceHistoryFlush());
	UE_NET_ASSERT_FALSE(Dst.PacketNotify.IsWaitingForSequenceHistoryFlush());

	// Now we should be able to acknowledge data again
	Src.SendAndDeliverTo(Dst);
	Dst.SendAndDeliverTo(Src);

	// Verify that we delivered what we expected
	UE_NET_ASSERT_TRUE(Src.ValidateAcknowledgedPackets(Dst.AcceptedInPackets));
	// Dst will not the latest packet accepted by Src.
	UE_NET_ASSERT_TRUE(Dst.ValidateAcknowledgedPackets(MakeArrayView(Src.AcceptedInPackets.GetData(), Src.AcceptedInPackets.Num() - 1)));
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, TestGetBothEndsWaitForSequenceWindowFlushDueToBurstPacketLoss)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetTraffic, ELogVerbosity::Error);

	FNetPacketNotify::SequenceNumberT SrcStartingSeq(1024);
	FNetPacketNotify::SequenceNumberT DstStartingSeq(0);

	FTestNode Src(DstStartingSeq, SrcStartingSeq + 1);
	FTestNode Dst(SrcStartingSeq, DstStartingSeq + 1);

	// Send off bursts on both ends, over committing send window but do not deliver
	Src.SendBurst(FNetPacketNotify::MaxSequenceHistoryLength + 1);
	Dst.SendBurst(FNetPacketNotify::MaxSequenceHistoryLength + 1);

	// Send new data that we will deliver, which should SequenceHistoryFlush but with no previously acked data we can do immediate reset
	Src.SendBurst(1);
	Dst.SendBurst(1);
	
	// Receive the data
	Dst.ReceiveBurst(Src.Writer);
	Src.ReceiveBurst(Dst.Writer);

	// Verify that we managed to avoid requesting a flush
	UE_NET_ASSERT_FALSE(Src.PacketNotify.IsWaitingForSequenceHistoryFlush());
	UE_NET_ASSERT_FALSE(Dst.PacketNotify.IsWaitingForSequenceHistoryFlush());

	// Verify that we delivered what we expected
	UE_NET_ASSERT_TRUE(Src.ValidateAcceptedPackets({DstStartingSeq.Get() + FNetPacketNotify::MaxSequenceHistoryLength + 2U}));
	UE_NET_ASSERT_TRUE(Dst.ValidateAcceptedPackets({SrcStartingSeq.Get() + FNetPacketNotify::MaxSequenceHistoryLength + 2U}));
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, TestGetBothEndsWaitForSequenceWindowFlushWithPartiallyFullHistory)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetTraffic, ELogVerbosity::Error);

	FNetPacketNotify::SequenceNumberT SrcStartingSeq(1024);
	FNetPacketNotify::SequenceNumberT DstStartingSeq(0);

	FTestNode Src(DstStartingSeq, SrcStartingSeq + 1);
	FTestNode Dst(SrcStartingSeq, DstStartingSeq + 1);

	// Send off a range with a packet to deliver and a bunch of missing ones, overshooting the sequence history
	Src.SendRange({SrcStartingSeq.Get() + FNetPacketNotify::MaxSequenceHistoryLength/2U, SrcStartingSeq.Get() + FNetPacketNotify::MaxSequenceHistoryLength + 1U});
	Dst.SendRange({DstStartingSeq.Get() + FNetPacketNotify::MaxSequenceHistoryLength/2U, DstStartingSeq.Get() + FNetPacketNotify::MaxSequenceHistoryLength + 1U});

	// Receive the data
	Dst.ReceiveBurst(Src.Writer);
	Src.ReceiveBurst(Dst.Writer);

	// We only expect the first packet to be accepted
	Src.ExpectedAcceptedInPackets.Add(DstStartingSeq.Get() + FNetPacketNotify::MaxSequenceHistoryLength/2U);
	Dst.ExpectedAcceptedInPackets.Add(SrcStartingSeq.Get() + FNetPacketNotify::MaxSequenceHistoryLength/2U);

	// Verify that we delivered what we expected
	UE_NET_ASSERT_TRUE(Src.ValidateExpectedAcceptedPackets());
	UE_NET_ASSERT_TRUE(Dst.ValidateExpectedAcceptedPackets());

	// Verify that we as expected triggered a SequenceHistoryFlush
	UE_NET_ASSERT_TRUE(Src.PacketNotify.IsWaitingForSequenceHistoryFlush());
	UE_NET_ASSERT_TRUE(Dst.PacketNotify.IsWaitingForSequenceHistoryFlush());

	// Tell remote that we are flushing sequence history
	Src.SendBurst(1);
	Dst.SendBurst(1);
	Dst.ReceiveBurst(Src.Writer, &Src);
	Src.ReceiveBurst(Dst.Writer, &Dst);

	// Verify that we discarded the packets as both ends are waiting for SequenceHistoryFlush
	UE_NET_ASSERT_TRUE(Src.ValidateExpectedAcceptedPackets());
	UE_NET_ASSERT_TRUE(Dst.ValidateExpectedAcceptedPackets());

	// Send a burst, overshooting while in recovery (which should be fine)
	Src.SendBurst(FNetPacketNotify::MaxSequenceHistoryLength);
	Dst.SendBurst(FNetPacketNotify::MaxSequenceHistoryLength);

	// We expect next sent packets to be delivered
	Src.ExpectedAcceptedInPackets.Add(Dst.OutPacketId);
	Dst.ExpectedAcceptedInPackets.Add(Src.OutPacketId);

	// Deliver some data, completing the flush
	Src.SendBurst(1);
	Dst.SendBurst(1);
	Dst.ReceiveBurst(Src.Writer, &Src);
	Src.ReceiveBurst(Dst.Writer, &Dst);

	// Verify that we managed to recover and are ready to proceed as normal
	UE_NET_ASSERT_FALSE(Src.PacketNotify.IsWaitingForSequenceHistoryFlush());
	UE_NET_ASSERT_FALSE(Dst.PacketNotify.IsWaitingForSequenceHistoryFlush());

	// Verify that we delivered what we expected
	UE_NET_ASSERT_TRUE(Src.ValidateExpectedAcceptedPackets());
	UE_NET_ASSERT_TRUE(Dst.ValidateExpectedAcceptedPackets());
}

UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, TestSendWithDeferredAck)
{
	FTestNode Src;
	FTestNode Dst;

	Src.SendBurst(1);
	Dst.ReceiveWithCallback(Src.Writer,
	[&Dst]() {
		Dst.SendBurst(1);
	}, &Src);

	Src.Receive(Dst.Writer, &Dst);
}

// Stress test to find random cases.
#if 0
UE_NET_TEST_FIXTURE(FNetPacketNotifyTestUtil, TestGetBothEndsWaitForSequenceWindowFlushRandomized)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetTraffic, ELogVerbosity::Verbose);

	FNetPacketNotify::SequenceNumberT SrcStartingSeq((uint16)FMath::RandRange(0, (int32)FNetPacketNotify::SequenceNumberT::SeqNumberMax));
	FNetPacketNotify::SequenceNumberT DstStartingSeq((uint16)FMath::RandRange(0, (int32)FNetPacketNotify::SequenceNumberT::SeqNumberMax));

	FTestNode Src(DstStartingSeq, SrcStartingSeq + 1);;
	FTestNode Dst(SrcStartingSeq, DstStartingSeq + 1);;
	
	SIZE_T TestIterations = 1024;

	uint32 SrcHitFlushCount = 0U;
	uint32 DstHitFlushCount = 0U;
	uint32 SrcResetFlushCount = 0U;
	uint32 DstResetFlushCount = 0U;
	uint32 SrcStuckFlushCount = 0U;
	uint32 DstStuckFlushCount = 0U;

	FMath::SRandInit(347856243);
	FMath::RandInit(347856243);

	const uint32 DeltaBatch = 128; //32
	while(TestIterations--)
	{
		// Verify that we as expected triggered a SequenceHistoryFlush
		bool bSrcWasFlush = Src.PacketNotify.IsWaitingForSequenceHistoryFlush();
		bool bDstWasFlush = Dst.PacketNotify.IsWaitingForSequenceHistoryFlush();

		SrcStuckFlushCount += bSrcWasFlush ? 1U : 0U;
		DstStuckFlushCount += bDstWasFlush ? 1U : 0U;

		{
			int32 NumPacketsToSend = FMath::RandRange(0, DeltaBatch);
			Src.SendBurst(NumPacketsToSend);
		}
		{
			int32 NumPacketsToSend = FMath::RandRange(0, DeltaBatch);
			Dst.SendBurst(NumPacketsToSend);
		}
		if (FMath::RandBool())
		{
			// Receive and verify success
			UE_NET_ASSERT_TRUE(Dst.ReceiveBurst(Src.Writer, &Src));
		}
		if (FMath::RandBool())
		{
			// Receive and verify success
			UE_NET_ASSERT_TRUE(Src.ReceiveBurst(Dst.Writer, &Dst));
		}
		
		if (Src.PacketNotify.IsWaitingForSequenceHistoryFlush())
		{			
			SrcHitFlushCount += bSrcWasFlush ? 0U : 1U;
		}
		else
		{
			SrcResetFlushCount += bSrcWasFlush ? 1U : 0U;			
			SrcStuckFlushCount = 0U;
		}
		if (Dst.PacketNotify.IsWaitingForSequenceHistoryFlush())
		{			
			DstHitFlushCount += bDstWasFlush ? 0U : 1U;
		}
		else
		{
			DstResetFlushCount += bDstWasFlush ? 1U : 0U;
			DstStuckFlushCount = 0U;
		}

		UE_NET_ASSERT_TRUE(SrcStuckFlushCount < 100);
		UE_NET_ASSERT_TRUE(DstStuckFlushCount < 100);
	};

	Src.SendBurst(1);
	Dst.SendBurst(1);
	Dst.ReceiveBurst(Src.Writer, &Src);
	Src.ReceiveBurst(Dst.Writer, &Dst);

	Src.SendBurst(1);
	Dst.SendBurst(1);
	Dst.ReceiveBurst(Src.Writer, &Src);
	Src.ReceiveBurst(Dst.Writer, &Dst);

	// Verify that we acknowledged everything we accepted on both ends.
	UE_NET_ASSERT_TRUE(Src.ValidateAcknowledgedPackets(MakeArrayView(Dst.AcceptedInPackets.GetData(), Dst.AcceptedInPackets.Num() - 1)));
	UE_NET_ASSERT_TRUE(Dst.ValidateAcknowledgedPackets(MakeArrayView(Src.AcceptedInPackets.GetData(), Src.AcceptedInPackets.Num() - 1)));

	// Verify that we managed to recover
	UE_NET_ASSERT_FALSE(Src.PacketNotify.IsWaitingForSequenceHistoryFlush());
	UE_NET_ASSERT_FALSE(Dst.PacketNotify.IsWaitingForSequenceHistoryFlush());
}
#endif

}
}

