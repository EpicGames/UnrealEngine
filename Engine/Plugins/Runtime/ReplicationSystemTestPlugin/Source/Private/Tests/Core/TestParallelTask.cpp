// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/Polling/ObjectPoller.h"
#include "Async/TaskGraphInterfaces.h"

namespace UE::Net::Private
{

class FTestParallelTaskFixture : public FReplicationSystemServerClientTestFixture
{
protected:
	virtual void SetUp() override
	{
		ServerParamsOverride.bAllowParallelTasks = true;

		FReplicationSystemServerClientTestFixture::SetUp();

		// Add a client
		Client = CreateClient();
	}
	
	FReplicationSystemTestClient* Client = nullptr;
};

UE_NET_TEST_FIXTURE(FTestParallelTaskFixture, PollTask)
{	
	//This test is designed to make sure that we're polling the correct number of objects when running Parallel Tasks
	
	const uint32 NumReplicatedTestObjects = 2048;// We need enough objects that several tasks have work do
	for (uint32 i = 0; i < NumReplicatedTestObjects; ++i)
	{
		UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	}
	
	Server->NetUpdate();

	const FNetRefHandleManager& NetRefHandleManager = Server->GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager();
	FNetBitArrayView ObjectsConsideredForPolling = NetRefHandleManager.GetAssignedInternalIndicesView();
	ensure(ObjectsConsideredForPolling.IsAnyBitSet());


	//Set up object poller and track number of objects polled
	FObjectPoller::FInitParams PollerInitParams;
	PollerInitParams.ObjectReplicationBridge = Server->GetReplicationBridge();
	PollerInitParams.ReplicationSystemInternal = Server->GetReplicationSystem()->GetReplicationSystemInternal();

	FObjectPoller Poller(PollerInitParams);
	Poller.PollAndCopyObjects(ObjectsConsideredForPolling);
	
	FObjectPoller::FPreUpdateAndPollStats PollStats = Poller.GetPollStats();

	UE_NET_ASSERT_EQ(NumReplicatedTestObjects, PollStats.PolledObjectCount);

	Server->PostSendUpdate();
}

} // end namespace UE::Net::Private
