// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockDataStream.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializationContext.h"

UMockDataStream::UMockDataStream()
: UDataStream()
, CallStatus({})
, CallSetup({})
{
}

UDataStream::EWriteResult UMockDataStream::WriteData(UE::Net::FNetSerializationContext& Context, FDataStreamRecord const*& OutRecord)
{
	++CallStatus.WriteDataCallCount;

	UE::Net::FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	if (CallSetup.WriteDataBitCount > 0)
	{
		Writer->Seek(Writer->GetPosBits() + CallSetup.WriteDataBitCount);
	}

	if (CallSetup.WriteDataReturnValue != EWriteResult::NoData)
	{
		FRecord& Record = Records.Enqueue();
		Record.MagicValue = CallSetup.WriteDataRecordMagicValue;
		OutRecord = &Record;
	}

	return CallSetup.WriteDataReturnValue;
}

void UMockDataStream::ReadData(UE::Net::FNetSerializationContext& Context)
{
	++CallStatus.ReadDataCallCount;

	UE::Net::FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	if (CallSetup.ReadDataBitCount > 0)
	{
		Reader->Seek(Reader->GetPosBits() + CallSetup.ReadDataBitCount);
	}
}

void UMockDataStream::ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status, FDataStreamRecord const* InRecord)
{
	++CallStatus.ProcessPacketDeliveryStatusCallCount;

	check(InRecord != nullptr);
	const FRecord* Record = static_cast<const FRecord*>(InRecord);
	check(&Records.Peek() == Record);
	Records.Pop();
	CallStatus.ProcessPacketDeliveryStatusMagicValue = Record->MagicValue;
}

bool UMockDataStream::HasAcknowledgedAllReliableData() const
{
	return true;
}

void UMockDataStream::Update(const FUpdateParameters& Params)
{
	++CallStatus.UpdateCallCount;
}
