// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/DataStream/DataStream.h"
#include "Net/Core/Misc/ResizableCircularQueue.h"
#include "MockDataStream.generated.h"

UCLASS()
class UMockDataStream : public UDataStream 
{
	GENERATED_BODY()

public:
	struct FFunctionCallStatus
	{
		uint32 WriteDataCallCount;
		uint32 ReadDataCallCount;
		uint32 ProcessPacketDeliveryStatusCallCount;
		uint32 ProcessPacketDeliveryStatusMagicValue;
		uint32 UpdateCallCount;
	};

	struct FFunctionCallSetup
	{
		// WriteData
		EWriteResult WriteDataReturnValue;
		uint32 WriteDataBitCount;
		uint32 WriteDataRecordMagicValue;

		// ReadData
		uint32 ReadDataBitCount;
	};

	inline void SetFunctionCallSetup(const FFunctionCallSetup& Setup) { CallSetup = Setup; }
	inline const FFunctionCallStatus& GetFunctionCallStatus() const { return CallStatus; }
	inline void ResetFunctionCallStatus() { CallStatus = FFunctionCallStatus({}); }

protected:
	virtual EWriteResult WriteData(UE::Net::FNetSerializationContext& context, FDataStreamRecord const*& OutRecord) override;
	virtual void ReadData(UE::Net::FNetSerializationContext& context) override;
	virtual void ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status, FDataStreamRecord const* Record) override;
	virtual bool HasAcknowledgedAllReliableData() const override;
	virtual void Update(const FUpdateParameters& Params) override;

private:
	struct FRecord : public FDataStreamRecord
	{
		uint32 MagicValue;
	};

	UMockDataStream();

	FFunctionCallStatus CallStatus;
	FFunctionCallSetup CallSetup;
	TResizableCircularQueue<FRecord, TFixedAllocator<256>> Records;
};
