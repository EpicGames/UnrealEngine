// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/LogScopedVerbosityOverride.h"
#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"

#include "Iris/Core/IrisLog.h"
#include "Iris/DataStream/DataStreamManager.h"
#include "Iris/DataStream/DataStreamDefinitions.h"
#include "Iris/DataStream/DataStreamManager.h"
#include "Iris/ReplicationSystem/ChunkedDataStream/ChunkedDataStream.h"
#include "Iris/ReplicationSystem/ChunkedDataStream/ChunkedDataStreamCommon.h"

#include "Iris/PacketControl/PacketNotification.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/NetTokenStore.h"
#include "Iris/ReplicationSystem/StringTokenStore.h"
#include "Iris/ReplicationSystem/NetTokenDataStream.h"
#include "Templates/Casts.h"

#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

namespace UE::Net::Private
{

class FTestChunkedDataStreamFixture : public FReplicationSystemServerClientTestFixture
{
protected:
	virtual void SetUp() override
	{
		FReplicationSystemServerClientTestFixture::SetUp();

		// We can override what we want as long as we do it before any connections are created.
		DataStreamUtil.SetUp();

		// Add a dynamic DataStream
		{
			FDataStreamTestUtil::FAddDataStreamDefinitionParams Params;
			Params.bDynamicCreate = true;
			DataStreamUtil.AddDataStreamDefinition(TEXT("ChunkedData"), TEXT("/Script/IrisCore.ChunkedDataStream"), Params);	
		}

		{
			DataStreamUtil.AddDataStreamDefinition(TEXT("Replication"), TEXT("/Script/IrisCore.ReplicationDataStream"));
		}

		DataStreamUtil.FixupDefinitions();

		// Add a client
		Client = CreateClient();
	}

	UDataStream::EDataStreamState GetDataStreamStateOnServer() const
	{
		return Server->GetConnectionInfo(Client->ConnectionIdOnServer).DataStreamManager->GetStreamState(DataStreamName);
	}

	UDataStream::EDataStreamState GetDataStreamStateOnClient() const
	{
		return Client->GetConnectionInfo(Client->LocalConnectionId).DataStreamManager->GetStreamState(DataStreamName);
	}

	void RoundTrip(bool bDeliver = true)
	{
		Server->UpdateAndSend({Client}, bDeliver);
		Client->UpdateAndSend(Server, true);
	}

	void RoundTripWithLatencyAndDeliverPercentage(uint32 InFlightCount = 1, int32 DropPercentage = 10, int32 PacketMinSize = 128, int32 PacketMaxSize = 2048)
	{
		uint32 SentCount = 0U;
		for (uint32 It = 0; It <= InFlightCount; ++It)
		{
			Server->SetMaxSendPacketSize(FMath::RandRange(PacketMinSize, PacketMaxSize) & ~0x3);
			Server->NetUpdate();

			SentCount += Server->SendTo(Client) ? 1U : 0U;

			Server->PostSendUpdate();
		}

		while (SentCount--)
		{
			const bool bDeliver =  FMath::RandRange(0, 100) > DropPercentage;
			Server->DeliverTo(Client, bDeliver);
		}
				
		Client->UpdateAndSend(Server, true);
	}

	const FName DataStreamName = "ChunkedData";
	FReplicationSystemTestClient* Client = nullptr;
	FDataStreamTestUtil DataStreamUtil;
};

// Basic functionality test
UE_NET_TEST_FIXTURE(FTestChunkedDataStreamFixture, TestChunkedDataStream)
{
	UReplicationSystem* ServerReplicationSystem = Server->ReplicationSystem;
	UReplicationSystem* ClientReplicationSystem = Client->ReplicationSystem;

	// Open stream
	UChunkedDataStream* ServerStream = Cast<UChunkedDataStream>(ServerReplicationSystem->OpenDataStream(Client->ConnectionIdOnServer, DataStreamName));

	// Double roundtrip to ensure that stream is fully open/acked
	RoundTrip();
	RoundTrip();

	UChunkedDataStream* ClientStream = Cast<UChunkedDataStream>(ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName));

	// Verify that stream is open
	UE_NET_ASSERT_TRUE(ClientStream->GetState() == UDataStream::EDataStreamState::Open);

	// Send a payload
	TSharedRef<TArray64<uint8>> Payload = MakeShared<TArray64<uint8>>();
	Payload->SetNumZeroed(1567);
	ServerStream->EnqueuePayload(Payload);

	// Send some data
	RoundTrip();
	RoundTrip();

	// Verify that we have got the payload
	UE_NET_ASSERT_EQ(ClientStream->GetNumReceivedPayloadsPendingDispatch(), 1U);

	// Loop until we have received the payload
	bool bHasReceivedPayload = false;	

	// Dispatch
	ClientStream->DispatchReceivedPayload([this, &Payload, &bHasReceivedPayload](TConstArrayView64<uint8> ReceivedPayload)
	{
		bHasReceivedPayload = true;
		UE_NET_ASSERT_EQ(ReceivedPayload.Num(), Payload->Num());
	});

	UE_NET_ASSERT_TRUE(bHasReceivedPayload);

	// Verify that we have dispatched all received payloads
	UE_NET_ASSERT_EQ(ClientStream->GetNumReceivedPayloadsPendingDispatch(), 0U);
}

// Verify that stream can be closed from client
UE_NET_TEST_FIXTURE(FTestChunkedDataStreamFixture, TestChunkedDataStream_RequestCloseFromClient)
{
	UReplicationSystem* ServerReplicationSystem = Server->ReplicationSystem;
	UReplicationSystem* ClientReplicationSystem = Client->ReplicationSystem;

	// Open stream
	UChunkedDataStream* ServerStream = Cast<UChunkedDataStream>(ServerReplicationSystem->OpenDataStream(Client->ConnectionIdOnServer, DataStreamName));

	RoundTrip();
	RoundTrip();

	UChunkedDataStream* ClientStream = Cast<UChunkedDataStream>(ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName));

	// Verify open
	UE_NET_ASSERT_TRUE(ClientStream != nullptr);

	// Request close from client
	ClientStream->RequestClose();

	// Verify that stream is open on both sides
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Open);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::PendingClose);

	// Send and deliver to server
	Client->UpdateAndSend(Server, true);
	
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::PendingClose);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::WaitOnCloseConfirmation);

	// Double roundtrip and we should be done
	RoundTrip();
	RoundTrip();

	// Verify that stream is invalidated
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Invalid);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::Invalid);	
}

// Send some payloads with varying size
UE_NET_TEST_FIXTURE(FTestChunkedDataStreamFixture, TestChunkedDataStream_SendManyWithVaryingPacketSizes)
{
	UReplicationSystem* ServerReplicationSystem = Server->ReplicationSystem;
	UReplicationSystem* ClientReplicationSystem = Client->ReplicationSystem;

	// Open stream
	UChunkedDataStream* ServerStream = Cast<UChunkedDataStream>(ServerReplicationSystem->OpenDataStream(Client->ConnectionIdOnServer, DataStreamName));
	RoundTrip();
	RoundTrip();
	UChunkedDataStream* ClientStream = Cast<UChunkedDataStream>(ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName));

	// Send payloads
	const uint32 NumPayloads = 313;
	TSharedPtr<TArray64<uint8>> Payloads[NumPayloads];
	for (uint32 It=0; It<NumPayloads; ++It)
	{
		Payloads[It] = MakeShared<TArray64<uint8>>();
		Payloads[It]->SetNumZeroed(FMath::RandRange(10, 5000));
		ServerStream->EnqueuePayload(Payloads[It]);
	}

	// Loop send and dispatch until we have received and dispatched all data.
	uint32 NumRecvdPayloads = 0;	
	while (NumRecvdPayloads < NumPayloads)
	{
		// Send data with varying packet loss and unacked packets in-flight
		RoundTripWithLatencyAndDeliverPercentage(128);

		// Verify received payloads
		ClientStream->DispatchReceivedPayloads([this, &Payloads, &NumRecvdPayloads](TConstArrayView64<uint8> Data)
		{
			UE_NET_ASSERT_EQ(Data.Num(), Payloads[NumRecvdPayloads]->Num());
			++NumRecvdPayloads;
		});
	}
	UE_NET_ASSERT_EQ(NumRecvdPayloads, NumPayloads);
}

// Send some payloads with varying size and exports
UE_NET_TEST_FIXTURE(FTestChunkedDataStreamFixture, TestChunkedDataStream_SendManyWithVaryingPacketSizesAndExports)
{
	UReplicationSystem* ServerReplicationSystem = Server->ReplicationSystem;
	UReplicationSystem* ClientReplicationSystem = Client->ReplicationSystem;

	// Open stream
	UChunkedDataStream* ServerStream = Cast<UChunkedDataStream>(ServerReplicationSystem->OpenDataStream(Client->ConnectionIdOnServer, DataStreamName));
	RoundTrip();
	RoundTrip();
	UChunkedDataStream* ClientStream = Cast<UChunkedDataStream>(ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName));

	// Some refs to test exports
	UObject* ObjectRefs[] =
	{
		nullptr,
		UTestReplicatedIrisObject::StaticClass(),
		UReplicationSystem::StaticClass(),
		UReplicatedTestObject::StaticClass()
	};
	const int32 MaxObjectRefsIndex = UE_ARRAY_COUNT(ObjectRefs);

	// Send some data
	const uint32 NumPayloads = 64;
	TSharedPtr<TArray64<uint8>> Payloads[NumPayloads];
	TArray<uint32> WrittenExportIndices[NumPayloads];

	for (uint32 It=0; It<NumPayloads; ++It)
	{
		// We use this to capture exports written
		FChunkedDataStreamExportWriteScope ScopedExports(ServerStream);

		Payloads[It] = MakeShared<TArray64<uint8>>();
		Payloads[It]->SetNumZeroed(FMath::RandRange(10, 5000));

		// Randomize a few object references and write them to the paylaod using the PackageMap associated with the DataStream
		{
			FMemoryWriter64 Ar(*Payloads[It]);

			int32 ReferenceCount = FMath::RandRange(0, MaxObjectRefsIndex);
			Ar << ReferenceCount;
			for (int32 RefIt = 0; RefIt < ReferenceCount; ++RefIt)
			{
				const int32 ReferenceIndex = FMath::RandRange(0, MaxObjectRefsIndex - 1);
				WrittenExportIndices[It].Add(ReferenceIndex);
				ScopedExports.GetPackageMap()->SerializeObject(Ar, UObject::StaticClass(), ObjectRefs[ReferenceIndex], nullptr);
			}
		}

		ServerStream->EnqueuePayload(Payloads[It]);
	}

	uint32 NumRecvdPayloads = 0;	
	while (NumRecvdPayloads < NumPayloads)
	{
		RoundTripWithLatencyAndDeliverPercentage(0, 0, 1500, 1500);

		// Init Packagemap for reading references
		FChunkedDataStreamExportReadScope ScopedExports(ClientStream);

		ClientStream->DispatchReceivedPayloads([this, &ScopedExports, &Payloads, &ObjectRefs, &WrittenExportIndices, &NumRecvdPayloads](TConstArrayView64<uint8> Data)
		{
			UE_NET_ASSERT_EQ(Data.Num(), Payloads[NumRecvdPayloads]->Num());

			// Read payload including references and validate them.
			{
				FMemoryReaderView Ar(MakeMemoryView(Data.GetData(), Data.Num()));

				// Read reference count.
				int32 ReferenceCount = 0U;
				Ar << ReferenceCount;

				// Validate that we recevied the expected number of references
				UE_NET_ASSERT_EQ(ReferenceCount, WrittenExportIndices[NumRecvdPayloads].Num());

				// Validate references
				for (int32 RefIt = 0; RefIt < ReferenceCount; ++RefIt)
				{
					UObject* SomePtr = nullptr;
					ScopedExports.GetPackageMap()->SerializeObject(Ar, UObject::StaticClass(), SomePtr, nullptr);

					const int32 ReferenceIndex = WrittenExportIndices[NumRecvdPayloads][RefIt];
					UE_NET_ASSERT_EQ(SomePtr, ObjectRefs[ReferenceIndex]);
				}
			}

			++NumRecvdPayloads;
		});
	}
	UE_NET_ASSERT_EQ(NumRecvdPayloads, NumPayloads);
}

UE_NET_TEST_FIXTURE(FTestChunkedDataStreamFixture, TestChunkedDataStream_CannotEnqueueMoreThanMaxEnqueuedPayloadBytes)
{
	UReplicationSystem* ServerReplicationSystem = Server->ReplicationSystem;
	UReplicationSystem* ClientReplicationSystem = Client->ReplicationSystem;

	// Open stream
	UChunkedDataStream* ServerStream = Cast<UChunkedDataStream>(ServerReplicationSystem->OpenDataStream(Client->ConnectionIdOnServer, DataStreamName));
	RoundTrip();
	RoundTrip();

	UChunkedDataStream* ClientStream = Cast<UChunkedDataStream>(ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName));

	// Set limit
	ServerStream->SetMaxEnqueuedPayloadBytes(1500);

	// Try to enqueue some data
	{
		TSharedRef<TArray64<uint8>> Payload = MakeShared<TArray64<uint8>>();
		Payload->SetNumZeroed(1024);
		const bool bResult = ServerStream->EnqueuePayload(Payload);
		UE_NET_ASSERT_TRUE(bResult);
	}

	// Suppress error, since we're intentionally overflowing
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogIrisChunkedDataStream, ELogVerbosity::Fatal);

	// Try to enqueue some data that should not be allowed
	{
		TSharedRef<TArray64<uint8>> Payload = MakeShared<TArray64<uint8>>();
		Payload->SetNumZeroed(1024);
		const bool bResult = ServerStream->EnqueuePayload(Payload);
		UE_NET_ASSERT_FALSE(bResult);
	}
}

UE_NET_TEST_FIXTURE(FTestChunkedDataStreamFixture, TestChunkedDataStream_ClientWillSetErrorIfTooManyUndispatchedPayloadBytes)
{
	UReplicationSystem* ServerReplicationSystem = Server->ReplicationSystem;
	UReplicationSystem* ClientReplicationSystem = Client->ReplicationSystem;

	// Open stream
	UChunkedDataStream* ServerStream = Cast<UChunkedDataStream>(ServerReplicationSystem->OpenDataStream(Client->ConnectionIdOnServer, DataStreamName));
	RoundTrip();
	RoundTrip();

	UChunkedDataStream* ClientStream = Cast<UChunkedDataStream>(ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName));

	// Set limit on receiving end on how much undispatched data we should allow to be queued up.
	ClientStream->SetMaxUndispatchedPayloadBytes(2500);

	// Enqueue some data
	{
		TSharedRef<TArray64<uint8>> Payload = MakeShared<TArray64<uint8>>();
		Payload->SetNumZeroed(1024);
		ServerStream->EnqueuePayload(Payload);
	}
	{
		TSharedRef<TArray64<uint8>> Payload = MakeShared<TArray64<uint8>>();
		Payload->SetNumZeroed(1024);
		ServerStream->EnqueuePayload(Payload);
	}

	// Tick send/receive
	RoundTrip();
	RoundTrip();

	// Verify that we have got the first two expected payloads
	UE_NET_ASSERT_EQ(ClientStream->GetNumReceivedPayloadsPendingDispatch(), 2U);

	// No error detected
	UE_NET_ASSERT_FALSE(ClientStream->HasError());

	// Send more data that will exceed the set limit.
	{
		TSharedRef<TArray64<uint8>> Payload = MakeShared<TArray64<uint8>>();
		Payload->SetNumZeroed(10000);
		ServerStream->EnqueuePayload(Payload);
	}


	// Suppress error, since we're intentionally overflowing
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogIrisChunkedDataStream, ELogVerbosity::Fatal);

	// Tick send/receive
	RoundTrip();

	// Stream should now be in error state, and will ignore incoming chunks (still read from bitstream but they will be discarded.)
	UE_NET_ASSERT_TRUE(ClientStream->HasError());

	// Tick send/receive
	RoundTrip();
	RoundTrip();
	RoundTrip();
	RoundTrip();
	RoundTrip();
	RoundTrip();

	// Verify that we have got the first two expected payloads
	UE_NET_ASSERT_EQ(ClientStream->GetNumReceivedPayloadsPendingDispatch(), 2U);
}

}	
