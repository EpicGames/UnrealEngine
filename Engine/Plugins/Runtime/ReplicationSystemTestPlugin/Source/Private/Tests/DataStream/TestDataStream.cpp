// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/LogScopedVerbosityOverride.h"
#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "MockDataStream.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/DataStream/DataStreamManager.h"
#include "Iris/DataStream/DataStreamDefinitions.h"
#include "Iris/DataStream/DataStreamManager.h"
#include "Iris/PacketControl/PacketNotification.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/NetTokenStore.h"
#include "Iris/ReplicationSystem/StringTokenStore.h"
#include "Iris/ReplicationSystem/NetTokenDataStream.h"
#include "Templates/Casts.h"

namespace UE::Net::Private
{

// These test cannot run in parallel with other code accessing data streams, like DataStreamManager and DataStreamDefinitions.
class FTestDataStream : public FNetworkAutomationTestSuiteFixture
{
public:
	FTestDataStream();

	virtual void SetUp() override;
	virtual void TearDown() override;

private:
	class UMyDataStreamDefinitions : public UDataStreamDefinitions
	{
	public:
		void FixupDefinitions() { return UDataStreamDefinitions::FixupDefinitions(); }
	};

	void StoreDataStreamDefinitions();
	void RestoreDataStreamDefinitions();
	void CreateMockDataStreamDefinition(FDataStreamDefinition& Definition, bool bValid);

protected:
	void AddMockDataStreamDefinition(bool bValid = true);
	UMockDataStream* CreateMockStream(const UMockDataStream::FFunctionCallSetup* Setup = nullptr);
	FNetSerializationContext& CreateDataStreamContext();
	void InitBitStreamReaderFromWriter();

	UDataStreamManager* DataStreamManager;
	UMyDataStreamDefinitions* DataStreamDefinitions;
	TArray<FDataStreamDefinition>* CurrentDataStreamDefinitions;
	TArray<FDataStreamDefinition> PreviousDataStreamDefinitions;
	bool* PointerToFixupComplete;

	//
	FNetSerializationContext DataStreamContext;
	FNetBitStreamReader BitStreamReader;
	FNetBitStreamWriter BitStreamWriter;
	alignas(16) uint8 BitStreamStorage[128];
};

FTestMessage& operator<<(FTestMessage& Ar, const UDataStream::EWriteResult WriteResult);

//

UE_NET_TEST_FIXTURE(FTestDataStream, CanCreateDataStream)
{
	constexpr bool bAddValidDefinition = true;
	AddMockDataStreamDefinition(bAddValidDefinition);

	ECreateDataStreamResult Result = DataStreamManager->CreateStream("Mock");
	UE_NET_ASSERT_EQ(unsigned(Result), unsigned(ECreateDataStreamResult::Success));
}

UE_NET_TEST_FIXTURE(FTestDataStream, CannotCreateSameDataStreamTwice)
{
	constexpr bool bAddValidDefinition = true;
	AddMockDataStreamDefinition(bAddValidDefinition);

	DataStreamManager->CreateStream("Mock");
	
	// Suppress Iris internal warning, since we're intentionally creating duplicate streams.
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Fatal);
	ECreateDataStreamResult Result = DataStreamManager->CreateStream("Mock");
	UE_NET_ASSERT_EQ(unsigned(Result), unsigned(ECreateDataStreamResult::Error_Duplicate));
}

UE_NET_TEST_FIXTURE(FTestDataStream, CannotCreateInvalidDataStream)
{
	constexpr bool bAddValidDefinition = false;

	// Suppress Iris internal error, since we're intentionally creating an invalid stream.
	{
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Fatal);
		AddMockDataStreamDefinition(bAddValidDefinition);
	}

	ECreateDataStreamResult Result = DataStreamManager->CreateStream("Mock");
	UE_NET_ASSERT_EQ(unsigned(Result), unsigned(ECreateDataStreamResult::Error_InvalidDefinition));
}

UE_NET_TEST_FIXTURE(FTestDataStream, DataStreamGetsWriteDataCall)
{
	UMockDataStream::FFunctionCallSetup MockSetup = {};
	MockSetup.WriteDataBitCount = 0;
	MockSetup.WriteDataReturnValue = UDataStream::EWriteResult::NoData;

	UMockDataStream* Mock = CreateMockStream(&MockSetup);

	FNetSerializationContext& Context = CreateDataStreamContext();
	const FDataStreamRecord* Record = nullptr;
	UDataStream::EWriteResult Result = DataStreamManager->WriteData(Context, Record);

	// Make sure WriteData was called.
	{
		const UMockDataStream::FFunctionCallStatus& CallStatus = Mock->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(CallStatus.WriteDataCallCount, 1U);
	}

	// Even though our data stream isn't writing any data doesn't prevent the manager itself from doing so.
	if (Result != UDataStream::EWriteResult::NoData)
	{
		DataStreamManager->ProcessPacketDeliveryStatus(EPacketDeliveryStatus::Discard, Record);

		// Our stream didn't write anything so should not be called.
		const UMockDataStream::FFunctionCallStatus& CallStatus = Mock->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(CallStatus.ProcessPacketDeliveryStatusCallCount, 0U);
	}
}

UE_NET_TEST_FIXTURE(FTestDataStream, DataStreamGetsProcessPacketDeliveryStatusCall)
{
	UMockDataStream* Mock = CreateMockStream();

	// Make sure the right records are supplied in the PacketDeliveryStatusCall as well

	const uint32 MagicValues[] = {0x35373931U, 0x32312D, 0x36312D};
	constexpr uint32 MagicValueCount = UE_ARRAY_COUNT(MagicValues);
	const FDataStreamRecord* Records[MagicValueCount] = {};
	for (SIZE_T It = 0, EndIt = MagicValueCount; It != EndIt; ++It)
	{
		FNetSerializationContext& Context = CreateDataStreamContext();

		UMockDataStream::FFunctionCallSetup MockSetup = {};
		MockSetup.WriteDataBitCount = 3;
		MockSetup.WriteDataReturnValue = UDataStream::EWriteResult::Ok;
		MockSetup.WriteDataRecordMagicValue = MagicValues[It];
		Mock->SetFunctionCallSetup(MockSetup);
		UDataStream::EWriteResult Result = DataStreamManager->WriteData(Context, Records[It]);
	}

	// Make sure WriteData was called.
	{
		const UMockDataStream::FFunctionCallStatus& CallStatus = Mock->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(CallStatus.WriteDataCallCount, MagicValueCount);
	}

	for (SIZE_T It = 0, EndIt = MagicValueCount; It != EndIt; ++It)
	{
		DataStreamManager->ProcessPacketDeliveryStatus(EPacketDeliveryStatus::Discard, Records[It]);
		const UMockDataStream::FFunctionCallStatus& CallStatus = Mock->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(CallStatus.ProcessPacketDeliveryStatusCallCount, uint32(It + 1));
		UE_NET_ASSERT_EQ(CallStatus.ProcessPacketDeliveryStatusMagicValue, MagicValues[It]);
	}
}

UE_NET_TEST_FIXTURE(FTestDataStream, DataStreamGetsReadDataCall)
{
	UMockDataStream::FFunctionCallSetup MockSetup = {};
	MockSetup.WriteDataBitCount = 15;
	MockSetup.WriteDataReturnValue = UDataStream::EWriteResult::Ok;
	MockSetup.ReadDataBitCount = MockSetup.WriteDataBitCount;

	UMockDataStream* Mock = CreateMockStream(&MockSetup);

	FNetSerializationContext& Context = CreateDataStreamContext();
	const FDataStreamRecord* Record = nullptr;
	DataStreamManager->WriteData(Context, Record);

	// Make sure ReadData was called and all bits have been read.
	{
		const uint32 WriterBitStreamPos = Context.GetBitStreamWriter()->GetPosBits();
		InitBitStreamReaderFromWriter();
		DataStreamManager->ReadData(Context);

		const UMockDataStream::FFunctionCallStatus& CallStatus = Mock->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(CallStatus.ReadDataCallCount, 1U);

		UE_NET_ASSERT_FALSE(Context.HasErrorOrOverflow());

		const uint32 ReaderBitStreamPos = Context.GetBitStreamReader()->GetPosBits();
		UE_NET_ASSERT_EQ(WriterBitStreamPos, ReaderBitStreamPos);
	}

	// Cleanup
	DataStreamManager->ProcessPacketDeliveryStatus(EPacketDeliveryStatus::Discard, Record);
}

UE_NET_TEST_FIXTURE(FTestDataStream, DataStreamGetsUpdateCall)
{
	UMockDataStream::FFunctionCallSetup MockSetup = {};

	UMockDataStream* Mock = CreateMockStream(&MockSetup);

	FNetSerializationContext& Context = CreateDataStreamContext();

	UDataStream::FUpdateParameters DataStreamUpdateParams = { .UpdateType = UDataStream::EUpdateType::PreSendUpdate };
	DataStreamManager->Update(DataStreamUpdateParams);

	// Make sure Update was called
	{
		const UMockDataStream::FFunctionCallStatus& CallStatus = Mock->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(CallStatus.UpdateCallCount, 1U);		
	}
}

// FTestDataStream implementation
FTestDataStream::FTestDataStream()
: FNetworkAutomationTestSuiteFixture()
, DataStreamManager(nullptr)
, DataStreamDefinitions(nullptr)
, CurrentDataStreamDefinitions(nullptr)
, DataStreamContext(&BitStreamReader, &BitStreamWriter)
{
}

void FTestDataStream::SetUp()
{
	UDataStreamManager::FInitParameters InitParams;
	InitParams.PacketWindowSize = 256;
	DataStreamManager = NewObject<UDataStreamManager>();
	DataStreamManager->Init(InitParams);
	StoreDataStreamDefinitions();
}

void FTestDataStream::TearDown()
{
	RestoreDataStreamDefinitions();
	DataStreamManager->Deinit();
	DataStreamManager->MarkAsGarbage();
	DataStreamManager = nullptr;
}

void FTestDataStream::StoreDataStreamDefinitions()
{
	DataStreamDefinitions = static_cast<UMyDataStreamDefinitions*>(GetMutableDefault<UDataStreamDefinitions>());
	check(DataStreamDefinitions != nullptr);
	CurrentDataStreamDefinitions = &DataStreamDefinitions->ReadWriteDataStreamDefinitions();
	PointerToFixupComplete = &DataStreamDefinitions->ReadWriteFixupComplete();

	PreviousDataStreamDefinitions.Empty();
	Swap(*CurrentDataStreamDefinitions, PreviousDataStreamDefinitions);
	*PointerToFixupComplete = false;
}

void FTestDataStream::RestoreDataStreamDefinitions()
{
	Swap(*CurrentDataStreamDefinitions, PreviousDataStreamDefinitions);
	*PointerToFixupComplete = false;
}

void FTestDataStream::CreateMockDataStreamDefinition(FDataStreamDefinition& Definition, bool bValid)
{
	Definition.DataStreamName = FName("Mock");
	Definition.ClassName = bValid ? FName("/Script/ReplicationSystemTestPlugin.MockDataStream") : FName();
	Definition.Class = nullptr;
	Definition.DefaultSendStatus = EDataStreamSendStatus::Send;
	Definition.bAutoCreate = false;
}

void FTestDataStream::AddMockDataStreamDefinition(bool bValid)
{
	FDataStreamDefinition Definition = {};
	CreateMockDataStreamDefinition(Definition, bValid);
	CurrentDataStreamDefinitions->Add(Definition);
	DataStreamDefinitions->FixupDefinitions();
}

UMockDataStream* FTestDataStream::CreateMockStream(const UMockDataStream::FFunctionCallSetup* Setup)
{
	AddMockDataStreamDefinition(true);
	DataStreamManager->CreateStream("Mock");
	UMockDataStream* Stream = StaticCast<UMockDataStream*>(DataStreamManager->GetStream("Mock"));
	if (Stream != nullptr && Setup != nullptr)
	{
		Stream->SetFunctionCallSetup(*Setup);
	}

	return Stream;
}

FNetSerializationContext& FTestDataStream::CreateDataStreamContext()
{
	BitStreamWriter.InitBytes(BitStreamStorage, sizeof(BitStreamStorage));
	// Reset reader
	BitStreamReader.InitBits(BitStreamStorage, 0U);

	return DataStreamContext;
}

void FTestDataStream::InitBitStreamReaderFromWriter()
{
	BitStreamWriter.CommitWrites();
	BitStreamReader.InitBits(BitStreamStorage, BitStreamWriter.GetPosBits());
}

FTestMessage& operator<<(FTestMessage& TestMessage, const UDataStream::EWriteResult WriteResult)
{
	switch (WriteResult)
	{
	case UDataStream::EWriteResult::NoData:
		return TestMessage << TEXT("");
	case UDataStream::EWriteResult::Ok:
		return TestMessage << TEXT("Ok");
	case UDataStream::EWriteResult::HasMoreData:
		return TestMessage << TEXT("HasMoreData");
	default:
		check(false);
		return TestMessage << TEXT("");
	}
}

class FTestDynamicCreateDataStreamFixture : public FReplicationSystemServerClientTestFixture
{
protected:
	virtual void SetUp() override
	{
		FReplicationSystemServerClientTestFixture::SetUp();

		// We can overrida what we want as long as we do it before any connections are created.
		DataStreamUtil.SetUp();

		// Add a dynamic DataStream
		{
			FDataStreamTestUtil::FAddDataStreamDefinitionParams Params;
			Params.bDynamicCreate = true;
			DataStreamUtil.AddDataStreamDefinition(TEXT("DynamicNetToken"), TEXT("/Script/IrisCore.NetTokenDataStream"), Params);	
		}

		{
			DataStreamUtil.AddDataStreamDefinition(TEXT("Replication"), TEXT("/Script/IrisCore.ReplicationDataStream"));
		}

		DataStreamUtil.FixupDefinitions();

		// Add a client
		Client = CreateClient();

		// Cache some data
		// Server
		{
			FNetTokenStore* TokenStore = Server->GetReplicationSystem()->GetNetTokenStore();
			ServerStringTokenStore = TokenStore->GetDataStore<FStringTokenStore>();
			ServerRemoteNetTokenStoreState = TokenStore->GetRemoteNetTokenStoreState(Client->ConnectionIdOnServer);
		}

		// Client
		{
			FNetTokenStore* TokenStore = Client->GetReplicationSystem()->GetNetTokenStore();
			ClientStringTokenStore = TokenStore->GetDataStore<FStringTokenStore>();
			ClientRemoteNetTokenStoreState = TokenStore->GetRemoteNetTokenStoreState(Client->LocalConnectionId);
		}
	}

	FNetToken CreateAndExportTokenToClient(const FString& TokenString)
	{
		const FNetToken Token = ServerStringTokenStore->GetOrCreateToken(TokenString);
		UNetTokenDataStream* NetTokenDataStream = Cast<UNetTokenDataStream>(Server->GetReplicationSystem()->GetDataStream(Client->ConnectionIdOnServer, DataStreamName));

		if (NetTokenDataStream)
		{
			NetTokenDataStream->AddNetTokenForExplicitExport(Token);
		}

		return Token;
	}

	FNetToken CreateAndExportTokenToServer(const FString& TokenString)
	{
		const FNetToken Token = ClientStringTokenStore->GetOrCreateToken(TokenString);
		UNetTokenDataStream* NetTokenDataStream = Cast<UNetTokenDataStream>(Client->GetReplicationSystem()->GetDataStream(Client->LocalConnectionId, DataStreamName));

		if (NetTokenDataStream)
		{
			NetTokenDataStream->AddNetTokenForExplicitExport(Token);
		}

		return Token;
	}

	UDataStream::EDataStreamState GetDataStreamStateOnServer() const
	{
		return Server->GetConnectionInfo(Client->ConnectionIdOnServer).DataStreamManager->GetStreamState(DataStreamName);
	}

	UDataStream::EDataStreamState GetDataStreamStateOnClient() const
	{
		return Client->GetConnectionInfo(Client->LocalConnectionId).DataStreamManager->GetStreamState(DataStreamName);
	}

	void RoundTrip()
	{
		Server->UpdateAndSend({Client}, true);
		Client->UpdateAndSend(Server, true);
	}
	const FName DataStreamName = "DynamicNetToken";

	FReplicationSystemTestClient* Client = nullptr;

	FStringTokenStore* ServerStringTokenStore = nullptr;
	FStringTokenStore* ClientStringTokenStore = nullptr;

	const FNetTokenStoreState* ClientRemoteNetTokenStoreState = nullptr;
	const FNetTokenStoreState* ServerRemoteNetTokenStoreState = nullptr;

	UDataStreamManager* ServerDataStreamManager = nullptr;
	UDataStreamManager* ClientDataStreamManager = nullptr;

	FDataStreamTestUtil DataStreamUtil;
};

// Basic functionality test
UE_NET_TEST_FIXTURE(FTestDynamicCreateDataStreamFixture, TestDynamicDataStream)
{
	UReplicationSystem* ServerReplicationSystem = Server->ReplicationSystem;
	UReplicationSystem* ClientReplicationSystem = Client->ReplicationSystem;

	// Verify that we cannot find DataStream
	{
		UDataStream* ServerNetTokenStream = ServerReplicationSystem->GetDataStream(Client->ConnectionIdOnServer, DataStreamName);
		UDataStream* ClientNetTokenStream = ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName);

		// It should not exist as it is a dynamic DataStream
		UE_NET_ASSERT_EQ(ServerNetTokenStream, nullptr);
		UE_NET_ASSERT_EQ(ClientNetTokenStream, nullptr);
	
		UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Invalid);
		UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::Invalid);		
	}

	// Open dynamic stream, and verify that it now exists on server
	{
		UDataStream* ServerNetTokenStream = ServerReplicationSystem->OpenDataStream(Client->ConnectionIdOnServer, DataStreamName);
		UDataStream* ClientNetTokenStream = ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName);

		// Should now exist on server
		UE_NET_ASSERT_NE(ServerNetTokenStream, nullptr);
		UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::PendingCreate);

		// But not on client
		UE_NET_ASSERT_EQ(ClientNetTokenStream, nullptr);
	}

	// Roundtrip
	RoundTrip();

	// Now we expect it to be created on client as well.
	{
		UDataStream* ServerNetTokenStream = ServerReplicationSystem->GetDataStream(Client->ConnectionIdOnServer, DataStreamName);
		UDataStream* ClientNetTokenStream = ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName);

		// Server state should now be UDataStream::EDataStreamState::Open
		UE_NET_ASSERT_NE(ServerNetTokenStream, nullptr);
		UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Open);

		// Should now exist on client and be in the WaitOnCreateConfirmation
		UE_NET_ASSERT_NE(ClientNetTokenStream, nullptr);
		UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::WaitOnCreateConfirmation);
	}

	// Send some data to client on the now open stream
	FNetToken ServerHelloToken = CreateAndExportTokenToClient(TEXT("Hello"));
	
	// Roundtrip
	RoundTrip();

	// We should be able to resolve this on client now.
	{
		// Client state should now be open as well
		UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::Open);

		const TCHAR* RcvdTokenString = ClientStringTokenStore->ResolveToken(ServerHelloToken, ClientRemoteNetTokenStoreState);
		UE_NET_ASSERT_NE(RcvdTokenString, nullptr);
	}

	// Send some data from client
	FNetToken ClientHelloToken = CreateAndExportTokenToServer(TEXT("HelloFromClient"));
	
	// Roundtrip
	RoundTrip();

	// We should be able to resolve this on server now.
	{
		const TCHAR* RcvdTokenString = ServerStringTokenStore->ResolveToken(ClientHelloToken, ServerRemoteNetTokenStoreState);
		UE_NET_ASSERT_NE(RcvdTokenString, nullptr);
	}

	// Close from server
	ServerReplicationSystem->CloseDataStream(Client->ConnectionIdOnServer, DataStreamName);

	// Double roundtrip and we should be done
	RoundTrip();
	RoundTrip();

	// Verify that we cannot find DataStream as it should be closed
	{
		UDataStream* ServerNetTokenStream = ServerReplicationSystem->GetDataStream(Client->ConnectionIdOnServer, DataStreamName);
		UDataStream* ClientNetTokenStream = ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName);

		// It should not exist as it is a dynamic DataStream
		UE_NET_ASSERT_EQ(ServerNetTokenStream, nullptr);
		UE_NET_ASSERT_EQ(ClientNetTokenStream, nullptr);
	}
}

// Verify that stream gets created even if we drop create request
UE_NET_TEST_FIXTURE(FTestDynamicCreateDataStreamFixture, TestDynamicDataStream_DropPendingCreateFromServer)
{
	UReplicationSystem* ServerReplicationSystem = Server->ReplicationSystem;
	UReplicationSystem* ClientReplicationSystem = Client->ReplicationSystem;

	// Open dynamic stream, and verify that it now exists on server but not on client
	{
		UDataStream* ServerNetTokenStream = ServerReplicationSystem->OpenDataStream(Client->ConnectionIdOnServer, DataStreamName);
		UDataStream* ClientNetTokenStream = ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName);

		UE_NET_ASSERT_NE(ServerNetTokenStream, nullptr);
		UE_NET_ASSERT_EQ(ClientNetTokenStream, nullptr);
	}

	// Drop PendingCreate
	Server->UpdateAndSend({Client}, false);

	// DataStream should not exist on client
	{
		UDataStream* ClientNetTokenStream = ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName);
		UE_NET_ASSERT_EQ(ClientNetTokenStream, nullptr);
	}

	// Send again
	Server->UpdateAndSend({Client}, true);

	// Should now be created on client
	{
		UDataStream* ClientNetTokenStream = ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName);
		UE_NET_ASSERT_NE(ClientNetTokenStream, nullptr);
	}
}

// Verify that stream gets to open even if we drop create request/confirmation from remote
UE_NET_TEST_FIXTURE(FTestDynamicCreateDataStreamFixture, TestDynamicDataStream_DropPendingCreateFromClient)
{
	UReplicationSystem* ServerReplicationSystem = Server->ReplicationSystem;
	UReplicationSystem* ClientReplicationSystem = Client->ReplicationSystem;

	// Open it, and verify that it now exists on server
	{
		UDataStream* ServerNetTokenStream = ServerReplicationSystem->OpenDataStream(Client->ConnectionIdOnServer, DataStreamName);
		UDataStream* ClientNetTokenStream = ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName);

		// Should now be valid on server
		UE_NET_ASSERT_NE(ServerNetTokenStream, nullptr);
		// But not on client
		UE_NET_ASSERT_EQ(ClientNetTokenStream, nullptr);
	}

	// Deliver PendingCreate from server
	Server->UpdateAndSend({Client}, true);

	// Client should have created it.
	{
		UDataStream* ClientNetTokenStream = ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName);
		UE_NET_ASSERT_NE(ClientNetTokenStream, nullptr);
	}

	// Drop PendingCreate from client
	Client->UpdateAndSend(Server, false);
	
	// Server state should be WaitForCreateConfirmation
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::WaitOnCreateConfirmation);

	// Client state should be PendingCreate
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::PendingCreate);

	// Send and deliver to server
	Client->UpdateAndSend(Server, true);

	// Server state should now be open
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Open);

	// Send and deliver to client
	Server->UpdateAndSend({Client}, true);

	// Client state should now also be open
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::Open);
}

// Verify that stream can be closed from client
UE_NET_TEST_FIXTURE(FTestDynamicCreateDataStreamFixture, TestDynamicDataStream_RequestCloseFromClient)
{
	UReplicationSystem* ServerReplicationSystem = Server->ReplicationSystem;
	UReplicationSystem* ClientReplicationSystem = Client->ReplicationSystem;

	UDataStream* ServerNetTokenStream = ServerReplicationSystem->OpenDataStream(Client->ConnectionIdOnServer, DataStreamName);

	RoundTrip();
	RoundTrip();

	// Verify that stream is open on both sides
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Open);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::Open);

	// Request close from client
	ClientReplicationSystem->CloseDataStream(Client->ConnectionIdOnServer, DataStreamName);

	// Verify that stream is open on server but pending close on client
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Open);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::PendingClose);

	// Send and deliver to server
	Client->UpdateAndSend(Server, true);
	
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::PendingClose);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::WaitOnCloseConfirmation);

	// Double roundtrip and we should be done
	RoundTrip();
	RoundTrip();

	// Verify that stream is invalidated
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Invalid);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::Invalid);	
}

// Verify that stream can be closed from client
UE_NET_TEST_FIXTURE(FTestDynamicCreateDataStreamFixture, TestDynamicDataStream_RequestCloseOnStreamFromClient)
{
	UReplicationSystem* ServerReplicationSystem = Server->ReplicationSystem;
	UReplicationSystem* ClientReplicationSystem = Client->ReplicationSystem;

	UDataStream* ServerNetTokenStream = ServerReplicationSystem->OpenDataStream(Client->ConnectionIdOnServer, DataStreamName);

	RoundTrip();
	RoundTrip();

	// Verify that stream is open on both sides
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Open);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::Open);

	// Request close from client
	UNetTokenDataStream* ClientNetTokenStream = Cast<UNetTokenDataStream>(ClientReplicationSystem->GetDataStream(Client->LocalConnectionId, DataStreamName));
	ClientNetTokenStream->RequestClose();

	// Verify that stream state
	UE_NET_ASSERT_TRUE(ServerNetTokenStream->GetState() == UDataStream::EDataStreamState::Open);
	UE_NET_ASSERT_TRUE(ClientNetTokenStream->GetState() == UDataStream::EDataStreamState::PendingClose);

	// Send and deliver to server
	Client->UpdateAndSend(Server, true);
	
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::PendingClose);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::WaitOnCloseConfirmation);

	// Double roundtrip and we should be done
	RoundTrip();
	RoundTrip();

	// Verify that stream is invalidated
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Invalid);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::Invalid);	
}

// Verify that stream can be closed from client even if we drop request
UE_NET_TEST_FIXTURE(FTestDynamicCreateDataStreamFixture, TestDynamicDataStream_RequestCloseFromClientIsResentIfDropped)
{
	UReplicationSystem* ServerReplicationSystem = Server->ReplicationSystem;
	UReplicationSystem* ClientReplicationSystem = Client->ReplicationSystem;

	UDataStream* ServerNetTokenStream = ServerReplicationSystem->OpenDataStream(Client->ConnectionIdOnServer, DataStreamName);

	RoundTrip();
	RoundTrip();

	// Verify that stream is open on both sides
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Open);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::Open);

	// Request close from client
	ClientReplicationSystem->CloseDataStream(Client->ConnectionIdOnServer, DataStreamName);

	// Verify expected stream state
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Open);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::PendingClose);

	// Drop send to server
	Client->UpdateAndSend(Server, false);
	
	// Verify expected stream state
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Open);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::PendingClose);

	// Send and deliver data
	Client->UpdateAndSend(Server, true);

	// Verify expected stream state
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::PendingClose);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::WaitOnCloseConfirmation);

	// Double roundtrip and we should be done
	RoundTrip();
	RoundTrip();

	// Verify that stream is invalidated
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Invalid);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::Invalid);	
}

// Verify that stream gets properly closed when changing state with create data in flight
UE_NET_TEST_FIXTURE(FTestDynamicCreateDataStreamFixture, TestDynamicDataStream_CloseWhileWaitingForCreateConfirmation)
{
	UReplicationSystem* ServerReplicationSystem = Server->ReplicationSystem;
	UReplicationSystem* ClientReplicationSystem = Client->ReplicationSystem;

	// Open Stream
	UDataStream* ServerNetTokenStream = ServerReplicationSystem->OpenDataStream(Client->ConnectionIdOnServer, DataStreamName);

	// Put data in flight containing create
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Server state should be WaitForCreateConfirmation
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::WaitOnCreateConfirmation);

	// Request close on server
	ServerReplicationSystem->CloseDataStream(Client->ConnectionIdOnServer, DataStreamName);

	// Server state should be PendingClose
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::PendingClose);

	// Put data in flight containing close?
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Deliver packet with create
	Server->DeliverTo(Client, true);

	// Deliver packet with close
	Server->DeliverTo(Client, true);

	// Double roundtrip and we should be done
	RoundTrip();
	RoundTrip();

	// Verify that stream is invalidated
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Invalid);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::Invalid);
}

// Verify that stream gets properly closed when changing state with create data in flight
UE_NET_TEST_FIXTURE(FTestDynamicCreateDataStreamFixture, TestDynamicDataStream_CloseBeforeFirstSend)
{
	UReplicationSystem* ServerReplicationSystem = Server->ReplicationSystem;
	UReplicationSystem* ClientReplicationSystem = Client->ReplicationSystem;

	// Open Stream
	UDataStream* ServerNetTokenStream = ServerReplicationSystem->OpenDataStream(Client->ConnectionIdOnServer, DataStreamName);
	// Close Stream
	ServerReplicationSystem->CloseDataStream(Client->ConnectionIdOnServer, DataStreamName);

	// Run a few updates and make sure stream is propertly closed.
	RoundTrip();
	RoundTrip();

	// Verify that stream is invalidated
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Invalid);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::Invalid);
}

UE_NET_TEST_FIXTURE(FTestDynamicCreateDataStreamFixture, TestDynamicDataStreamCloseRespectsHasAcknowledgedAllReliableData)
{
	UReplicationSystem* ServerReplicationSystem = Server->ReplicationSystem;
	UReplicationSystem* ClientReplicationSystem = Client->ReplicationSystem;

	// Open dynamic stream, and verify that it now exists on server
	UDataStream* ServerNetTokenStream = ServerReplicationSystem->OpenDataStream(Client->ConnectionIdOnServer, DataStreamName);

	// Roundtrip
	RoundTrip();

	// Put some data in flight in separate packets
	FNetToken ServerHelloToken = CreateAndExportTokenToClient(TEXT("Hello"));
	Server->NetUpdate();
	Server->SendTo(Client, TEXT("Hello"));
	Server->PostSendUpdate();

	FNetToken ServerHello2Token = CreateAndExportTokenToClient(TEXT("Hello2"));
	Server->NetUpdate();
	Server->SendTo(Client, TEXT("Hello2"));
	Server->PostSendUpdate();

	FNetToken ServerHello3Token = CreateAndExportTokenToClient(TEXT("Hello3"));
	Server->NetUpdate();
	Server->SendTo(Client, TEXT("Hello3"));
	Server->PostSendUpdate();
	
	// Close from server
	ServerReplicationSystem->CloseDataStream(Client->ConnectionIdOnServer, DataStreamName);
	Server->NetUpdate();
	Server->SendTo(Client, TEXT("Close"));
	Server->PostSendUpdate();

	// We expect the stream to be PendingClose as we still have important data in flight
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::PendingClose);

	// Drop a few packets
	Server->DeliverTo(Client, false);
	Server->DeliverTo(Client, false);
	Server->DeliverTo(Client, false);

	// Deliver PendingClose
	Server->DeliverTo(Client, true);

	// Acknowledge pending close
	Client->UpdateAndSend(Server, true);

	// We expect the stream to be PendingClose as we still have important data in flight
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::PendingClose);

	// Deliver PendingClose
	Server->UpdateAndSend({Client}, true);

	// Acknowledge pending close
	Client->UpdateAndSend(Server, true);

	// Deliver PendingClose
	RoundTrip();

	// Validate that the tokens are resolvable on the client
	UE_NET_ASSERT_NE(ClientStringTokenStore->ResolveToken(ServerHelloToken, ClientRemoteNetTokenStoreState), nullptr);
	UE_NET_ASSERT_NE(ClientStringTokenStore->ResolveToken(ServerHello2Token, ClientRemoteNetTokenStoreState), nullptr);
	UE_NET_ASSERT_NE(ClientStringTokenStore->ResolveToken(ServerHello3Token, ClientRemoteNetTokenStoreState), nullptr);

	// And streams are destroyed on both server and client	
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnServer() == UDataStream::EDataStreamState::Invalid);
	UE_NET_ASSERT_TRUE(GetDataStreamStateOnClient() == UDataStream::EDataStreamState::Invalid);
}








}
