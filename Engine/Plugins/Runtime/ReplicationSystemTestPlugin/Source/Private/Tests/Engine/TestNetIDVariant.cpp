// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Net/NetIDVariant.h"
#include "UObject/CoreNet.h"

namespace UE::Net
{

FTestMessage& operator<<(FTestMessage& Message, const FNetIDVariant& NetID)
{
	return Message << *NetID.ToString();
}

FTestMessage& operator<<(FTestMessage& Message, const FNetworkGUID& NetGUID)
{
	return Message << *NetGUID.ToString();
}

}

namespace UE::Net::Private
{
	UE_NET_TEST(NetIDVariant, Empty)
	{
		FNetIDVariant DefaultVariant;

		UE_NET_ASSERT_FALSE(DefaultVariant.GetVariant().IsType<FNetworkGUID>());
		UE_NET_ASSERT_FALSE(DefaultVariant.GetVariant().IsType<FNetRefHandle>());

		// Serialization
		FNetBitWriter TempWriter(1024);
		TempWriter << DefaultVariant;

		FNetBitReader TempReader(nullptr, TempWriter.GetData(), TempWriter.GetNumBits());
		FNetIDVariant DeserializedDefaultVariant;
		TempReader << DeserializedDefaultVariant;

		UE_NET_ASSERT_TRUE(DeserializedDefaultVariant.GetVariant().IsType<FNetIDVariant::FEmptyID>());
	}

	UE_NET_TEST(NetIDVariant, Hashing)
	{
		// Testing hashing, equality, and TSet/TMap support
		FNetIDVariant DefaultVariant;
		FNetIDVariant InvalidGUID{FNetworkGUID()};
		FNetIDVariant InvalidRefHandle{FNetRefHandle()};

		const uint32 DefaultHash = GetTypeHash(DefaultVariant);
		const uint32 InvalidGUIDHash = GetTypeHash(InvalidGUID);
		const uint32 InvalidRefHandleHash = GetTypeHash(InvalidRefHandle);

		// Try to avoid some simple hash collisions
		UE_NET_ASSERT_NE(DefaultHash, InvalidGUIDHash);
		UE_NET_ASSERT_NE(DefaultHash, InvalidRefHandleHash);
		UE_NET_ASSERT_NE(InvalidGUIDHash, InvalidRefHandleHash);

		// Set uniqueness
		TSet<FNetIDVariant> SetTest;
		SetTest.Add(DefaultVariant);
		SetTest.Add(InvalidGUID);
		SetTest.Add(InvalidRefHandle);
		UE_NET_ASSERT_EQ(SetTest.Num(), 3);

		// Set duplicates
		SetTest.Add(DefaultVariant);
		SetTest.Add(InvalidGUID);
		SetTest.Add(InvalidRefHandle);
		UE_NET_ASSERT_EQ(SetTest.Num(), 3);

		FNetIDVariant ValidGUID1{FNetworkGUID::CreateFromIndex(1, false)};
		FNetIDVariant ValidGUID2{FNetworkGUID::CreateFromIndex(2, false)};

		const uint32 ValidGUID1Hash = GetTypeHash(ValidGUID1);
		const uint32 ValidGUID2Hash = GetTypeHash(ValidGUID2);

		UE_NET_ASSERT_NE(ValidGUID1Hash, ValidGUID2Hash);

		// Need to get valid FNetRefHandles
		UE::Net::FReplicationSystemTestNode TestNode(true, TEXT("NetIDVariantSetTest"));
		UTestReplicatedIrisObject* TestObject1 = TestNode.CreateObject(0, 0);
		UTestReplicatedIrisObject* TestObject2 = TestNode.CreateObject(0, 0);

		FNetIDVariant ValidRefHandle1{TestNode.GetReplicationBridge()->BeginReplication(TestObject1)};
		FNetIDVariant ValidRefHandle2{TestNode.GetReplicationBridge()->BeginReplication(TestObject2)};

		const uint32 ValidRefHandle1Hash = GetTypeHash(ValidRefHandle1);
		const uint32 ValidRefHandle2Hash = GetTypeHash(ValidRefHandle2);

		UE_NET_ASSERT_NE(ValidRefHandle1Hash, ValidRefHandle2Hash);

		// Set uniqueness
		SetTest.Add(ValidGUID1);
		SetTest.Add(ValidGUID2);
		SetTest.Add(ValidRefHandle1);
		SetTest.Add(ValidRefHandle2);
		UE_NET_ASSERT_EQ(SetTest.Num(), 7);

		// Set duplicates
		SetTest.Add(ValidGUID1);
		SetTest.Add(ValidGUID2);
		SetTest.Add(ValidRefHandle1);
		SetTest.Add(ValidRefHandle2);
		UE_NET_ASSERT_EQ(SetTest.Num(), 7);
	}

	UE_NET_TEST(NetIDVariant, NetworkGUID)
	{
		const FNetworkGUID ValidGUID = FNetworkGUID::CreateFromIndex(100, false);
		FNetIDVariant ValidVariantGUID(ValidGUID);

		const FNetworkGUID InvalidGUID;
		FNetIDVariant InvalidVariantGUID(InvalidGUID);

		// Basic round-trip
		FNetworkGUID ValidOutputGUID = ValidVariantGUID.GetVariant().Get<FNetworkGUID>();
		UE_NET_ASSERT_EQ(ValidGUID, ValidOutputGUID);

		FNetworkGUID InvalidOutputGUID = InvalidVariantGUID.GetVariant().Get<FNetworkGUID>();
		UE_NET_ASSERT_EQ(InvalidGUID, InvalidOutputGUID);

		// Validity
		UE_NET_ASSERT_EQ(ValidGUID.IsValid(), ValidVariantGUID.IsValid());
		UE_NET_ASSERT_EQ(InvalidGUID.IsValid(), InvalidVariantGUID.IsValid());

		// Serialization
		FNetBitWriter TempWriter(1024);
		TempWriter << ValidVariantGUID;
		TempWriter << InvalidVariantGUID;

		FNetBitReader TempReader(nullptr, TempWriter.GetData(), TempWriter.GetNumBits());
		FNetIDVariant DeserializedValidVariant;
		TempReader << DeserializedValidVariant;

		FNetIDVariant DeserializedInvalidVariant;
		TempReader << DeserializedInvalidVariant;

		UE_NET_ASSERT_TRUE(DeserializedValidVariant.GetVariant().IsType<FNetworkGUID>());
		UE_NET_ASSERT_EQ(DeserializedValidVariant.GetVariant().Get<FNetworkGUID>(), ValidGUID);

		UE_NET_ASSERT_TRUE(DeserializedInvalidVariant.GetVariant().IsType<FNetworkGUID>());
		UE_NET_ASSERT_EQ(DeserializedInvalidVariant.GetVariant().Get<FNetworkGUID>(), InvalidGUID);
	}

	UE_NET_TEST(NetIDVariant, NetRefHandle)
	{
		// Just need to get a valid FNetRefHandle
		UE::Net::FReplicationSystemTestNode TestNode(true, TEXT("NetIDVariantTest"));
		UTestReplicatedIrisObject* TestObject = TestNode.CreateObject(0, 0);

		FNetRefHandle ValidRefHandle = TestNode.GetReplicationBridge()->BeginReplication(TestObject);
		FNetIDVariant ValidVariantRefHandle(ValidRefHandle);

		FNetRefHandle InvalidRefHandle;
		FNetIDVariant InvalidVariantRefHandle(InvalidRefHandle);

		// Basic round-trip
		FNetRefHandle ValidOutputRefHandle = ValidVariantRefHandle.GetVariant().Get<FNetRefHandle>();
		UE_NET_ASSERT_EQ(ValidRefHandle, ValidOutputRefHandle);

		FNetRefHandle InvalidOutputRefHandle = InvalidVariantRefHandle.GetVariant().Get<FNetRefHandle>();
		UE_NET_ASSERT_EQ(InvalidRefHandle, InvalidOutputRefHandle);

		// Validity
		UE_NET_ASSERT_EQ(ValidRefHandle.IsValid(), ValidVariantRefHandle.IsValid());
		UE_NET_ASSERT_EQ(InvalidRefHandle.IsValid(), InvalidVariantRefHandle.IsValid());

		// Serialization
		FNetBitWriter TempWriter(1024);
		TempWriter << ValidVariantRefHandle;
		TempWriter << InvalidVariantRefHandle;

		FNetBitReader TempReader(nullptr, TempWriter.GetData(), TempWriter.GetNumBits());
		FNetIDVariant DeserializedValidVariant;
		TempReader << DeserializedValidVariant;

		FNetIDVariant DeserializedInvalidVariant;
		TempReader << DeserializedInvalidVariant;

		UE_NET_ASSERT_TRUE(DeserializedValidVariant.GetVariant().IsType<FNetRefHandle>());
		UE_NET_ASSERT_EQ(DeserializedValidVariant.GetVariant().Get<FNetRefHandle>(), ValidRefHandle);

		UE_NET_ASSERT_TRUE(DeserializedInvalidVariant.GetVariant().IsType<FNetRefHandle>());
		UE_NET_ASSERT_EQ(DeserializedInvalidVariant.GetVariant().Get<FNetRefHandle>(), InvalidRefHandle);
	}
}
