// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Engine/NetSerialization.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Serialization/BitReader.h"
#include "Serialization/BitWriter.h"
#include <limits>

namespace UE::Net::Private
{
	UE_NET_TEST(NetSerialization, PackedVectorFloat10)
	{
		{
			const float Quantize10_Values[] =
			{
				0.0f,
				-180817.42f,
				47.11f,
				-FMath::Exp2(25.0f),
				std::numeric_limits<float>::infinity(), // non-finite
			};

			const int32 ValueCount = UE_ARRAY_COUNT(Quantize10_Values);

			// We are intentionally testing at least one value that would cause an error to be logged during serialization. We need to suppress it in order for other software that thinks that is a test fail.
			LOG_SCOPE_VERBOSITY_OVERRIDE(LogCore, ELogVerbosity::Fatal);

			constexpr bool bAllowResize = false;
			FBitWriter Writer(128, bAllowResize);

			for (size_t ValueIt = 0, ValueEndIt = ValueCount; ValueIt != ValueEndIt; ++ValueIt)
			{
				Writer.Reset();

				const float ScalarValue = Quantize10_Values[ValueIt];
				const FVector3f WriteValue(ScalarValue);
				FVector3f ReadValue;

				const bool bOverflowOrNan = !WritePackedVector<10, 24>(WriteValue, Writer);

				UE_NET_ASSERT_FALSE(Writer.GetError());

				FBitReader Reader(Writer.GetData(), Writer.GetNumBits());
				ReadPackedVector<10, 24>(ReadValue, Reader);

				UE_NET_ASSERT_FALSE(Reader.GetError());

				UE_NET_ASSERT_EQ(ReadValue.X, ReadValue.Y);
				UE_NET_ASSERT_EQ(ReadValue.X, ReadValue.Z);

				// At this point we should have similar values as the original ones, except for NaN and overflowed values
				if (bOverflowOrNan)
				{
					UE_NET_ASSERT_TRUE(WriteValue.ContainsNaN());
					UE_NET_ASSERT_EQ(ReadValue, FVector3f::ZeroVector);
				}
				else
				{
					const float ValueDiff = FMath::Abs(ReadValue.X - WriteValue.X);
					UE_NET_ASSERT_LT(ValueDiff, 2.0f / 10); // The diff test might need some adjustment
				}
			}
		}
	}
	
	UE_NET_TEST(NetSerialization, PackedVectorFloat100)
	{
		{
			const float Quantize100_Values[] =
			{
				0.0f,
				+180720.42f,
				-19751216.0f,
				FMath::Exp2(31.0f),
				-std::numeric_limits<float>::infinity(),
			};

			const int32 ValueCount = UE_ARRAY_COUNT(Quantize100_Values);

			// We are intentionally testing at least one value that would cause an error to be logged during serialization. We need to suppress it in order for other software that thinks that is a test fail.
			LOG_SCOPE_VERBOSITY_OVERRIDE(LogCore, ELogVerbosity::Fatal);

			constexpr bool bAllowResize = false;
			FBitWriter Writer(128, bAllowResize);

			for (size_t ValueIt = 0, ValueEndIt = ValueCount; ValueIt != ValueEndIt; ++ValueIt)
			{
				Writer.Reset();

				const float ScalarValue = Quantize100_Values[ValueIt];
				const FVector3f WriteValue(ScalarValue);
				FVector3f ReadValue;

				const bool bOverflowOrNan = !WritePackedVector<100, 30>(WriteValue, Writer);

				UE_NET_ASSERT_FALSE(Writer.GetError());

				FBitReader Reader(Writer.GetData(), Writer.GetNumBits());
				ReadPackedVector<100, 30>(ReadValue, Reader);

				UE_NET_ASSERT_FALSE(Reader.GetError());

				UE_NET_ASSERT_EQ(ReadValue.X, ReadValue.Y);
				UE_NET_ASSERT_EQ(ReadValue.X, ReadValue.Z);

				// At this point we should have similar values as the original ones, except for NaN and overflowed values
				if (bOverflowOrNan)
				{
					UE_NET_ASSERT_TRUE(WriteValue.ContainsNaN());
					UE_NET_ASSERT_EQ(ReadValue, FVector3f::ZeroVector);
				}
				else
				{
					const float ValueDiff = FMath::Abs(ReadValue.X - WriteValue.X);
					UE_NET_ASSERT_LT(ValueDiff, 2.0f / 100); // The diff test might need some adjustment
				}
			}
		}
	}

	UE_NET_TEST(NetSerialization, PackedVectorDouble10)
	{
		{
			const double Quantize10_Values[] =
			{
				0.0f,
				-180817.42,
				47.11f,
				-FMath::Exp2(25.0),
				std::numeric_limits<double>::infinity(), // non-finite
			};

			const int32 ValueCount = UE_ARRAY_COUNT(Quantize10_Values);

			// We are intentionally testing at least one value that would cause an error to be logged during serialization. We need to suppress it in order for other software that thinks that is a test fail.
			LOG_SCOPE_VERBOSITY_OVERRIDE(LogCore, ELogVerbosity::Fatal);

			constexpr bool bAllowResize = false;
			FBitWriter Writer(512, bAllowResize);

			for (size_t ValueIt = 0, ValueEndIt = ValueCount; ValueIt != ValueEndIt; ++ValueIt)
			{
				Writer.Reset();

				const double ScalarValue = Quantize10_Values[ValueIt];
				const FVector3d WriteValue(ScalarValue);
				FVector3d ReadValue;

				const bool bOverflowOrNan = !WritePackedVector<10, 24>(WriteValue, Writer);

				UE_NET_ASSERT_FALSE(Writer.GetError());

				FBitReader Reader(Writer.GetData(), Writer.GetNumBits());
				ReadPackedVector<10, 24>(ReadValue, Reader);

				UE_NET_ASSERT_FALSE(Reader.GetError());

				UE_NET_ASSERT_EQ(ReadValue.X, ReadValue.Y);
				UE_NET_ASSERT_EQ(ReadValue.X, ReadValue.Z);

				// At this point we should have similar values as the original ones, except for NaN and overflowed values
				if (bOverflowOrNan)
				{
					UE_NET_ASSERT_TRUE(WriteValue.ContainsNaN());
					UE_NET_ASSERT_EQ(ReadValue, FVector3d::ZeroVector);
				}
				else
				{
					const double ValueDiff = FMath::Abs(ReadValue.X - WriteValue.X);
					UE_NET_ASSERT_LT(ValueDiff, 2.0 / 10); // The diff test might need some adjustment
				}
			}
		}
	}

	UE_NET_TEST(NetSerialization, PackedVectorDouble100)
	{
		{
			const double Quantize100_Values[] =
			{
				0.0f,
				+180720.42,
				-19751216.0,
				FMath::Exp2(40.0),
				FMath::Exp2(59.0),
				-std::numeric_limits<double>::infinity(),
			};

			const int32 ValueCount = UE_ARRAY_COUNT(Quantize100_Values);

			// We are intentionally testing at least one value that would cause an error to be logged during serialization. We need to suppress it in order for other software that thinks that is a test fail.
			LOG_SCOPE_VERBOSITY_OVERRIDE(LogCore, ELogVerbosity::Fatal);

			constexpr bool bAllowResize = false;
			FBitWriter Writer(512, bAllowResize);

			for (size_t ValueIt = 0, ValueEndIt = ValueCount; ValueIt != ValueEndIt; ++ValueIt)
			{
				Writer.Reset();

				const double ScalarValue = Quantize100_Values[ValueIt];
				const FVector3d WriteValue(ScalarValue);
				FVector3d ReadValue;

				const bool bOverflowOrNan = !WritePackedVector<100, 30>(WriteValue, Writer);

				UE_NET_ASSERT_FALSE(Writer.GetError());

				FBitReader Reader(Writer.GetData(), Writer.GetNumBits());
				ReadPackedVector<100, 30>(ReadValue, Reader);

				UE_NET_ASSERT_FALSE(Reader.GetError());

				UE_NET_ASSERT_EQ(ReadValue.X, ReadValue.Y);
				UE_NET_ASSERT_EQ(ReadValue.X, ReadValue.Z);

				// At this point we should have similar values as the original ones, except for NaN and overflowed values
				if (bOverflowOrNan)
				{
					UE_NET_ASSERT_TRUE(WriteValue.ContainsNaN());
					UE_NET_ASSERT_EQ(ReadValue, FVector3d::ZeroVector);
				}
				else
				{
					const double ValueDiff = FMath::Abs(ReadValue.X - WriteValue.X);
					UE_NET_ASSERT_LT(ValueDiff, 2.0 / 100); // The diff test might need some adjustment
				}
			}
		}
	}

	UE_NET_TEST(NetSerialization, PackedVectorWriteDoubleReadFloat)
	{
		{
			const double Quantize100_Values[] =
			{
				0.0f,
				+180720.42,
				-19751216.0,
				FMath::Exp2(40.0),
				FMath::Exp2(59.0),
				-std::numeric_limits<double>::infinity(),
			};

			const int32 ValueCount = UE_ARRAY_COUNT(Quantize100_Values);

			// We are intentionally testing at least one value that would cause an error to be logged during serialization. We need to suppress it in order for other software that thinks that is a test fail.
			LOG_SCOPE_VERBOSITY_OVERRIDE(LogCore, ELogVerbosity::Fatal);

			constexpr bool bAllowResize = false;
			FBitWriter Writer(512, bAllowResize);

			for (size_t ValueIt = 0, ValueEndIt = ValueCount; ValueIt != ValueEndIt; ++ValueIt)
			{
				Writer.Reset();

				const double ScalarValue = Quantize100_Values[ValueIt];
				const FVector3d WriteValue(ScalarValue);
				FVector3f ReadValue;

				const bool bOverflowOrNan = !WritePackedVector<100, 30>(WriteValue, Writer);

				UE_NET_ASSERT_FALSE(Writer.GetError());

				FBitReader Reader(Writer.GetData(), Writer.GetNumBits());
				ReadPackedVector<100, 30>(ReadValue, Reader);

				UE_NET_ASSERT_FALSE(Reader.GetError());

				UE_NET_ASSERT_EQ(ReadValue.X, ReadValue.Y);
				UE_NET_ASSERT_EQ(ReadValue.X, ReadValue.Z);

				// At this point we should have similar values as the original ones, except for NaN and overflowed values
				if (bOverflowOrNan)
				{
					UE_NET_ASSERT_TRUE(WriteValue.ContainsNaN());
					UE_NET_ASSERT_EQ(ReadValue, FVector3f::ZeroVector);
				}
				else
				{
					const double ValueDiff = FMath::Abs(ReadValue.X - WriteValue.X);
					UE_NET_ASSERT_LT(ValueDiff, 2.0 / 100); // The diff test might need some adjustment
				}
			}
		}
	}
}
