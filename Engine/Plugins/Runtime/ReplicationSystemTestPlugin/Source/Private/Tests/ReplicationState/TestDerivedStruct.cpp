// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDerivedStruct.h"
#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "Net/UnrealNetwork.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"

namespace UE::Net::Private
{

struct FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer
{
	struct FQuantizedData
	{
		uint8 ByteMember0;
		uint8 ByteMember1;
	};

	typedef FTestDerivedStruct_Inherited_WithNetSerializer SourceType;
	typedef FQuantizedData QuantizedType;

	typedef FNetSerializerConfig ConfigType;

	static const uint32 Version = 0;
	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);


	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;

		static inline const FName PropertyNetSerializerRegistry_NAME_TestDerivedStruct_Inherited_WithNetSerializer = FName("TestDerivedStruct_Inherited_WithNetSerializer");
		UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_TestDerivedStruct_Inherited_WithNetSerializer, FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer);
	};
	
	static inline FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
};
UE_NET_IMPLEMENT_SERIALIZER(FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer);

struct FTestDerivedStruct_Inherited_WithNetSerializerWithApply_NetSerializer
{
	struct FQuantizedData
	{
		uint8 ByteMember0;
		uint8 ByteMember1;
		uint8 ByteMemberNotSetOnApply;
	};

	typedef FTestDerivedStruct_Inherited_WithNetSerializerWithApply SourceType;
	typedef FQuantizedData QuantizedType;

	typedef FNetSerializerConfig ConfigType;

	static const uint32 Version = 0;
	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);

	static void Apply(FNetSerializationContext&, const FNetApplyArgs&);

	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;

		static inline const FName PropertyNetSerializerRegistry_NAME_TestDerivedStruct_Inherited_WithNetSerializerWithApply = FName("TestDerivedStruct_Inherited_WithNetSerializerWithApply");
		UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_TestDerivedStruct_Inherited_WithNetSerializerWithApply, FTestDerivedStruct_Inherited_WithNetSerializerWithApply_NetSerializer);
	};
	
	static inline FTestDerivedStruct_Inherited_WithNetSerializerWithApply_NetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
};
UE_NET_IMPLEMENT_SERIALIZER(FTestDerivedStruct_Inherited_WithNetSerializerWithApply_NetSerializer);

class FTestDerivedStructFixture : public FNetworkAutomationTestSuiteFixture
{
};

class FTestDerivedStructInObjectFixture : public FReplicationSystemServerClientTestFixture
{
protected:
	UTestDerivedStruct_TestObject_Member* CreateObjectWithMember();
	UTestDerivedStruct_TestObject_Array* CreateObjectWithArray();
};

// We want the struct deriving from the struct with custom NetSerializer to have the same size. This allows us to detect if the struct with the NetSerializer overwrites any of the properties in the derived struct.
UE_NET_TEST_FIXTURE(FTestDerivedStructFixture, VerifyTestAssumptions)
{
	UE_NET_EXPECT_EQ_MSG(sizeof(FTestDerivedStruct_Inherited_WithNetSerializer), sizeof(FTestDerivedStruct_Inherited_WithNetSerializer_Inherited_WithoutNetSerializer), "Expected FTestDerivedStruct_Inherited_WithNetSerializer to have same size as FTestDerivedStruct_Inherited_WithNetSerializer_Inherited_WithoutNetSerializer.");
}

UE_NET_TEST_FIXTURE(FTestDerivedStructFixture, StructWithNetSerializerGetsNetSerializer)
{
	TRefCountPtr<const FReplicationStateDescriptor> StructDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(FTestDerivedStruct_Inherited_WithNetSerializer::StaticStruct());
	UE_NET_ASSERT_EQ(StructDescriptor->MemberCount, uint16(1));
}

UE_NET_TEST_FIXTURE(FTestDerivedStructFixture, DerivedStructWithoutNetSerializerMakesUseOfParentNetSerializer)
{
	TRefCountPtr<const FReplicationStateDescriptor> StructDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(FTestDerivedStruct_Inherited_WithNetSerializer_Inherited_WithoutNetSerializer::StaticStruct());
	
	// The derived struct should have two members, the first using the NetSerializer of the parent struct.
	UE_NET_ASSERT_EQ(StructDescriptor->MemberCount, uint16(2));
	UE_NET_ASSERT_EQ(StructDescriptor->MemberSerializerDescriptors[0].Serializer, &UE_NET_GET_SERIALIZER(FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer));
	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(StructDescriptor->Traits, EReplicationStateTraits::IsDerivedFromStructWithCustomSerializer));
}

UE_NET_TEST_FIXTURE(FTestDerivedStructFixture, DerivedStructHasValidDebugDescriptor)
{
	TRefCountPtr<const FReplicationStateDescriptor> StructDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(FTestDerivedStruct_Inherited_WithNetSerializer_Inherited_WithoutNetSerializer::StaticStruct());

	const FReplicationStateMemberDebugDescriptor* DebugDescriptors = StructDescriptor->MemberDebugDescriptors;
	UE_NET_ASSERT_NE(DebugDescriptors, nullptr);

	for (const FReplicationStateMemberDebugDescriptor& MemberDebugDescriptor : MakeArrayView(DebugDescriptors, StructDescriptor->MemberCount))
	{
		UE_NET_ASSERT_NE_MSG(MemberDebugDescriptor.DebugName, nullptr, FString().Appendf(TEXT("Member %u lacking DebugName"), (&MemberDebugDescriptor - DebugDescriptors)));
		UE_NET_ASSERT_NE_MSG(MemberDebugDescriptor.DebugName->Name, nullptr, FString().Appendf(TEXT("Member %u lacking DebugName.Name"), (&MemberDebugDescriptor - DebugDescriptors)));
	}
}

UE_NET_TEST_FIXTURE(FTestDerivedStructFixture, DeepInheritanceOfStructWithNetSerializerMakesUseOfParentNetSerializer)
{
	TRefCountPtr<const FReplicationStateDescriptor> StructDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(FTestDerivedStruct_DeepInheritanceOfStructWithNetSerializer::StaticStruct());

	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(StructDescriptor->Traits, EReplicationStateTraits::IsDerivedFromStructWithCustomSerializer));

	// The derived struct should have three members, the first using the NetSerializer of the parent struct.
	UE_NET_ASSERT_EQ(StructDescriptor->MemberCount, uint16(3));
	UE_NET_ASSERT_EQ(StructDescriptor->MemberSerializerDescriptors[0].Serializer, &UE_NET_GET_SERIALIZER(FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer));

	const FProperty* LastProperty = StructDescriptor->MemberProperties[StructDescriptor->MemberCount - 1];
	UE_NET_ASSERT_NE(LastProperty, nullptr);
	UE_NET_ASSERT_EQ(LastProperty->GetName(), FString(TEXT("YetAnotherMember")));
}

UE_NET_TEST_FIXTURE(FTestDerivedStructFixture, DeepInheritanceAndStructWithNetSerializerUsesSerializer)
{
	TRefCountPtr<const FReplicationStateDescriptor> StructDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(FTestDerivedStruct_DeepInheritanceOfStructWithNetSerializer::StaticStruct());

	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(StructDescriptor->Traits, EReplicationStateTraits::IsDerivedFromStructWithCustomSerializer));

	// The derived struct should have three members, the first using the NetSerializer of the parent struct.
	UE_NET_ASSERT_EQ(StructDescriptor->MemberCount, uint16(3));
	UE_NET_ASSERT_EQ(StructDescriptor->MemberSerializerDescriptors[0].Serializer, &UE_NET_GET_SERIALIZER(FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer));

	const FProperty* LastProperty = StructDescriptor->MemberProperties[StructDescriptor->MemberCount - 1];
	UE_NET_ASSERT_NE(LastProperty, nullptr);
	UE_NET_ASSERT_EQ(LastProperty->GetName(), FString(TEXT("YetAnotherMember")));
}

UE_NET_TEST_FIXTURE(FTestDerivedStructFixture, StructUsesClosestNetSerializerInHierachy)
{
	TRefCountPtr<const FReplicationStateDescriptor> StructDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(FTestDerivedStruct_Inherited_Inherited_WithNetSerializer_DeepInheritanceOfStructWithNetSerializer::StaticStruct());

	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(StructDescriptor->Traits, EReplicationStateTraits::IsDerivedFromStructWithCustomSerializer));

	// The derived struct should have two members, the first using the NetSerializer of the parent struct.
	UE_NET_ASSERT_EQ(StructDescriptor->MemberCount, uint16(2));
	// We've registered the struct to use an already existing serializer so need to look for the serializer we're forwarding to.
	UE_NET_ASSERT_EQ(StructDescriptor->MemberSerializerDescriptors[0].Serializer, &UE_NET_GET_SERIALIZER(FNopNetSerializer));

	const FProperty* LastProperty = StructDescriptor->MemberProperties[StructDescriptor->MemberCount - 1];
	UE_NET_ASSERT_NE(LastProperty, nullptr);
	UE_NET_ASSERT_EQ(LastProperty->GetName(), FString(TEXT("ByteMember5")));
}

UE_NET_TEST_FIXTURE(FTestDerivedStructInObjectFixture, CanReplicateDerivedStructMember)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestDerivedStruct_TestObject_Member* ServerObject = CreateObjectWithMember();
	ServerObject->DerivedStruct.ByteMember2 ^= 47;

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	const UTestDerivedStruct_TestObject_Member* ClientObject = Cast<UTestDerivedStruct_TestObject_Member>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the member has been replicated
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMember2, ServerObject->DerivedStruct.ByteMember2);
}

UE_NET_TEST_FIXTURE(FTestDerivedStructInObjectFixture, CanReplicateDerivedStructArray)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestDerivedStruct_TestObject_Array* ServerObject = CreateObjectWithArray();

	// Add items to the array
	{
		FTestDerivedStruct_Inherited_WithNetSerializer_Inherited_WithoutNetSerializer Item;
		Item.ByteMember2 ^= 11;
		ServerObject->DerivedStructArray.Add(Item);
		Item.ByteMember2 ^= 47;
		ServerObject->DerivedStructArray.Add(Item);
	}

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	const UTestDerivedStruct_TestObject_Array* ClientObject = Cast<UTestDerivedStruct_TestObject_Array>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the members have been replicated
	{
		UE_NET_ASSERT_EQ(ClientObject->DerivedStructArray.Num(), ServerObject->DerivedStructArray.Num());
		int32 MemberIt = 0;
		for (const FTestDerivedStruct_Inherited_WithNetSerializer_Inherited_WithoutNetSerializer& ServerItem : ServerObject->DerivedStructArray)
		{
			const FTestDerivedStruct_Inherited_WithNetSerializer_Inherited_WithoutNetSerializer& ClientItem = ClientObject->DerivedStructArray[MemberIt];
			++MemberIt;
			UE_NET_ASSERT_EQ(ClientItem.ByteMember2, ServerItem.ByteMember2);
		}
	}
}

UE_NET_TEST_FIXTURE(FTestDerivedStructInObjectFixture, NotReplicatedMemberInDerivedStructIsNotOverwritten)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestDerivedStruct_TestObject_Member* ServerObject = CreateObjectWithMember();

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestDerivedStruct_TestObject_Member* ClientObject = Cast<UTestDerivedStruct_TestObject_Member>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// On the client modify the non-replicated property
	constexpr uint8 ClientModifiedNotReplicatedPropertyValue = 255U;
	ClientObject->DerivedStruct.ByteMember3_NotReplicated = ClientModifiedNotReplicatedPropertyValue;

	// On the server modify all members
	ServerObject->DerivedStruct.ByteMember0 = 0;
	ServerObject->DerivedStruct.ByteMember1 = 1;
	ServerObject->DerivedStruct.ByteMember2 = 2;
	ServerObject->DerivedStruct.ByteMember3_NotReplicated = ClientModifiedNotReplicatedPropertyValue ^ 111;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify state
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMember0, ServerObject->DerivedStruct.ByteMember0);
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMember1, ServerObject->DerivedStruct.ByteMember1);
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMember2, ServerObject->DerivedStruct.ByteMember2);
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMember3_NotReplicated, ClientModifiedNotReplicatedPropertyValue);
}

UE_NET_TEST_FIXTURE(FTestDerivedStructInObjectFixture, CanReplicateDerivedStructMembersInArbitraryOrder)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestDerivedStruct_TestObject_Member* ServerObject = CreateObjectWithMember();

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	const UTestDerivedStruct_TestObject_Member* ClientObject = Cast<UTestDerivedStruct_TestObject_Member>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Modify the property not replicated by the custom serializer.
	ServerObject->DerivedStruct.ByteMember2 ^= 47;
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify state
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMember0, ServerObject->DerivedStruct.ByteMember0);
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMember1, ServerObject->DerivedStruct.ByteMember1);
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMember2, ServerObject->DerivedStruct.ByteMember2);

	// Modify properties replicated by custom serializer
	ServerObject->DerivedStruct.ByteMember0 ^= 13;
	ServerObject->DerivedStruct.ByteMember1 ^= 37;
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify state
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMember0, ServerObject->DerivedStruct.ByteMember0);
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMember1, ServerObject->DerivedStruct.ByteMember1);
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMember2, ServerObject->DerivedStruct.ByteMember2);
}

UE_NET_TEST_FIXTURE(FTestDerivedStructInObjectFixture, NotAppliedMemberInDerivedStructIsNotOverwritten)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestDerivedStructWithNetSerializerWithApply_TestObject_Member* ServerObject = Server->CreateObject<UTestDerivedStructWithNetSerializerWithApply_TestObject_Member>();

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestDerivedStructWithNetSerializerWithApply_TestObject_Member* ClientObject = Cast<UTestDerivedStructWithNetSerializerWithApply_TestObject_Member>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// On the client modify the non-applied property
	const uint8 ClientModifiedNotAppliedPropertyValue = ClientObject->DerivedStruct.ByteMemberNotSetOnApply ^ 123U;
	ClientObject->DerivedStruct.ByteMemberNotSetOnApply = ClientModifiedNotAppliedPropertyValue;

	// On the server modify all members
	ServerObject->DerivedStruct.ByteMember0 ^= 1U;
	ServerObject->DerivedStruct.ByteMember1 ^= 2U;
	ServerObject->DerivedStruct.ByteMemberNotSetOnApply = ClientModifiedNotAppliedPropertyValue ^ 231U;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify state, most importantly that the not applied member remains as is.
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMember0, ServerObject->DerivedStruct.ByteMember0);
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMember1, ServerObject->DerivedStruct.ByteMember1);
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMemberNotSetOnApply, ClientModifiedNotAppliedPropertyValue);
}

UE_NET_TEST_FIXTURE(FTestDerivedStructInObjectFixture, NotAppliedMemberInDerivedStructInArrayIsNotOverwritten)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestDerivedStructWithNetSerializerWithApply_TestObject_Array* ServerObject = Server->CreateObject<UTestDerivedStructWithNetSerializerWithApply_TestObject_Array>();

	// Add items to the array
	{
		FTestDerivedStruct_Inherited_WithNetSerializerWithApply_Inherited_WithoutNetSerializer Item;
		for (int It = 0; It < 3; ++It)
		{
			ServerObject->DerivedStructArray.Add(Item);
		}
	}

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestDerivedStructWithNetSerializerWithApply_TestObject_Array* ClientObject = Cast<UTestDerivedStructWithNetSerializerWithApply_TestObject_Array>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_GT(ClientObject->DerivedStructArray.Num(), 0);

	// On the client modify some non-applied properties
	const uint8 ClientModifiedNotAppliedPropertyBaseValue = ClientObject->DerivedStructArray[0].ByteMemberNotSetOnApply ^ 123U;
	for (auto& Element : ClientObject->DerivedStructArray)
	{
		const uint8 Index = IntCastChecked<uint8>(&Element - ClientObject->DerivedStructArray.GetData());
		Element.ByteMemberNotSetOnApply = ClientModifiedNotAppliedPropertyBaseValue + Index;
	}

	// On the server modify all elements in the array
	for (auto& Element : ServerObject->DerivedStructArray)
	{
		Element.ByteMember0 ^= 1U;
		Element.ByteMember1 ^= 2U;
		Element.ByteMemberNotSetOnApply ^= 231U;
	}

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the not applied member remains as is.
	for (const auto& Element : ClientObject->DerivedStructArray)
	{
		const uint8 Index = IntCastChecked<uint8>(&Element - ClientObject->DerivedStructArray.GetData());
		UE_NET_ASSERT_EQ(Element.ByteMemberNotSetOnApply, uint8(ClientModifiedNotAppliedPropertyBaseValue + Index));
	}
}

UE_NET_TEST_FIXTURE(FTestDerivedStructInObjectFixture, NotAppliedMemberInStructDerivedFromDerivedStructIsNotOverwritten)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestDerivedStructWithNetSerializerWithApply_Inherited_TestObject_Member* ServerObject = Server->CreateObject<UTestDerivedStructWithNetSerializerWithApply_Inherited_TestObject_Member>();

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestDerivedStructWithNetSerializerWithApply_Inherited_TestObject_Member* ClientObject = Cast<UTestDerivedStructWithNetSerializerWithApply_Inherited_TestObject_Member>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// On the client modify the non-applied property
	const uint8 ClientModifiedNotAppliedPropertyValue = ClientObject->DerivedStruct.ByteMemberNotSetOnApply ^ 123U;
	ClientObject->DerivedStruct.ByteMemberNotSetOnApply = ClientModifiedNotAppliedPropertyValue;

	// On the server modify all members
	ServerObject->DerivedStruct.ByteMember0 ^= 1U;
	ServerObject->DerivedStruct.ByteMember2 ^= 2U;
	ServerObject->DerivedStruct.ByteMemberNotSetOnApply = ClientModifiedNotAppliedPropertyValue ^ 231U;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify state, most importantly that the not applied member remains as is.
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMember0, ServerObject->DerivedStruct.ByteMember0);
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMember2, ServerObject->DerivedStruct.ByteMember2);
	UE_NET_ASSERT_EQ(ClientObject->DerivedStruct.ByteMemberNotSetOnApply, ClientModifiedNotAppliedPropertyValue);
}

UE_NET_TEST_FIXTURE(FTestDerivedStructInObjectFixture, NotAppliedMemberInStructDerivedFromDerivedStructInArrayIsNotOverwritten)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestDerivedStructWithNetSerializerWithApply_Inherited_TestObject_Array* ServerObject = Server->CreateObject<UTestDerivedStructWithNetSerializerWithApply_Inherited_TestObject_Array>();

	// Add items to the array
	{
		FTestDerivedStruct_Inherited_WithNetSerializerWithApply_Inherited_WithoutNetSerializer Item;
		for (int It = 0; It < 3; ++It)
		{
			ServerObject->DerivedStructArray.Add(Item);
		}
	}

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestDerivedStructWithNetSerializerWithApply_Inherited_TestObject_Array* ClientObject = Cast<UTestDerivedStructWithNetSerializerWithApply_Inherited_TestObject_Array>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_GT(ClientObject->DerivedStructArray.Num(), 0);

	// On the client modify some non-applied properties
	// On the client modify the non-replicated property
	constexpr uint8 ClientModifiedNotReplicatedPropertyValue = 255U;
	const uint8 ClientModifiedNotAppliedPropertyBaseValue = ClientObject->DerivedStructArray[0].ByteMemberNotSetOnApply ^ 123U;
	for (auto& Element : ClientObject->DerivedStructArray)
	{
		const uint8 Index = IntCastChecked<uint8>(&Element - ClientObject->DerivedStructArray.GetData());
		Element.ByteMemberNotSetOnApply = ClientModifiedNotAppliedPropertyBaseValue + Index;
		Element.ByteMember3_NotReplicated = ClientModifiedNotReplicatedPropertyValue;
	}

	// On the server modify all elements in the array
	for (auto& Element : ServerObject->DerivedStructArray)
	{
		Element.ByteMember0 ^= 1U;
		Element.ByteMemberNotSetOnApply ^= 231U;
		Element.ByteMember2 ^= 2U;
		Element.ByteMember3_NotReplicated = ClientModifiedNotReplicatedPropertyValue ^ 0x7FU;
	}

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the not applied member remains as is.
	for (const auto& Element : ClientObject->DerivedStructArray)
	{
		const uint8 Index = IntCastChecked<uint8>(&Element - ClientObject->DerivedStructArray.GetData());
		UE_NET_ASSERT_EQ(Element.ByteMember0, ServerObject->DerivedStructArray[Index].ByteMember0);
		UE_NET_ASSERT_EQ(Element.ByteMemberNotSetOnApply, uint8(ClientModifiedNotAppliedPropertyBaseValue + Index));
		UE_NET_ASSERT_EQ(Element.ByteMember2, ServerObject->DerivedStructArray[Index].ByteMember2);
		UE_NET_ASSERT_EQ(Element.ByteMember3_NotReplicated, ClientModifiedNotReplicatedPropertyValue);
	}
}

// FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer implementation
void FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(Value.ByteMember0, 8U);
	Writer->WriteBits(Value.ByteMember1, 8U);
}

void FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	Target.ByteMember0 = IntCastChecked<uint8>(Reader->ReadBits(8U));
	Target.ByteMember1 = IntCastChecked<uint8>(Reader->ReadBits(8U));
}

void FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	Target.ByteMember0 = Source.ByteMember0;
	Target.ByteMember1 = Source.ByteMember1;
}

void FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	Target.ByteMember0 = Source.ByteMember0;
	Target.ByteMember1 = Source.ByteMember1;
}

bool FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);
		return Value0.ByteMember0 == Value1.ByteMember0 && Value0.ByteMember1 == Value1.ByteMember1;
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<const SourceType*>(Args.Source1);
		return Value0.ByteMember0 == Value1.ByteMember0 && Value0.ByteMember1 == Value1.ByteMember1;
	}
}

FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_TestDerivedStruct_Inherited_WithNetSerializer);
}

void FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_TestDerivedStruct_Inherited_WithNetSerializer);

}

// FTestDerivedStruct_Inherited_WithNetSerializerWithApply_NetSerializer implementation
void FTestDerivedStruct_Inherited_WithNetSerializerWithApply_NetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(Value.ByteMember0, 8U);
	Writer->WriteBits(Value.ByteMember1, 8U);
	// All serialization, dequantization etc is done on ByteMemberNotSetOnApply so that we can detect it's not applied.
	Writer->WriteBits(Value.ByteMemberNotSetOnApply, 8U);
}

void FTestDerivedStruct_Inherited_WithNetSerializerWithApply_NetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	Target.ByteMember0 = IntCastChecked<uint8>(Reader->ReadBits(8U));
	Target.ByteMember1 = IntCastChecked<uint8>(Reader->ReadBits(8U));
	Target.ByteMemberNotSetOnApply = IntCastChecked<uint8>(Reader->ReadBits(8U));
}

void FTestDerivedStruct_Inherited_WithNetSerializerWithApply_NetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	Target.ByteMember0 = Source.ByteMember0;
	Target.ByteMember1 = Source.ByteMember1;
	Target.ByteMemberNotSetOnApply = Source.ByteMemberNotSetOnApply;
}

void FTestDerivedStruct_Inherited_WithNetSerializerWithApply_NetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	Target.ByteMember0 = Source.ByteMember0;
	Target.ByteMember1 = Source.ByteMember1;
	Target.ByteMemberNotSetOnApply = Source.ByteMemberNotSetOnApply;
}

bool FTestDerivedStruct_Inherited_WithNetSerializerWithApply_NetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);
		return Value0.ByteMember0 == Value1.ByteMember0 && Value0.ByteMember1 == Value1.ByteMember1 && Value0.ByteMemberNotSetOnApply == Value1.ByteMemberNotSetOnApply;
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<const SourceType*>(Args.Source1);
		return Value0.ByteMember0 == Value1.ByteMember0 && Value0.ByteMember1 == Value1.ByteMember1 && Value0.ByteMemberNotSetOnApply == Value1.ByteMemberNotSetOnApply;
	}
}

void FTestDerivedStruct_Inherited_WithNetSerializerWithApply_NetSerializer::Apply(FNetSerializationContext&, const FNetApplyArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	Target.ByteMember0 = Source.ByteMember0;
	Target.ByteMember1 = Source.ByteMember1;
	// Not setting ByteMemberNotSetOnApply
}

FTestDerivedStruct_Inherited_WithNetSerializerWithApply_NetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_TestDerivedStruct_Inherited_WithNetSerializerWithApply);
}

void FTestDerivedStruct_Inherited_WithNetSerializerWithApply_NetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_TestDerivedStruct_Inherited_WithNetSerializerWithApply);

}

// FTestDerivedStruct_Inherited_WithNetSerializer_DeepInheritanceOfStructWithNetSerializer will not be used for replication testing so it's ok to use the NopNetSerializer.
UE_NET_IMPLEMENT_FORWARDING_NETSERIALIZER_AND_REGISTRY_DELEGATES(TestDerivedStruct_Inherited_WithNetSerializer_DeepInheritanceOfStructWithNetSerializer, FNopNetSerializer);

UTestDerivedStruct_TestObject_Member* FTestDerivedStructInObjectFixture::CreateObjectWithMember()
{
	return Server->CreateObject<UTestDerivedStruct_TestObject_Member>();
}

UTestDerivedStruct_TestObject_Array* FTestDerivedStructInObjectFixture::CreateObjectWithArray()
{
	return Server->CreateObject<UTestDerivedStruct_TestObject_Array>();
}

}

FTestDerivedStruct_Base::~FTestDerivedStruct_Base() = default;

void UTestDerivedStruct_TestObject_Member::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = false;

	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, DerivedStruct, Params);
}

void UTestDerivedStruct_TestObject_Member::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

void UTestDerivedStruct_TestObject_Array::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = false;

	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, DerivedStructArray, Params);
}

void UTestDerivedStruct_TestObject_Array::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

void UTestDerivedStructWithNetSerializerWithApply_TestObject_Member::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = false;

	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, DerivedStruct, Params);
}

void UTestDerivedStructWithNetSerializerWithApply_TestObject_Member::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

void UTestDerivedStructWithNetSerializerWithApply_TestObject_Array::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = false;

	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, DerivedStructArray, Params);
}

void UTestDerivedStructWithNetSerializerWithApply_TestObject_Array::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

void UTestDerivedStructWithNetSerializerWithApply_Inherited_TestObject_Member::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = false;

	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, DerivedStruct, Params);
}

void UTestDerivedStructWithNetSerializerWithApply_Inherited_TestObject_Member::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

void UTestDerivedStructWithNetSerializerWithApply_Inherited_TestObject_Array::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = false;

	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, DerivedStructArray, Params);
}

void UTestDerivedStructWithNetSerializerWithApply_Inherited_TestObject_Array::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}
