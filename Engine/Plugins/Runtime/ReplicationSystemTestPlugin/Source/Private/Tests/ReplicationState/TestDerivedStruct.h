// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkAutomationTest.h"
#include "Iris/Serialization/NetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "TestDerivedStruct.generated.h"

USTRUCT()
struct alignas(16) FTestDerivedStruct_Base
{
	GENERATED_BODY()
	
	virtual ~FTestDerivedStruct_Base();

	UPROPERTY(Transient)
	uint8 ByteMember0 = 5;
};

USTRUCT()
struct FTestDerivedStruct_Inherited_WithNetSerializer : public FTestDerivedStruct_Base
{
	GENERATED_BODY()
	
	UPROPERTY(Transient)
	uint8 ByteMember1 = 25;
};

USTRUCT()
struct FTestDerivedStruct_Inherited_WithNetSerializer_Inherited_WithoutNetSerializer : public FTestDerivedStruct_Inherited_WithNetSerializer
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	uint8 ByteMember2 = 125;

	UPROPERTY(Transient, NotReplicated)
	uint8 ByteMember3_NotReplicated = 0;
};

USTRUCT()
struct FTestDerivedStruct_Inherited_WithNetSerializerWithApply : public FTestDerivedStruct_Base
{
	GENERATED_BODY()
	
	UPROPERTY(Transient)
	uint8 ByteMember1 = 25;

	UPROPERTY(Transient)
	uint8 ByteMemberNotSetOnApply = 33;
};

USTRUCT()
struct FTestDerivedStruct_Inherited_WithNetSerializerWithApply_Inherited_WithoutNetSerializer : public FTestDerivedStruct_Inherited_WithNetSerializerWithApply
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	uint8 ByteMember2 = 125;

	UPROPERTY(Transient, NotReplicated)
	uint8 ByteMember3_NotReplicated = 0;
};

USTRUCT()
struct FTestDerivedStruct_DeepInheritanceOfStructWithNetSerializer : public FTestDerivedStruct_Inherited_WithNetSerializer_Inherited_WithoutNetSerializer
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	float YetAnotherMember;
};

// This struct will use a forwarding serializer rather than implement a separate custom one.
USTRUCT()
struct FTestDerivedStruct_Inherited_WithNetSerializer_DeepInheritanceOfStructWithNetSerializer : public FTestDerivedStruct_DeepInheritanceOfStructWithNetSerializer
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	uint8 ByteMember4;
};

USTRUCT()
struct FTestDerivedStruct_Inherited_Inherited_WithNetSerializer_DeepInheritanceOfStructWithNetSerializer : public FTestDerivedStruct_Inherited_WithNetSerializer_DeepInheritanceOfStructWithNetSerializer
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	uint8 ByteMember5;
};

UCLASS()
class UTestDerivedStruct_TestObject_Member : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Replicated, Transient)
	FTestDerivedStruct_Inherited_WithNetSerializer_Inherited_WithoutNetSerializer DerivedStruct;

protected:
	virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;
};

UCLASS()
class UTestDerivedStruct_TestObject_Array : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Replicated, Transient)
	TArray<FTestDerivedStruct_Inherited_WithNetSerializer_Inherited_WithoutNetSerializer> DerivedStructArray;

protected:
	virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;
};

UCLASS()
class UTestDerivedStructWithNetSerializerWithApply_TestObject_Member : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Replicated, Transient)
	FTestDerivedStruct_Inherited_WithNetSerializerWithApply DerivedStruct;

protected:
	virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;
};

UCLASS()
class UTestDerivedStructWithNetSerializerWithApply_TestObject_Array : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Replicated, Transient)
	TArray<FTestDerivedStruct_Inherited_WithNetSerializerWithApply> DerivedStructArray;

protected:
	virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;
};

UCLASS()
class UTestDerivedStructWithNetSerializerWithApply_Inherited_TestObject_Member : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Replicated, Transient)
	FTestDerivedStruct_Inherited_WithNetSerializerWithApply_Inherited_WithoutNetSerializer DerivedStruct;

protected:
	virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;
};

UCLASS()
class UTestDerivedStructWithNetSerializerWithApply_Inherited_TestObject_Array : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Replicated, Transient)
	TArray<FTestDerivedStruct_Inherited_WithNetSerializerWithApply_Inherited_WithoutNetSerializer> DerivedStructArray;

protected:
	virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;
};

namespace UE::Net::Private
{

UE_NET_DECLARE_SERIALIZER(FTestDerivedStruct_Inherited_WithNetSerializer_NetSerializer, REPLICATIONSYSTEMTESTPLUGIN_API);
UE_NET_DECLARE_SERIALIZER(FTestDerivedStruct_Inherited_WithNetSerializerWithApply_NetSerializer, REPLICATIONSYSTEMTESTPLUGIN_API);

}
