// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestFastArrayReplicationState.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationState/InternalReplicationStateDescriptorUtils.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/InternalNetSerializerUtils.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Misc/EnumClassFlags.h"
#include "Net/UnrealNetwork.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Iris/ReplicationSystem/FastArrayReplicationFragment.h"

UTestFastArrayReplicationState_FastArray_TestClassFastArray::UTestFastArrayReplicationState_FastArray_TestClassFastArray()
: UReplicatedTestObject()
{
}

void UTestFastArrayReplicationState_FastArray_TestClassFastArray::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;

	DOREPLIFETIME_WITH_PARAMS_FAST(UTestFastArrayReplicationState_FastArray_TestClassFastArray, FastArray, Params);

}

void UTestFastArrayReplicationState_FastArray_TestClassFastArray::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

UTestFastArrayReplicationState_FastArray_TestClassFastArrayWithExtraProperty::UTestFastArrayReplicationState_FastArray_TestClassFastArrayWithExtraProperty()
: UReplicatedTestObject()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Note: this is a test for something that should not be done. This is purely to avoid refactoring existing code before full transition to iris.
// A FastArraySerializer struct should not contain any additional data.
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
class FTestFastArrayReplicationState_FastArrayWithExtraProperty::FastArrayWithExtraPropertiesReplicationFragment : public UE::Net::TFastArrayReplicationFragment<typename TFastArrayTypeHelper<FTestFastArrayReplicationState_FastArrayWithExtraProperty>::FastArrayItemType, FTestFastArrayReplicationState_FastArrayWithExtraProperty>
{
public:
	typedef TFastArrayReplicationFragment<typename TFastArrayTypeHelper<FTestFastArrayReplicationState_FastArrayWithExtraProperty>::FastArrayItemType, FTestFastArrayReplicationState_FastArrayWithExtraProperty> SuperT;

	FastArrayWithExtraPropertiesReplicationFragment(UE::Net::EReplicationFragmentTraits InTraits, UObject* InOwner, const UE::Net::FReplicationStateDescriptor* InDescriptor)
	: SuperT(InTraits, InOwner, InDescriptor, EAllowAdditionalPropertiesType::AllowAdditionalProperties)
	{
	}

	virtual void ApplyReplicatedState(UE::Net::FReplicationStateApplyContext& Context) const override
	{
		using namespace UE::Net;
		using namespace UE::Net::Private;
		
		// Forward to fast array serializer
		SuperT::ApplyReplicatedState(Context);

		// Must explicitly deal with extra properties here, but there is a utility function to do it for us!
		SuperT::ApplyReplicatedStateForExtraProperties(Context);
	}

	virtual bool PollReplicatedState(UE::Net::EReplicationFragmentPollFlags PollOption) override
	{
		// Lookup source data, we need the actual FastArraySerializer and the Array it is wrapping
		FTestFastArrayReplicationState_FastArrayWithExtraProperty* SrcArraySerializer = this->GetFastArraySerializerFromOwner();
	
		// Lookup destination data
		FTestFastArrayReplicationState_FastArrayWithExtraProperty* DstArraySerializer = this->GetFastArraySerializerFromReplicationState();

		// Explicit poll of additional properties in the fastarray and mark dirty if needed
		if (SrcArraySerializer->ExtraInt != DstArraySerializer->ExtraInt)
		{	
			this->MarkDirty();
		}
		DstArraySerializer->ExtraInt = SrcArraySerializer->ExtraInt;

		return SuperT::PollReplicatedState(PollOption);
	}
};

UE::Net::FReplicationFragment* FTestFastArrayReplicationState_FastArrayWithExtraProperty::CreateAndRegisterReplicationFragment(UObject* Owner, const UE::Net::FReplicationStateDescriptor* Descriptor, UE::Net::FFragmentRegistrationContext& Context)
{
	using namespace UE::Net;
	static_assert(TFastArrayTypeHelper<FTestFastArrayReplicationState_FastArrayWithExtraProperty>::HasValidFastArrayItemType(), "Invalid FastArrayItemType detected. Make sure that FastArraySerializer has a single replicated dynamic array");

	if (FTestFastArrayReplicationState_FastArrayWithExtraProperty::FastArrayWithExtraPropertiesReplicationFragment* Fragment = new FTestFastArrayReplicationState_FastArrayWithExtraProperty::FastArrayWithExtraPropertiesReplicationFragment(Context.GetFragmentTraits(), Owner, Descriptor))
	{
		Fragment->Register(Context, EReplicationFragmentTraits::DeleteWithInstanceProtocol);
		return Fragment;
	}
	return nullptr;
}

//////////////////////////////////////////////////////////////////////////

void UTestFastArrayReplicationState_FastArray_TestClassFastArrayWithExtraProperty::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params;
	Params.CreateAndRegisterReplicationFragmentFunction = &FTestFastArrayReplicationState_FastArrayWithExtraProperty::CreateAndRegisterReplicationFragment;
	DOREPLIFETIME_WITH_PARAMS_FAST(UTestFastArrayReplicationState_FastArray_TestClassFastArrayWithExtraProperty, FastArray, Params);
}

void UTestFastArrayReplicationState_FastArray_TestClassFastArrayWithExtraProperty::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	// Explicitly allow building descriptors for FastArray with additional properties to be allow to test this special case.
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags | UE::Net::EFragmentRegistrationFlags::AllowFastArraysWithAdditionalProperties);
}

void FTestFastArrayReplicationState_FastArrayItem::PostReplicatedAdd(const struct FTestFastArrayReplicationState_FastArraySerializer& InArraySerializer)
{
	const_cast<FTestFastArrayReplicationState_FastArraySerializer&>(InArraySerializer).bHitReplicatedAdd = true;
}

void FTestFastArrayReplicationState_FastArrayItem::PostReplicatedChange(const struct FTestFastArrayReplicationState_FastArraySerializer& InArraySerializer)
{
	const_cast<FTestFastArrayReplicationState_FastArraySerializer&>(InArraySerializer).bHitReplicatedChange = true;
}

void FTestFastArrayReplicationState_FastArrayItem::PreReplicatedRemove(const struct FTestFastArrayReplicationState_FastArraySerializer& InArraySerializer)
{
	const_cast<FTestFastArrayReplicationState_FastArraySerializer&>(InArraySerializer).bHitReplicatedRemove = true;
}
namespace UE::Net::Private
{

struct FTestFastArrayReplicationStateContext : public FNetworkAutomationTestSuiteFixture
{
	FReplicationStateDescriptorBuilder::FResult Descriptors;
	const FReplicationStateDescriptor* Descriptor;

	void InitDescriptor(UClass* StaticClass, int32 RepIndex = -1)
	{
		FReplicationStateDescriptorBuilder::FParameters BuilderParameters;
		BuilderParameters.DescriptorRegistry = nullptr;
		BuilderParameters.SinglePropertyIndex = RepIndex;

		if (FReplicationStateDescriptorBuilder::CreateDescriptorsForClass(Descriptors, StaticClass, BuilderParameters))
		{
			Descriptor = Descriptors[0];
		}
		else
		{
			Descriptor = nullptr;
		}		
	}

	void ValidateFastArrayDescriptor(const FReplicationStateDescriptor* FastArrayDescriptor)
	{
		// Verify class descriptor
		UE_NET_ASSERT_TRUE(FastArrayDescriptor != nullptr);
		UE_NET_ASSERT_TRUE(EnumHasAnyFlags(FastArrayDescriptor->Traits, EReplicationStateTraits::IsFastArrayReplicationState));	
		UE_NET_ASSERT_EQ((uint32)FastArrayDescriptor->MemberCount, 1U);
		UE_NET_ASSERT_TRUE(IsUsingStructNetSerializer(FastArrayDescriptor->MemberSerializerDescriptors[0]));

		// Verify Fast array descriptor
		const FReplicationStateDescriptor* FastArrayStructDescriptor = static_cast<const FStructNetSerializerConfig*>(FastArrayDescriptor->MemberSerializerDescriptors[0].SerializerConfig)->StateDescriptor;
		UE_NET_ASSERT_EQ((uint32)FastArrayStructDescriptor->MemberCount, 1U);

		// This is the struct descriptor describing the array property
		UE_NET_ASSERT_TRUE(IsUsingArrayPropertyNetSerializer(FastArrayStructDescriptor->MemberSerializerDescriptors[0]));
		const FReplicationStateDescriptor* FastArrayItemsDescriptor = static_cast<const FArrayPropertyNetSerializerConfig*>(FastArrayStructDescriptor->MemberSerializerDescriptors[0].SerializerConfig)->StateDescriptor;
		UE_NET_ASSERT_EQ((uint32)FastArrayItemsDescriptor->MemberCount, 1U);

		UE_NET_ASSERT_TRUE(IsUsingStructNetSerializer(FastArrayItemsDescriptor->MemberSerializerDescriptors[0]));
		const FReplicationStateDescriptor* FastArrayItemDescriptor = static_cast<const FStructNetSerializerConfig*>(FastArrayItemsDescriptor->MemberSerializerDescriptors[0].SerializerConfig)->StateDescriptor;
		UE_NET_ASSERT_EQ((uint32)FastArrayItemDescriptor->MemberCount, 4U);
	}
};

UE_NET_TEST_FIXTURE(FTestFastArrayReplicationStateContext, BuildClassDescriptorForSingleProperty)
{
	InitDescriptor(UTestFastArrayReplicationState_FastArray_TestClassFastArray::StaticClass(), (int32)UTestFastArrayReplicationState_FastArray_TestClassFastArray::ENetFields_Private::FastArray);

	ValidateFastArrayDescriptor(Descriptor);
}

UE_NET_TEST_FIXTURE(FTestFastArrayReplicationStateContext, BuildClassDescriptorForSinglePropertyForFastArrayWithExtraProperty)
{
	// We expect this test to fail as we do not pass the flag allowing fastarrays to contain extra properties
	{
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Fatal);
		InitDescriptor(UTestFastArrayReplicationState_FastArray_TestClassFastArrayWithExtraProperty::StaticClass(), (int32)UTestFastArrayReplicationState_FastArray_TestClassFastArrayWithExtraProperty::ENetFields_Private::FastArray);
	}
	UE_NET_ASSERT_FALSE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::IsFastArrayReplicationState));	
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestAddItem)
{
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ServerObject = Server->CreateObject<UTestFastArrayReplicationState_FastArray_TestClassFastArray>();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Store Pointer to objects
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ClientObject = Cast<UTestFastArrayReplicationState_FastArray_TestClassFastArray>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Verify initial replication
	UE_NET_ASSERT_TRUE(ClientObject!= nullptr);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Verify that we received the data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());

	// Add data to array
	FTestFastArrayReplicationState_FastArrayItem NewEntry;
	NewEntry.bRepBool = false;
	NewEntry.RepInt32 = 0U;
	
	ServerObject->FastArray.Edit().Add(NewEntry);

	NewEntry.bRepBool = true;
	NewEntry.RepInt32 = 1U;
	
	ServerObject->FastArray.Edit().Add(NewEntry);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we received the data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());

	// In this case we expect order to be the same
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);

	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestRemoveItem)
{
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ServerObject = Server->CreateObject<UTestFastArrayReplicationState_FastArray_TestClassFastArray>();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Store Pointer to objects
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ClientObject = Cast<UTestFastArrayReplicationState_FastArray_TestClassFastArray>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Verify initial replication
	UE_NET_ASSERT_TRUE(ClientObject!= nullptr);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Verify that we received the data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());

	// Add data to array
	FTestFastArrayReplicationState_FastArrayItem NewEntry;
	NewEntry.bRepBool = false;
	NewEntry.RepInt32 = 0U;
	
	ServerObject->FastArray.Edit().Add(NewEntry);

	NewEntry.bRepBool = true;
	NewEntry.RepInt32 = 1U;
	
	ServerObject->FastArray.Edit().Add(NewEntry);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we received the data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());

	// In this case we expect order to be the same
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);

	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Remove Item
	ServerObject->FastArray.Edit().Remove(0);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we received the data
	UE_NET_ASSERT_TRUE(ClientObject!= nullptr);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());

	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedRemove);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestLocalItemArePreserved)
{
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ServerObject = Server->CreateObject<UTestFastArrayReplicationState_FastArray_TestClassFastArray>();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Store Pointer to objects
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ClientObject = Cast<UTestFastArrayReplicationState_FastArray_TestClassFastArray>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Verify initial replication
	UE_NET_ASSERT_TRUE(ClientObject!= nullptr);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Verify that we received the data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());

	// Add local data to client array
	{
		FTestFastArrayReplicationState_FastArrayItem NewEntry;
		NewEntry.bRepBool = false;
		NewEntry.RepInt32 = 0U;

		ClientObject->FastArray.Edit().AddLocal(NewEntry);
	}

	// Add data to array
	FTestFastArrayReplicationState_FastArrayItem NewEntry;
	NewEntry.bRepBool = true;
	NewEntry.RepInt32 = 1U;
	
	ServerObject->FastArray.Edit().Add(NewEntry);

	NewEntry.bRepBool = true;
	NewEntry.RepInt32 = 2U;
	
	ServerObject->FastArray.Edit().Add(NewEntry);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we received the data
	UE_NET_ASSERT_EQ(3, ClientObject->FastArray.GetItemArray().Num());

	// Verify local item
	UE_NET_ASSERT_EQ(0, ClientObject->FastArray.GetItemArray()[0].RepInt32);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[1].RepInt32);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[1].RepInt32, ClientObject->FastArray.GetItemArray()[2].RepInt32);

	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Remove Item
	ServerObject->FastArray.Edit().Remove(0);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we received the remove
	UE_NET_ASSERT_EQ(2, ClientObject->FastArray.GetItemArray().Num());

	// Verify local item
	UE_NET_ASSERT_EQ(0, ClientObject->FastArray.GetItemArray()[0].RepInt32);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[1].RepInt32);

	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedRemove);
}

// Add test for change item
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestModifyItem)
{
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ServerObject = Server->CreateObject<UTestFastArrayReplicationState_FastArray_TestClassFastArray>();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Store Pointer to objects
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ClientObject = Cast<UTestFastArrayReplicationState_FastArray_TestClassFastArray>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Verify initial replication
	UE_NET_ASSERT_TRUE(ClientObject!= nullptr);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Verify that we received the data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());

	// Add data to array
	FTestFastArrayReplicationState_FastArrayItem NewEntry;
	NewEntry.bRepBool = false;
	NewEntry.RepInt32 = 0U;
	
	ServerObject->FastArray.Edit().Add(NewEntry);

	NewEntry.bRepBool = true;
	NewEntry.RepInt32 = 1U;
	
	ServerObject->FastArray.Edit().Add(NewEntry);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we received the data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());

	// In this case we expect order to be the same
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);

	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	ClientObject->FastArray.bHitReplicatedAdd = false;
	ClientObject->FastArray.bHitReplicatedChange = false;
	ClientObject->FastArray.bHitReplicatedRemove = false;

	// Modify data
	ServerObject->FastArray.Edit()[0].RepInt32 = 3U;
	
	// Send and deliver packet
	Server->UpdateAndSend({Client});

	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Verify that we received the modified data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);
}

// Add test for change item
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestInsertItem)
{
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ServerObject = Server->CreateObject<UTestFastArrayReplicationState_FastArray_TestClassFastArray>();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Store Pointer to objects
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ClientObject = Cast<UTestFastArrayReplicationState_FastArray_TestClassFastArray>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Verify initial replication
	UE_NET_ASSERT_TRUE(ClientObject!= nullptr);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Verify that we received the data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());

	// Add data to array
	FTestFastArrayReplicationState_FastArrayItem NewEntry;
	NewEntry.bRepBool = false;
	NewEntry.RepInt32 = 0U;
	
	ServerObject->FastArray.Edit().Add(NewEntry);

	NewEntry.bRepBool = false;
	NewEntry.RepInt32 = 12U;

	ServerObject->FastArray.Edit().AddLocal(NewEntry);

	NewEntry.bRepBool = true;
	NewEntry.RepInt32 = 1U;
	
	ServerObject->FastArray.Edit().Add(NewEntry);

	// Dirty middle
	ServerObject->FastArray.Edit()[1].RepInt32 = 13U;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we received the data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());

	// In this case we expect order to be the same
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);

	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	ClientObject->FastArray.bHitReplicatedAdd = false;
	ClientObject->FastArray.bHitReplicatedChange = false;
	ClientObject->FastArray.bHitReplicatedRemove = false;

	// Modify data
	ServerObject->FastArray.Edit()[0].RepInt32 = 3U;
	
	// Send and deliver packet
	Server->UpdateAndSend({Client});

	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Verify that we received the modified data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);
}


// Add test for change item
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestModifyItemDoesNotEverwriteLocalVariables)
{
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ServerObject = Server->CreateObject<UTestFastArrayReplicationState_FastArray_TestClassFastArray>();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Store Pointer to objects
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ClientObject = Cast<UTestFastArrayReplicationState_FastArray_TestClassFastArray>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Verify initial replication
	UE_NET_ASSERT_TRUE(ClientObject!= nullptr);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Verify that we received the data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());

	// Grab array editor
	auto FastArrayEditor = ServerObject->FastArray.Edit();

	// Add data to array
	FTestFastArrayReplicationState_FastArrayItem NewEntry;
	NewEntry.bRepBool = false;
	NewEntry.RepInt32 = 0U;
	
	FastArrayEditor.Add(NewEntry);

	NewEntry.bRepBool = true;
	NewEntry.RepInt32 = 1U;
	
	FastArrayEditor.Add(NewEntry);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we received the data
	UE_NET_ASSERT_EQ(FastArrayEditor.Num(), ClientObject->FastArray.GetItemArray().Num());

	// In this case we expect order to be the same
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);

	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Store some local data in the not replicated var
	const int32 LocallyStoredInt32 = 32;
	ClientObject->FastArray.Edit().EditLocal(0).NotRepInt32 = LocallyStoredInt32;
	
	ClientObject->FastArray.bHitReplicatedAdd = false;
	ClientObject->FastArray.bHitReplicatedChange = false;
	ClientObject->FastArray.bHitReplicatedRemove = false;

	// Modify data
	FastArrayEditor.Edit(0).RepInt32 = 3;
	
	// Send and deliver packet
	Server->UpdateAndSend({Client});

	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Verify that we received the modified data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);

	// Verify that the locally stored data was not overwritten
	UE_NET_ASSERT_EQ(LocallyStoredInt32, ClientObject->FastArray.GetItemArray()[0].NotRepInt32);	
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestAddItemWithoutMarkDirty)
{
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ServerObject = Server->CreateObject<UTestFastArrayReplicationState_FastArray_TestClassFastArray>();

	FTestFastArrayReplicationState_FastArrayItem NewEntry;
	NewEntry.bRepBool = false;
	NewEntry.RepInt32 = 0U;
	
	ServerObject->FastArray.Edit().Add(NewEntry);

	NewEntry.bRepBool = true;
	NewEntry.RepInt32 = 1U;
	
	// Modify item without marking it dirty
	ServerObject->FastArray.Edit().AddLocal(NewEntry);

	// Verify that only first entry is assigned ReplicationID
	UE_NET_ASSERT_NE(ServerObject->FastArray.GetItemArray()[0].ReplicationID, -1);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[1].ReplicationID, -1);

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Send and deliver packet
	Server->NetUpdate();

	// Verify that poll has modified the index
	UE_NET_ASSERT_NE(ServerObject->FastArray.GetItemArray()[0].ReplicationID, -1);
	UE_NET_ASSERT_NE(ServerObject->FastArray.GetItemArray()[1].ReplicationID, -1);

	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Store Pointer to objects
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ClientObject = Cast<UTestFastArrayReplicationState_FastArray_TestClassFastArray>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Verify that we received the data
	UE_NET_ASSERT_TRUE(ClientObject!= nullptr);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());

	// In this case we expect order to be the same
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestChangeMaskWrapAround)
{
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ServerObject = Server->CreateObject<UTestFastArrayReplicationState_FastArray_TestClassFastArray>();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Store Pointer to objects
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ClientObject = Cast<UTestFastArrayReplicationState_FastArray_TestClassFastArray>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Verify initial replication
	UE_NET_ASSERT_TRUE(ClientObject!= nullptr);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Verify that we received the data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());

	// Create FastArrayEditor, note this is only valid as long as the FastArray is valid
	auto FastArrayEditor = ServerObject->FastArray.Edit();

	// Add data to array
	constexpr int32 InitialItemCount = 32;
	for (int32 ItemIt = 0; ItemIt < InitialItemCount; ++ItemIt)
	{
		FTestFastArrayReplicationState_FastArrayItem NewEntry;
		NewEntry.bRepBool = false;
		NewEntry.RepInt32 = ItemIt;
	
		FastArrayEditor.Add(NewEntry);
	}
	int32 CurrentItemCount = FastArrayEditor.Num();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we received the data
	UE_NET_ASSERT_EQ(CurrentItemCount, ClientObject->FastArray.GetItemArray().Num());

	// In this case we expect order to be the same
	for (int32 ItemIt = 0; ItemIt < InitialItemCount; ++ItemIt)
	{
		UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[ItemIt].RepInt32, ClientObject->FastArray.GetItemArray()[ItemIt].RepInt32);		
	}

	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Add Item that reuses a bit
	{
		FTestFastArrayReplicationState_FastArrayItem NewEntry;
		NewEntry.bRepBool = true;
		NewEntry.RepInt32 = 32;

		FastArrayEditor.Add(NewEntry);
		++CurrentItemCount;
	}
	
	// Send and deliver packet
	Server->UpdateAndSend({Client});

	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[CurrentItemCount - 1].RepInt32, ClientObject->FastArray.GetItemArray()[CurrentItemCount - 1].RepInt32);

	// Add another item and modify the one sharing a changemask
	{
		FTestFastArrayReplicationState_FastArrayItem NewEntry;
		NewEntry.bRepBool = true;
		NewEntry.RepInt32 = 33;

		FastArrayEditor.Add(NewEntry);
		++CurrentItemCount;

		FastArrayEditor.Edit(1).bRepBool = true;
	}

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// We should both have a change and an add
	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[CurrentItemCount - 1].RepInt32, ClientObject->FastArray.GetItemArray()[CurrentItemCount - 1].RepInt32);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[1].bRepBool, ClientObject->FastArray.GetItemArray()[1].bRepBool);
}

static_assert(TModels_V<FFastArraySerializer::CPostReplicatedReceiveFuncable, FTestFastArrayReplicationState_FastArraySerializer, const FFastArraySerializer::FPostReplicatedReceiveParameters&>, "FTestFastArrayReplicationState_FastArraySerializer should have a function matching the requirements of FFastArraySerializer::CPostReplicatedReceiveFuncable");

// Test partial out of order resolve of references in  fast array
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestPostReplicatedReceiveCallback)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ServerObject = Server->CreateObject<UTestFastArrayReplicationState_FastArray_TestClassFastArray>();

	// Spawn objects to reference
	UReplicatedTestObject* ServerReferencedObjectA = Server->CreateObject(0 ,0);
	UReplicatedTestObject* ServerReferencedObjectB = Server->CreateObject(0 ,0);
	UReplicatedTestObject* ServerReferencedObjectC = Server->CreateObject(0 ,0);

	// Setup references
	auto FastArrayEditor = ServerObject->FastArray.Edit();

	FTestFastArrayReplicationState_FastArrayItem Item;
	Item.ObjectRef = ServerReferencedObjectA;
	FastArrayEditor.Add(Item);

	Item.ObjectRef = ServerReferencedObjectB;
	FastArrayEditor.Add(Item);

	// Make sure that references are not replicated to client
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectB->NetRefHandle);
	
	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that object has been spawned on client
	auto ClientObject = Client->GetObjectAs<UTestFastArrayReplicationState_FastArray_TestClassFastArray>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(ClientObject != nullptr);

	auto& ClientFastArray = ClientObject->FastArray;

	// Verify that array has been replicated
	UE_NET_ASSERT_EQ(2, ClientFastArray.GetItemArray().Num());

	// Verify that objects could not be resolved
	UE_NET_ASSERT_TRUE(ClientFastArray.GetItemArray()[0].ObjectRef == nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.GetItemArray()[1].ObjectRef == nullptr);

	// Verify that we hit the callback
	UE_NET_ASSERT_TRUE(ClientFastArray.bHitPostReplicatedReceive);
	UE_NET_ASSERT_TRUE(ClientFastArray.bHitDerivedPostReplicatedReceive);
	UE_NET_ASSERT_TRUE(ClientFastArray.bPostReplicatedReceiveWasHitWithUnresolvedReferences);

	// Reset
	ClientFastArray.bHitPostReplicatedReceive = false;
	ClientFastArray.bPostReplicatedReceiveWasHitWithUnresolvedReferences = false;

	// Enable replication for ServerReferenceObjectA
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we managed to resolve objectA even though we have other unresolved objects in the array
	UE_NET_ASSERT_TRUE(ClientFastArray.GetItemArray()[0].ObjectRef != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.GetItemArray()[1].ObjectRef == nullptr);

	UE_NET_ASSERT_TRUE(ClientFastArray.bHitPostReplicatedReceive);
	UE_NET_ASSERT_TRUE(ClientFastArray.bPostReplicatedReceiveWasHitWithUnresolvedReferences);

	// Reset
	ClientFastArray.bHitPostReplicatedReceive = false;
	ClientFastArray.bPostReplicatedReceiveWasHitWithUnresolvedReferences = false;

	// Enable replication for ServerReferenceObjectB 
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectB->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we managed to resolve both objects and that the 
	UE_NET_ASSERT_TRUE(ClientFastArray.GetItemArray()[0].ObjectRef != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.GetItemArray()[1].ObjectRef != nullptr);

	// Verify that we hit the callback with no more unresolved references
	UE_NET_ASSERT_TRUE(ClientFastArray.bHitPostReplicatedReceive);
	UE_NET_ASSERT_FALSE(ClientFastArray.bPostReplicatedReceiveWasHitWithUnresolvedReferences);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestFastArrayWithExtraProperty)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	UTestFastArrayReplicationState_FastArray_TestClassFastArrayWithExtraProperty* ServerObject = Server->CreateObject<UTestFastArrayReplicationState_FastArray_TestClassFastArrayWithExtraProperty>();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Store Pointer to objects
	UTestFastArrayReplicationState_FastArray_TestClassFastArrayWithExtraProperty* ClientObject = Cast<UTestFastArrayReplicationState_FastArray_TestClassFastArrayWithExtraProperty>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Verify initial replication
	UE_NET_ASSERT_TRUE(ClientObject!= nullptr);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Verify that we received the data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());
	UE_NET_ASSERT_EQ(ServerObject->FastArray.ExtraInt, ClientObject->FastArray.ExtraInt);

	// Add data to array
	FTestFastArrayReplicationState_FastArrayItem NewEntry;
	NewEntry.bRepBool = false;
	NewEntry.RepInt32 = 0U;
	
	ServerObject->FastArray.Edit().Add(NewEntry);

	NewEntry.bRepBool = true;
	NewEntry.RepInt32 = 1U;
	
	ServerObject->FastArray.Edit().Add(NewEntry);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we received the data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());
	UE_NET_ASSERT_EQ(ServerObject->FastArray.ExtraInt, ClientObject->FastArray.ExtraInt);

	// In this case we expect order to be the same
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);

	UE_NET_ASSERT_TRUE(ClientObject->FastArray.bHitReplicatedAdd);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedChange);
	UE_NET_ASSERT_FALSE(ClientObject->FastArray.bHitReplicatedRemove);

	// Modify extraint
	ServerObject->FastArray.ExtraInt = 3;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we received the data
	UE_NET_ASSERT_EQ(ServerObject->FastArray.ExtraInt, ClientObject->FastArray.ExtraInt);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestLostDataDirtiesPropertyBitWithDataInFlight)
{
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ServerObject = Server->CreateObject<UTestFastArrayReplicationState_FastArray_TestClassFastArray>();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Store Pointer to objects
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ClientObject = Cast<UTestFastArrayReplicationState_FastArray_TestClassFastArray>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Add entry to array
	FTestFastArrayReplicationState_FastArrayItem NewEntry;
	NewEntry.bRepBool = false;
	NewEntry.RepInt32 = 0U;
	ServerObject->FastArray.Edit().Add(NewEntry);
	NewEntry.bRepBool = false;
	NewEntry.RepInt32 = 3U;
	ServerObject->FastArray.Edit().Add(NewEntry);

	// Put data on wire, but hold off delivery
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Modify entry to trigger new update
	++ServerObject->FastArray.Edit()[0].RepInt32;

	// Put latest state on the wire
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Notify that we dropped the first packet, which should mark property dirty even if we already have the latest state in flight
	Server->DeliverTo(Client, false);

	// Notify that we delivered second packet
	Server->DeliverTo(Client, true);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we received the expected data, for both elements
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray().Num(), ClientObject->FastArray.GetItemArray().Num());
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[1].RepInt32, ClientObject->FastArray.GetItemArray()[1].RepInt32);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestNativeFastArrayWithNotReplicatedItems)
{
	IConsoleVariable* CVarToOverride = IConsoleManager::Get().FindConsoleVariable(TEXT("net.Iris.UseNativeFastArray"));
	UE_NET_ASSERT_NE(CVarToOverride, nullptr);
	UE_NET_ASSERT_TRUE(CVarToOverride->IsVariableBool());
	const bool bPrevUseNativeFastArray = CVarToOverride->GetBool();
	ON_SCOPE_EXIT
	{
		CVarToOverride->Set(bPrevUseNativeFastArray, ECVF_SetByCode);
	};

	// Use native fastarray
	CVarToOverride->Set(true, ECVF_SetByCode);

	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ServerObject = Server->CreateObject<UTestFastArrayReplicationState_FastArray_TestClassFastArray>();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Store Pointer to objects
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ClientObject = Cast<UTestFastArrayReplicationState_FastArray_TestClassFastArray>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Verify initial replication
	UE_NET_ASSERT_TRUE(ClientObject!= nullptr);

	// Add Replicated item
	{
		FTestFastArrayReplicationState_FastArrayItem ReplicatedEntry;
		ReplicatedEntry.RepInt32 = 0U;
		ServerObject->FastArray.Edit().Add(ReplicatedEntry);
	}

	// Add local explicitly not Replicated item
	{
		FTestFastArrayReplicationState_FastArrayItem ReplicatedEntry;
		ReplicatedEntry.bSkipWriteFastArrayItem = true;
		ReplicatedEntry.RepInt32 = 1000U;
		ServerObject->FastArray.Edit().AddLocal(ReplicatedEntry);
	}

	// Add Replicated item
	{
		FTestFastArrayReplicationState_FastArrayItem ReplicatedEntry;
		ReplicatedEntry.RepInt32 = 1U;
		ServerObject->FastArray.Edit().Add(ReplicatedEntry);
	}

	// Add local explicitly not Replicated item
	{
		FTestFastArrayReplicationState_FastArrayItem ReplicatedEntry;
		ReplicatedEntry.bSkipWriteFastArrayItem = true;
		ReplicatedEntry.RepInt32 = 1000U;
		ServerObject->FastArray.Edit().AddLocal(ReplicatedEntry);
	}

	{
		// We do expect an ensure if we are using the Native IrisFastArraySerializer
		FTestEnsureScope SuppressEnsureScope;

		// Send and deliver packet
		Server->UpdateAndSend({Client});
	}

	// Verify status
	UE_NET_ASSERT_EQ(ClientObject->FastArray.GetItemArray().Num(), 2);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[2].RepInt32, ClientObject->FastArray.GetItemArray()[1].RepInt32);

	// Remove not replicated item at index 1
	ServerObject->FastArray.Edit().Remove(1);
	
	// Remove replicated item at index 0
	ServerObject->FastArray.Edit().Remove(0);

	{
		// We do expect an ensure if we are using the Native IrisFastArraySerializer
		FTestEnsureScope SuppressEnsureScope;

		// Send and deliver packet
		Server->UpdateAndSend({Client});
	}

	// Verify status
	UE_NET_ASSERT_EQ(ClientObject->FastArray.GetItemArray().Num(), 1);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);

	// Remove local item (now index 1)
	ServerObject->FastArray.Edit().Remove(1);

	{
		// We do expect an ensure if we are using the Native IrisFastArraySerializer
		FTestEnsureScope SuppressEnsureScope;

		// Send and deliver packet
		Server->UpdateAndSend({Client});
	}

	// Verify status, should be same as last time
	UE_NET_ASSERT_EQ(ClientObject->FastArray.GetItemArray().Num(), 1);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);

	// Remove remaining replicated item (now index 0)
	ServerObject->FastArray.Edit().Remove(0);

	{
		// We do expect an ensure if we are using the Native IrisFastArraySerializer
		FTestEnsureScope SuppressEnsureScope;

		// Send and deliver packet
		Server->UpdateAndSend({Client});
	}

	// Verify status, should be same as last time
	UE_NET_ASSERT_EQ(ClientObject->FastArray.GetItemArray().Num(), 0);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestFastArrayWithNotReplicatedItems)
{
	IConsoleVariable* CVarToOverride = IConsoleManager::Get().FindConsoleVariable(TEXT("net.Iris.UseNativeFastArray"));
	UE_NET_ASSERT_NE(CVarToOverride, nullptr);
	UE_NET_ASSERT_TRUE(CVarToOverride->IsVariableBool());
	const bool bPrevUseNativeFastArray = CVarToOverride->GetBool();
	ON_SCOPE_EXIT
	{
		CVarToOverride->Set(bPrevUseNativeFastArray, ECVF_SetByCode);
	};

	// Use "normal" fastarray
	CVarToOverride->Set(false, ECVF_SetByCode);

	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ServerObject = Server->CreateObject<UTestFastArrayReplicationState_FastArray_TestClassFastArray>();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Store Pointer to objects
	UTestFastArrayReplicationState_FastArray_TestClassFastArray* ClientObject = Cast<UTestFastArrayReplicationState_FastArray_TestClassFastArray>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Verify initial replication
	UE_NET_ASSERT_TRUE(ClientObject!= nullptr);

	// Add Replicated item
	{
		FTestFastArrayReplicationState_FastArrayItem ReplicatedEntry;
		ReplicatedEntry.RepInt32 = 0U;
		ServerObject->FastArray.Edit().Add(ReplicatedEntry);
	}

	// Add local explicitly not Replicated item
	{
		FTestFastArrayReplicationState_FastArrayItem ReplicatedEntry;
		ReplicatedEntry.bSkipWriteFastArrayItem = true;
		ReplicatedEntry.RepInt32 = 1000U;
		ServerObject->FastArray.Edit().AddLocal(ReplicatedEntry);
	}

	// Add Replicated item
	{
		FTestFastArrayReplicationState_FastArrayItem ReplicatedEntry;
		ReplicatedEntry.RepInt32 = 1U;
		ServerObject->FastArray.Edit().Add(ReplicatedEntry);
	}

	// Add local explicitly not Replicated item
	{
		FTestFastArrayReplicationState_FastArrayItem ReplicatedEntry;
		ReplicatedEntry.bSkipWriteFastArrayItem = true;
		ReplicatedEntry.RepInt32 = 1000U;
		ServerObject->FastArray.Edit().AddLocal(ReplicatedEntry);
	}

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify status
	UE_NET_ASSERT_EQ(ClientObject->FastArray.GetItemArray().Num(), 2);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[2].RepInt32, ClientObject->FastArray.GetItemArray()[1].RepInt32);

	// Remove not replicated item at index 1
	ServerObject->FastArray.Edit().Remove(1);
	
	// Remove replicated item at index 0
	ServerObject->FastArray.Edit().Remove(0);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify status
	UE_NET_ASSERT_EQ(ClientObject->FastArray.GetItemArray().Num(), 1);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);

	// Remove local item (now index 1)
	ServerObject->FastArray.Edit().Remove(1);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify status, should be same as last time
	UE_NET_ASSERT_EQ(ClientObject->FastArray.GetItemArray().Num(), 1);
	UE_NET_ASSERT_EQ(ServerObject->FastArray.GetItemArray()[0].RepInt32, ClientObject->FastArray.GetItemArray()[0].RepInt32);

	// Remove remaining replicated item (now index 0)
	ServerObject->FastArray.Edit().Remove(0);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify status, should be same as last time
	UE_NET_ASSERT_EQ(ClientObject->FastArray.GetItemArray().Num(), 0);

}

}
