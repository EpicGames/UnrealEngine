// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkAutomationTest.h"
#include "UObject/ObjectMacros.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "Iris/ReplicationState/IrisFastArraySerializer.h"
#include "Iris/ReplicationState/Private/IrisFastArraySerializerInternal.h"
#include "TestFastArrayReplicationState.generated.h"

USTRUCT()
struct FTestFastArrayReplicationState_FastArrayItem : public FFastArraySerializerItem
{
	GENERATED_BODY()

	// Bool
	UPROPERTY()
	bool  bRepBool;

	UPROPERTY(NotReplicated)
	int32 NotRepInt32;

	// Integers
	UPROPERTY()
	int32 RepInt32;

	UPROPERTY()
	TObjectPtr<UObject> ObjectRef = nullptr;

	UPROPERTY(NotReplicated)
	bool bSkipWriteFastArrayItem = false;

	// Callbacks
	void PostReplicatedAdd(const struct FTestFastArrayReplicationState_FastArraySerializer& InArraySerializer);
	void PostReplicatedChange(const struct FTestFastArrayReplicationState_FastArraySerializer& InArraySerializer);
	void PreReplicatedRemove(const struct FTestFastArrayReplicationState_FastArraySerializer& InArraySerializer);
};

USTRUCT()
struct FTestFastArrayReplicationState_FastArraySerializer : public FIrisFastArraySerializer
{
	GENERATED_BODY()

	FTestFastArrayReplicationState_FastArraySerializer()
		: FIrisFastArraySerializer()
		, bHitReplicatedAdd(false)
		, bHitReplicatedChange(false)
		, bHitReplicatedRemove(false)
		, bHitPostReplicatedReceive(false)
		, bPostReplicatedReceiveWasHitWithUnresolvedReferences(false)
	{
	}

	// Test of experimental TIrisFastArrayEditor interface
	typedef TArray<FTestFastArrayReplicationState_FastArrayItem> ItemArrayType;
	const ItemArrayType& GetItemArray() const { return Items; }
	ItemArrayType& GetItemArray() { return Items; }

	typedef UE::Net::TIrisFastArrayEditor<FTestFastArrayReplicationState_FastArraySerializer> FFastArrayEditor;
	FFastArrayEditor Edit() { return FFastArrayEditor(*this); }	

	uint8 bHitReplicatedAdd    : 1;
	uint8 bHitReplicatedChange : 1;
	uint8 bHitReplicatedRemove : 1;
	uint8 bHitPostReplicatedReceive : 1;

	bool bPostReplicatedReceiveWasHitWithUnresolvedReferences;

	// Should really never have non replicated elements in a fastarray especially on server, but that does not mean that we should not test it.
	template<typename Type, typename SerializerType>
	bool ShouldWriteFastArrayItem(const Type& Item, const bool bIsWritingOnClient) const
	{
		if (Item.bSkipWriteFastArrayItem)
		{
			return false;
		}

		if (bIsWritingOnClient)
		{
			return Item.ReplicationID != INDEX_NONE;
		}

		return true;
	}
	
	void PostReplicatedReceive(const FFastArraySerializer::FPostReplicatedReceiveParameters& Parameters)
	{
		bHitPostReplicatedReceive = 1U;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bPostReplicatedReceiveWasHitWithUnresolvedReferences = Parameters.bHasMoreUnmappedReferences;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

protected:

	UPROPERTY()
	TArray<FTestFastArrayReplicationState_FastArrayItem> Items;
};


USTRUCT()
struct FTestFastArrayReplicationState_FastArray : public FTestFastArrayReplicationState_FastArraySerializer
{
	GENERATED_BODY()

	bool bHitDerivedPostReplicatedReceive = false;

	FTestFastArrayReplicationState_FastArray() : FTestFastArrayReplicationState_FastArraySerializer()
	{
	}

	bool NetDeltaSerialize(FNetDeltaSerializeInfo & DeltaParms)
	{
		return FFastArraySerializer::FastArrayDeltaSerialize<FTestFastArrayReplicationState_FastArrayItem, FTestFastArrayReplicationState_FastArray>(Items, DeltaParms, *this);
	}

	void PostReplicatedReceive(const FFastArraySerializer::FPostReplicatedReceiveParameters& Parameters)
	{
		Super::PostReplicatedReceive(Parameters);
		bHitDerivedPostReplicatedReceive = true;
	}
};

template<>
struct TStructOpsTypeTraits< FTestFastArrayReplicationState_FastArray > : public TStructOpsTypeTraitsBase2< FTestFastArrayReplicationState_FastArray >
{
	enum
	{
		WithNetDeltaSerializer = true,
	};
};

USTRUCT()
struct FTestFastArrayReplicationState_FastArrayWithExtraProperty : public FTestFastArrayReplicationState_FastArraySerializer
{
	GENERATED_BODY()

	FTestFastArrayReplicationState_FastArrayWithExtraProperty()
		: FTestFastArrayReplicationState_FastArraySerializer()
	{
	}

	UPROPERTY()
	int32 ExtraInt;

	bool NetDeltaSerialize(FNetDeltaSerializeInfo & DeltaParms)
	{
		return FFastArraySerializer::FastArrayDeltaSerialize<FTestFastArrayReplicationState_FastArrayItem, FTestFastArrayReplicationState_FastArrayWithExtraProperty>(Items, DeltaParms, *this);
	}

	// Note: !!This is not an example of something that should be used as we want to remove the need for users to declare custom types for FastArray-style replication
	// As we do not really support replication of FastArrays with additional replicated properties in the derived FastArray-struct we use a custom replication fragment to poll and apply the additional replicated property
	class FastArrayWithExtraPropertiesReplicationFragment;
	static UE::Net::FReplicationFragment* CreateAndRegisterReplicationFragment(UObject* Owner, const UE::Net::FReplicationStateDescriptor* Descriptor, UE::Net::FFragmentRegistrationContext& Context);
};

template<>
struct TStructOpsTypeTraits< FTestFastArrayReplicationState_FastArrayWithExtraProperty > : public TStructOpsTypeTraitsBase2< FTestFastArrayReplicationState_FastArrayWithExtraProperty >
{
	enum
	{
		WithNetDeltaSerializer = true,
	};
};

UCLASS()
class UTestFastArrayReplicationState_FastArray_TestClassFastArray : public UReplicatedTestObject
{
	GENERATED_BODY()
public:
	UTestFastArrayReplicationState_FastArray_TestClassFastArray();

	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	UPROPERTY(Replicated)
	FTestFastArrayReplicationState_FastArray FastArray;
};

UCLASS()
class UTestFastArrayReplicationState_FastArray_TestClassFastArrayWithExtraProperty : public UReplicatedTestObject
{
	GENERATED_BODY()
public:
	UTestFastArrayReplicationState_FastArray_TestClassFastArrayWithExtraProperty();

	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

public:
	UPROPERTY(Replicated)
	FTestFastArrayReplicationState_FastArrayWithExtraProperty FastArray;
};





