// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/ReplicationState/TestGeneratedReplicationState.h"
#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorImplementationMacros.h"

//////////////////////////////////////////////////////////////////////////
// THIS SECTION WILL BE GENERATED FROM UHT
//////////////////////////////////////////////////////////////////////////

constexpr UE::Net::FReplicationStateMemberChangeMaskDescriptor FIrisTestReplicationState::sReplicationStateChangeMaskDescriptors[4];

IRIS_BEGIN_SERIALIZER_DESCRIPTOR(FIrisTestReplicationState)
IRIS_SERIALIZER_DESCRIPTOR(UE::Net::FInt32NetSerializer, nullptr )
IRIS_SERIALIZER_DESCRIPTOR(UE::Net::FInt32NetSerializer, nullptr )
IRIS_SERIALIZER_DESCRIPTOR(UE::Net::FInt32NetSerializer, nullptr )
IRIS_SERIALIZER_DESCRIPTOR(UE::Net::FVectorNetSerializer, nullptr)
IRIS_END_SERIALIZER_DESCRIPTOR()

// A bit unfortunate that we need to specify serializers multiple times but due to the undefined order of static initialization we cannot rely on data from the Serializer structs at this point
IRIS_BEGIN_INTERNAL_TYPE_INFO(FIrisTestReplicationState)
IRIS_INTERNAL_TYPE_INFO(UE::Net::FInt32NetSerializer)
IRIS_INTERNAL_TYPE_INFO(UE::Net::FInt32NetSerializer)
IRIS_INTERNAL_TYPE_INFO(UE::Net::FInt32NetSerializer)
IRIS_INTERNAL_TYPE_INFO(UE::Net::FVectorNetSerializer)
IRIS_END_INTERNAL_TYPE_INFO()

IRIS_BEGIN_MEMBER_DESCRIPTOR(FIrisTestReplicationState)
IRIS_MEMBER_DESCRIPTOR(FIrisTestReplicationState, IntA, 0)
IRIS_MEMBER_DESCRIPTOR(FIrisTestReplicationState, IntB, 1)
IRIS_MEMBER_DESCRIPTOR(FIrisTestReplicationState, IntC, 2)
IRIS_MEMBER_DESCRIPTOR(FIrisTestReplicationState, Pos, 3)
IRIS_END_MEMBER_DESCRIPTOR()

IRIS_BEGIN_MEMBER_DEBUG_DESCRIPTOR(FIrisTestReplicationState)
IRIS_MEMBER_DEBUG_DESCRIPTOR(FIrisTestReplicationState, IntA)
IRIS_MEMBER_DEBUG_DESCRIPTOR(FIrisTestReplicationState, IntB)
IRIS_MEMBER_DEBUG_DESCRIPTOR(FIrisTestReplicationState, IntC)
IRIS_MEMBER_DEBUG_DESCRIPTOR(FIrisTestReplicationState, Pos)
IRIS_END_MEMBER_DEBUG_DESCRIPTOR()

IRIS_BEGIN_TRAITS_DESCRIPTOR(FIrisTestReplicationState)
IRIS_TRAITS_DESCRIPTOR(UE::Net::EReplicationStateMemberTraits::None)
IRIS_TRAITS_DESCRIPTOR(UE::Net::EReplicationStateMemberTraits::None)
IRIS_TRAITS_DESCRIPTOR(UE::Net::EReplicationStateMemberTraits::None)
IRIS_TRAITS_DESCRIPTOR(UE::Net::EReplicationStateMemberTraits::None)
IRIS_END_TRAITS_DESCRIPTOR()

IRIS_BEGIN_TAG_DESCRIPTOR(FIrisTestReplicationState)
IRIS_END_TAG_DESCRIPTOR()

IRIS_BEGIN_FUNCTION_DESCRIPTOR(FIrisTestReplicationState)
IRIS_END_FUNCTION_DESCRIPTOR()

IRIS_BEGIN_REFERENCE_DESCRIPTOR(FIrisTestReplicationState)
IRIS_END_REFERENCE_DESCRIPTOR()

IRIS_IMPLEMENT_CONSTRUCT_AND_DESTRUCT(FIrisTestReplicationState)
IRIS_IMPLEMENT_REPLICATIONSTATEDESCRIPTOR(FIrisTestReplicationState)

//////////////////////////////////////////////////////////////////////////

namespace UE::Net::Private
{

UE_NET_TEST(FTestTypedReplicationState, ValidateDescriptor)
{
	UE_NET_EXPECT_TRUE(FIrisTestReplicationState::GetReplicationStateDescriptor() != nullptr);
	
	const FReplicationStateDescriptor* Descriptor = FIrisTestReplicationState::GetReplicationStateDescriptor();

	UE_NET_ASSERT_EQ(4u, (uint32)Descriptor->MemberCount);
	UE_NET_ASSERT_EQ(0u, (uint32)Descriptor->TagCount);
	UE_NET_ASSERT_EQ(Descriptor->MemberTagDescriptors, nullptr);
	UE_NET_ASSERT_EQ(Descriptor->MemberProperties, nullptr);
	UE_NET_ASSERT_EQ(Align(40u + static_cast<uint32>(sizeof(FReplicationStateHeader)), 8U), Descriptor->ExternalSize);
	UE_NET_ASSERT_EQ(40u, Descriptor->InternalSize);
	UE_NET_ASSERT_EQ(8u, (uint32)Descriptor->ExternalAlignment);
	UE_NET_ASSERT_EQ(8u, (uint32)Descriptor->InternalAlignment);
	UE_NET_ASSERT_EQ(4u, (uint32)Descriptor->ChangeMaskBitCount);
	UE_NET_ASSERT_EQ(static_cast<uint32>(sizeof(FReplicationStateHeader)), Descriptor->ChangeMasksExternalOffset);
}

UE_NET_TEST(FTestTypedReplicationState, ValidateState)
{
	FIrisTestReplicationState State;

	UE_NET_EXPECT_TRUE(State.GetReplicationStateDescriptor() != nullptr);

	// no way to access internals yet
	// so test by casting as it always should be the first member
	const UE::Net::FReplicationStateHeader& Header = UE::Net::Private::GetReplicationStateHeader(reinterpret_cast<uint8*>(&State), State.GetReplicationStateDescriptor());

	UE_NET_ASSERT_EQ(FReplicationStateHeaderAccessor::GetNetHandleId(Header), 0ULL);
	UE_NET_ASSERT_FALSE(FReplicationStateHeaderAccessor::GetIsInitStateDirty(Header));
	UE_NET_ASSERT_FALSE(FReplicationStateHeaderAccessor::GetIsStateDirty(Header));

	// Verify defaults
	const int32 ExpectedDefaultValue = 0;
	UE_NET_ASSERT_EQ(ExpectedDefaultValue, State.GetIntA());
	UE_NET_ASSERT_EQ(ExpectedDefaultValue, State.GetIntB());
	UE_NET_ASSERT_EQ(ExpectedDefaultValue, State.GetIntC());

	UE_NET_ASSERT_FALSE(State.IsIntADirty());
	UE_NET_ASSERT_FALSE(State.IsIntBDirty());
	UE_NET_ASSERT_FALSE(State.IsIntBDirty());
	UE_NET_ASSERT_FALSE(State.IsPosDirty());

	const int32 ExpectedIntAValue = 3;
	State.SetIntA(ExpectedIntAValue);

	UE_NET_ASSERT_TRUE(State.IsIntADirty());
	UE_NET_ASSERT_FALSE(State.IsIntBDirty());
	UE_NET_ASSERT_FALSE(State.IsIntBDirty());
	UE_NET_ASSERT_FALSE(State.IsPosDirty());

	UE_NET_ASSERT_EQ(ExpectedIntAValue, State.GetIntA());

	const FVector ExpectedPosValue(FVector::OneVector);
	State.SetPos(ExpectedPosValue);

	UE_NET_ASSERT_TRUE(State.IsIntADirty());
	UE_NET_ASSERT_FALSE(State.IsIntBDirty());
	UE_NET_ASSERT_FALSE(State.IsIntBDirty());
	UE_NET_ASSERT_TRUE(State.IsPosDirty());

	UE_NET_ASSERT_EQ(ExpectedPosValue, State.GetPos());
}

}
