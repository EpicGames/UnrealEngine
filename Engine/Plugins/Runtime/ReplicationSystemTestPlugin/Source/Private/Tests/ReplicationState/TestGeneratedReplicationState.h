// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkAutomationTest.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorMacros.h"

/**
* Fake a generated ReplicationStation
* This will be generated by either UHT or some other code generation solution
* A Declaration for the state below could look something like this?
*/
#if 0
ReplicationState FakeGeneratedReplicationState
{
	[NetSerializer(FInt32NetSerializer)]
	int32 IntA;

	[NetSerializer(FInt32NetSerializer)]
	int32 IntB;

	[NetSerializer(FInt32NetSerializer)]
	int32 IntC;

	[NetSerializer(FVectorNetSerializer)]
	FVector Pos;
};
#endif

#define IRIS_GENERATED_SECTION_FOR_FIrisTestReplicationState() \
private: \
	/* state mask bits definition must be in header so that we can inline the dirty checks methods */ \
	static constexpr UE::Net::FReplicationStateMemberChangeMaskDescriptor sReplicationStateChangeMaskDescriptors[4] = { {0,1}, {1,1}, {2,1}, {3,1} }; \
	/* $IRIS: Generate from UHT Private generated members */ \
	IRIS_DECLARE_COMMON(); \
\
/* Accessors */ \
public: \
	IRIS_ACCESS_BY_VALUE(IntA, int32, 0); \
	IRIS_ACCESS_BY_VALUE(IntB, int32, 1); \
	IRIS_ACCESS_BY_VALUE(IntC, int32, 2); \
	IRIS_ACCESS_BY_REFERENCE(Pos, FVector, 3);

// Example state
class FIrisTestReplicationState
{
	IRIS_GENERATED_SECTION_FOR_FIrisTestReplicationState();
private:
	// Member List
	int32 IntA = 0;
	int32 IntB = 0;
	int32 IntC = 0;
	FVector Pos;
};
