// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestPropertyReplicationState.h"
#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationState/InternalPropertyReplicationState.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/StrongObjectPtr.h"
#include "Net/UnrealNetwork.h"
#include "Net/Core/NetHandle/NetHandleManager.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"


void UTestPropertyReplicationState_TestClass::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	DOREPLIFETIME(ThisClass, IntA);
	DOREPLIFETIME(ThisClass, IntB);
	DOREPLIFETIME(ThisClass, IntC);
}

void UTestPropertyReplicationState_TestClassWithRepNotify::OnRep_IntA(int32 OldInt)
{
}

void UTestPropertyReplicationState_TestClassWithRepNotify::OnRep_IntB(int32 OldInt)
{
}

void UTestPropertyReplicationState_TestClassWithRepNotify::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	DOREPLIFETIME_CONDITION_NOTIFY(ThisClass, IntA, COND_None, REPNOTIFY_Always);
	DOREPLIFETIME_CONDITION_NOTIFY(ThisClass, IntB, COND_None, REPNOTIFY_OnChanged);
}

void UTestPropertyReplicationState_TestClassWithInitAndCArrays::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty>& OutLifetimeProps) const
{
	DOREPLIFETIME_CONDITION(ThisClass, InitArrayOfFullyReplicatedStruct, COND_InitialOnly);
	DOREPLIFETIME_CONDITION(ThisClass, InitArrayOfNotFullyReplicatedStruct, COND_InitialOnly);
	DOREPLIFETIME_CONDITION(ThisClass, ArrayOfFullyReplicatedStruct, COND_None);
	DOREPLIFETIME_CONDITION(ThisClass, ArrayOfNotFullyReplicatedStruct, COND_None);
	DOREPLIFETIME(ThisClass, StructWithArrayOfNotFullyReplicatedStruct);
}

void UTestPropertyReplicationState_TestClassWithTArray::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty>& OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, ReferencedObjects);
	DOREPLIFETIME(ThisClass, ForceReplication);
}

void UTestPropertyReplicationState_TestClassWithTArray::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

void UTestPropertyReplicationState_TestClassWithTArray::OnRep_ReferencedObjects()
{
	bOnRepWasCalled = true;
}

void UTestPropertyReplicationState_NoRegisterFragments::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, IntA);
}

namespace UE::Net::Private
{

class FTestPropertyReplicationStateContext : public FReplicationSystemServerClientTestFixture
{
protected:
	FReplicationStateDescriptorBuilder::FResult Descriptors;
	const FReplicationStateDescriptor* Descriptor;

	void InitDescriptorsFromClass(UClass* StaticClass)
	{		
		FReplicationStateDescriptorBuilder::CreateDescriptorsForClass(Descriptors, StaticClass);

		Descriptor = Descriptors[0];

		UE_NET_ASSERT_NE(Descriptor, nullptr);
	}

	FNetHandle GenerateNetHandle()
	{
		UObject* Object = NewObject<UTestPropertyReplicationState_TestClass>();
		FNetHandle NetHandle = FNetHandleManager::GetOrCreateNetHandle(Object);
		return NetHandle;
	}
};

UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, ConstructAndDestructPropertyReplicationStateBuffer)
{
	InitDescriptorsFromClass(UTestPropertyReplicationState_TestClass::StaticClass());

	uint8 Buffer[1024];

	ConstructPropertyReplicationState(Buffer, Descriptors[0].GetReference());

	DestructPropertyReplicationState(Buffer, Descriptors[0].GetReference());
}

UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, PropertyReplicationState_Construct)
{
	InitDescriptorsFromClass(UTestPropertyReplicationState_TestClass::StaticClass());

	FPropertyReplicationState ReplicationState(Descriptor);

	UE_NET_ASSERT_TRUE(ReplicationState.IsValid());
	
	// Member it?
	for (uint32 MemberIt = 0; MemberIt < ReplicationState.GetReplicationStateDescriptor()->MemberCount; ++MemberIt)
	{
		UE_NET_ASSERT_FALSE(ReplicationState.IsDirty(MemberIt));
	}
}

UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, PropertyReplicationState_SetAndGetProperties)
{
	InitDescriptorsFromClass(UTestPropertyReplicationState_TestClass::StaticClass());

	FPropertyReplicationState ReplicationState(Descriptor);

	UE_NET_ASSERT_TRUE(ReplicationState.IsValid());

	const int IntCIndex = 2;

	const int SrcValue = 3;
	int DstValue = 0;

	UE_NET_ASSERT_NE(SrcValue, DstValue);

	UE_NET_ASSERT_FALSE(ReplicationState.IsDirty(IntCIndex));
	ReplicationState.SetPropertyValue(IntCIndex, &SrcValue);
	UE_NET_ASSERT_TRUE(ReplicationState.IsDirty(2));
	ReplicationState.GetPropertyValue(IntCIndex, &DstValue);
	
	UE_NET_ASSERT_EQ(SrcValue, DstValue);
}

// Convenience macro for tests
#define UE_NET_TEST_VERIFY_MEMBER(SrcRef, CmpRef, MemberName, ExpectDirty) \
{ \
	decltype(SrcRef.MemberName) TempCheckValue; \
	CmpRef.GetPropertyValue(MemberName##Index, &TempCheckValue); \
	UE_NET_ASSERT_EQ(ExpectDirty, CmpRef.IsDirty(MemberName##Index)); \
	UE_NET_ASSERT_TRUE(SrcRef.MemberName == TempCheckValue); \
} \

UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, PropertyReplicationState_PollPropertyReplicationState)
{
	TStrongObjectPtr<UTestPropertyReplicationState_TestClass> SrcClassPtr(NewObject<UTestPropertyReplicationState_TestClass>());
	UTestPropertyReplicationState_TestClass& SrcClass = *SrcClassPtr;

	InitDescriptorsFromClass(UTestPropertyReplicationState_TestClass::StaticClass());

	FPropertyReplicationState ReplicationState(Descriptor);
	
	// Some test data
	enum { IntAIndex = 0, IntBIndex, IntCIndex, IntArrayIndex };
	
	// Poll defaults
	ReplicationState.PollPropertyReplicationState(&SrcClass);

	// Verify
	UE_NET_TEST_VERIFY_MEMBER(SrcClass, ReplicationState, IntA, false);
	UE_NET_TEST_VERIFY_MEMBER(SrcClass, ReplicationState, IntB, false);
	UE_NET_TEST_VERIFY_MEMBER(SrcClass, ReplicationState, IntC, false);

	// Set some values in Src
	const int ExpectedValueIntC = 3;
	SrcClass.IntC = ExpectedValueIntC;

	// Poll updated Values
	ReplicationState.PollPropertyReplicationState(&SrcClass);
	
	// Verify that updated values have been copied and updated the dirty states as expected
	UE_NET_TEST_VERIFY_MEMBER(SrcClass, ReplicationState, IntA, false);
	UE_NET_TEST_VERIFY_MEMBER(SrcClass, ReplicationState, IntB, false);
	UE_NET_TEST_VERIFY_MEMBER(SrcClass, ReplicationState, IntC, true);

	int TestValueIntC = 0;
	ReplicationState.GetPropertyValue(IntCIndex, &TestValueIntC);

	UE_NET_ASSERT_EQ(ExpectedValueIntC, TestValueIntC);
}

UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, ConstructByInjectingExternalBuffer)
{
	InitDescriptorsFromClass(UTestPropertyReplicationState_TestClass::StaticClass());

	uint8 Buffer[1024];

	ConstructPropertyReplicationState(Buffer, Descriptor);

	{
		FPropertyReplicationState ReplicationState(Descriptor, Buffer);

		// Compare to default
		TStrongObjectPtr<UTestPropertyReplicationState_TestClass> SrcClassPtr(NewObject<UTestPropertyReplicationState_TestClass>());
		UTestPropertyReplicationState_TestClass& SrcClass = *SrcClassPtr;

		enum { IntAIndex = 0, IntBIndex, IntCIndex, IntArrayIndex };

		UE_NET_ASSERT_TRUE(ReplicationState.IsValid());
	
		UE_NET_TEST_VERIFY_MEMBER(SrcClass, ReplicationState, IntA, false);
		UE_NET_TEST_VERIFY_MEMBER(SrcClass, ReplicationState, IntB, false);
		UE_NET_TEST_VERIFY_MEMBER(SrcClass, ReplicationState, IntC, false);
	}

	DestructPropertyReplicationState(Buffer, Descriptor);
}
#undef UE_NET_TEST_VERIFY_MEMBER

UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, PropertyReplicationState_CopyConstruct)
{
	InitDescriptorsFromClass(UTestPropertyReplicationState_TestClass::StaticClass());

	FPropertyReplicationState ReplicationState(Descriptor);

	UE_NET_ASSERT_TRUE(ReplicationState.IsValid());

	// Set some data
	const int IntCIndex = 2;
	const int8 SrcValue = 3;

	ReplicationState.SetPropertyValue(IntCIndex, &SrcValue);

	// Copy construct state
	FPropertyReplicationState CopyConstructedState(ReplicationState);

	// Verify dirtiness
	UE_NET_ASSERT_FALSE(CopyConstructedState.IsDirty(0));
	UE_NET_ASSERT_FALSE(CopyConstructedState.IsDirty(1));
	UE_NET_ASSERT_TRUE(CopyConstructedState.IsDirty(IntCIndex));

	// Verify value
	int8 CopiedValue = 0;
	CopyConstructedState.GetPropertyValue(IntCIndex, &CopiedValue);

	UE_NET_ASSERT_EQ(CopiedValue, SrcValue);
}

UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, PropertyReplicationState_AssignToBoundStateDoesNotOverwriteInternals)
{
	InitDescriptorsFromClass(UTestPropertyReplicationState_TestClass::StaticClass());

	// Create a state and fake that it is bound
	FPropertyReplicationState BoundState(Descriptor);

	// Mark dirty
	BoundState.MarkDirty(0);
	BoundState.MarkDirty(1);

	// Fake that we are bound
	const FNetHandle NetHandle = GenerateNetHandle();
	FReplicationStateHeaderAccessor::SetNetHandleId(GetReplicationStateHeader(BoundState.GetStateBuffer(), Descriptor), NetHandle);

	// Create other state
	FPropertyReplicationState OtherState(Descriptor);

	// Set some data
	const int IntCIndex = 2;
	const int8 SrcValue = 3;

	OtherState.SetPropertyValue(IntCIndex, &SrcValue);

	// Assign to the bound state, this should result in all states being dirty
	BoundState = OtherState;

	// Verify that we did not overwrite the NetHandle
	UE_NET_ASSERT_EQ(Private::FReplicationStateHeaderAccessor::GetNetHandleId(GetReplicationStateHeader(BoundState.GetStateBuffer(), Descriptor)), NetHandle.GetId());

	// check that original states are still dirty and that we also have dirtied the value that was modified in OtherState
	UE_NET_ASSERT_TRUE(BoundState.IsDirty(0));
	UE_NET_ASSERT_TRUE(BoundState.IsDirty(1));
	UE_NET_ASSERT_TRUE(BoundState.IsDirty(2));
}

UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, PropertyReplicationState_CopyFromBoundStateDoesNotIncludeInternals)
{
	InitDescriptorsFromClass(UTestPropertyReplicationState_TestClass::StaticClass());

	// Create a state and fake that it is bound
	FPropertyReplicationState BoundState(Descriptor);

	// Mark dirty
	BoundState.MarkDirty(0);
	BoundState.MarkDirty(1);

	// Fake that we are bound
	const FNetHandle NetHandle = GenerateNetHandle();
	FReplicationStateHeaderAccessor::SetNetHandleId(GetReplicationStateHeader(BoundState.GetStateBuffer(), Descriptor), NetHandle);

	// Create other state
	FPropertyReplicationState OtherState(Descriptor);

	// Set some data
	const int IntCIndex = 2;
	const int8 SrcValue = 3;

	OtherState.SetPropertyValue(IntCIndex, &SrcValue);

	// copy the bound state, this should overwrite all local changes but not copy the internalindex
	OtherState = BoundState;

	// Verify that we did not overwrite the NetHandle
	UE_NET_ASSERT_EQ(FReplicationStateHeaderAccessor::GetNetHandleId(GetReplicationStateHeader(OtherState.GetStateBuffer(), Descriptor)), 0ULL);

	// check that original states are still dirty and that we also have dirtied the value that was modified in OtherState
	UE_NET_ASSERT_TRUE(OtherState.IsDirty(0));
	UE_NET_ASSERT_TRUE(OtherState.IsDirty(1));
	UE_NET_ASSERT_FALSE(OtherState.IsDirty(2));

	// Verify value is default
	const int8 ExpectedValue = 0;
	int8 CopiedValue = 0;	
	OtherState.GetPropertyValue(IntCIndex, &CopiedValue);

	UE_NET_ASSERT_EQ(ExpectedValue, CopiedValue);
}


UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, PropertyReplicationState_Set)
{
	InitDescriptorsFromClass(UTestPropertyReplicationState_TestClass::StaticClass());

	FPropertyReplicationState ReplicationState(Descriptor);
	FPropertyReplicationState ReplicationStateB(Descriptor);

	// Set some data
	const int IntCIndex = 2;
	const int8 SrcValue = 3;

	ReplicationState.SetPropertyValue(IntCIndex, &SrcValue);

	// Set state
	ReplicationStateB.Set(ReplicationState);

	// Verify dirtiness
	UE_NET_ASSERT_FALSE(ReplicationStateB.IsDirty(0));
	UE_NET_ASSERT_FALSE(ReplicationStateB.IsDirty(1));
	UE_NET_ASSERT_TRUE(ReplicationStateB.IsDirty(IntCIndex));

	// Verify value
	int8 CopiedValue = 0;
	ReplicationStateB.GetPropertyValue(IntCIndex, &CopiedValue);

	UE_NET_ASSERT_EQ(CopiedValue, SrcValue);
}

UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, PropertyReplicationState_RepNotify)
{
	InitDescriptorsFromClass(UTestPropertyReplicationState_TestClassWithRepNotify::StaticClass());

	UE_NET_ASSERT_TRUE(EnumHasAllFlags(Descriptor->Traits, EReplicationStateTraits::HasRepNotifies | EReplicationStateTraits::KeepPreviousState));
	UE_NET_ASSERT_TRUE(EnumHasAllFlags(Descriptor->MemberTraitsDescriptors[0].Traits, EReplicationStateMemberTraits::HasRepNotifyAlways));
	UE_NET_ASSERT_NE(Descriptor->MemberPropertyDescriptors[0].RepNotifyFunction, nullptr);

	UE_NET_ASSERT_FALSE(EnumHasAllFlags(Descriptor->MemberTraitsDescriptors[1].Traits, EReplicationStateMemberTraits::HasRepNotifyAlways));
	UE_NET_ASSERT_NE(Descriptor->MemberPropertyDescriptors[1].RepNotifyFunction, nullptr);
}

UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, PropertyReplicationState_SetInitArray)
{
	InitDescriptorsFromClass(UTestPropertyReplicationState_TestClassWithInitAndCArrays::StaticClass());

	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::InitOnly));

	FPropertyReplicationState ReplicationState(Descriptor);
	FPropertyReplicationState ReplicationStateB(Descriptor);

	// Set some data
	constexpr SIZE_T FullyReplicatedArrayElementIndex = 1;
	FTestPropertyReplicationState_FullyReplicatedStruct FullyReplicatedArrayElementValue = {};
	FullyReplicatedArrayElementValue.IntB = 1;
	ReplicationState.SetPropertyValue(FullyReplicatedArrayElementIndex, &FullyReplicatedArrayElementValue);

	constexpr SIZE_T NotFullyReplicatedArrayElementIndex = 4;
	FTestPropertyReplicationState_NotFullyReplicatedStruct NotFullyReplicatedArrayElementValue = {};
	NotFullyReplicatedArrayElementValue.IntC = 1;
	ReplicationState.SetPropertyValue(NotFullyReplicatedArrayElementIndex, &NotFullyReplicatedArrayElementValue);

	ReplicationStateB.Set(ReplicationState);
	for (uint32 MemberIt = 0, MemberEndIt = Descriptor->MemberCount; MemberIt != MemberEndIt; ++MemberIt)
	{
		// Init state doesn't have changemasks
		UE_NET_ASSERT_TRUE_MSG(ReplicationStateB.IsDirty(MemberIt), "Member '" << Descriptor->MemberProperties[MemberIt]->GetName() << "' index " << MemberIt);
	}
}

UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, PropertyReplicationState_SetDifferentValueInArray)
{
	InitDescriptorsFromClass(UTestPropertyReplicationState_TestClassWithInitAndCArrays::StaticClass());

	UE_NET_ASSERT_EQ(Descriptors.Num(), 2);

	Descriptor = Descriptors[1];
	UE_NET_ASSERT_FALSE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::InitOnly));

	FPropertyReplicationState ReplicationState(Descriptor);
	FPropertyReplicationState ReplicationStateB(Descriptor);

	// Set some data
	constexpr SIZE_T FullyReplicatedArrayElementIndex = 1;
	FTestPropertyReplicationState_FullyReplicatedStruct FullyReplicatedArrayElementValue = {};
	FullyReplicatedArrayElementValue.IntB = 1;
	ReplicationState.SetPropertyValue(FullyReplicatedArrayElementIndex, &FullyReplicatedArrayElementValue);

	constexpr SIZE_T NotFullyReplicatedArrayElementIndex = 4;
	FTestPropertyReplicationState_NotFullyReplicatedStruct NotFullyReplicatedArrayElementValue = {};
	NotFullyReplicatedArrayElementValue.IntC = 1;
	ReplicationState.SetPropertyValue(NotFullyReplicatedArrayElementIndex, &NotFullyReplicatedArrayElementValue);

	ReplicationStateB.Set(ReplicationState);
	for (uint32 MemberIt = 0, MemberEndIt = Descriptor->MemberCount; MemberIt != MemberEndIt; ++MemberIt)
	{
		if (MemberIt == FullyReplicatedArrayElementIndex || (MemberIt == NotFullyReplicatedArrayElementIndex))
		{
			UE_NET_ASSERT_TRUE_MSG(ReplicationStateB.IsDirty(MemberIt), "Member '" << Descriptor->MemberProperties[MemberIt]->GetName() << "' index " << MemberIt);
		}
		else
		{
			UE_NET_ASSERT_FALSE_MSG(ReplicationStateB.IsDirty(MemberIt), "Member '" << Descriptor->MemberProperties[MemberIt]->GetName() << "' index " << MemberIt);
		}
	}
}

UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, PropertyReplicationState_SetSameValueInArray)
{
	InitDescriptorsFromClass(UTestPropertyReplicationState_TestClassWithInitAndCArrays::StaticClass());

	UE_NET_ASSERT_EQ(Descriptors.Num(), 2);

	Descriptor = Descriptors[1];
	UE_NET_ASSERT_FALSE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::InitOnly));

	FPropertyReplicationState ReplicationState(Descriptor);
	FPropertyReplicationState ReplicationStateB(Descriptor);

	// Set some data to same value
	constexpr SIZE_T FullyReplicatedArrayElementIndex = 1;
	FTestPropertyReplicationState_FullyReplicatedStruct FullyReplicatedArrayElementValue;
	ReplicationState.GetPropertyValue(FullyReplicatedArrayElementIndex, &FullyReplicatedArrayElementValue);
	ReplicationState.SetPropertyValue(FullyReplicatedArrayElementIndex, &FullyReplicatedArrayElementValue);

	constexpr SIZE_T NotFullyReplicatedArrayElementIndex = 4;
	FTestPropertyReplicationState_NotFullyReplicatedStruct NotFullyReplicatedArrayElementValue;
	ReplicationState.GetPropertyValue(NotFullyReplicatedArrayElementIndex, &NotFullyReplicatedArrayElementValue);
	ReplicationState.SetPropertyValue(NotFullyReplicatedArrayElementIndex, &NotFullyReplicatedArrayElementValue);

	ReplicationStateB.Set(ReplicationState);
	for (uint32 MemberIt = 0, MemberEndIt = Descriptor->MemberCount; MemberIt != MemberEndIt; ++MemberIt)
	{
		UE_NET_ASSERT_FALSE_MSG(ReplicationStateB.IsDirty(MemberIt), "Member '" << Descriptor->MemberProperties[MemberIt]->GetName() << "' index " << MemberIt);
	}
}

UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, PropertyReplicationState_SetValueInArrayComplex)
{
	InitDescriptorsFromClass(UTestPropertyReplicationState_TestClassWithInitAndCArrays::StaticClass());

	UE_NET_ASSERT_EQ(Descriptors.Num(), 2);

	Descriptor = Descriptors[1];
	UE_NET_ASSERT_FALSE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::InitOnly));

	FPropertyReplicationState ReplicationState(Descriptor);
	FPropertyReplicationState ReplicationStateB(Descriptor);

	// Set some data to same value and some to a different value
	constexpr SIZE_T FullyReplicatedArrayElementIndex = 1;
	FTestPropertyReplicationState_FullyReplicatedStruct FullyReplicatedArrayElementValue0 = {};
	FullyReplicatedArrayElementValue0.IntB = 1;
	FTestPropertyReplicationState_FullyReplicatedStruct FullyReplicatedArrayElementValue1;
	ReplicationState.SetPropertyValue(FullyReplicatedArrayElementIndex, &FullyReplicatedArrayElementValue0);
	ReplicationState.GetPropertyValue(FullyReplicatedArrayElementIndex + 1, &FullyReplicatedArrayElementValue1);
	ReplicationState.SetPropertyValue(FullyReplicatedArrayElementIndex + 1, &FullyReplicatedArrayElementValue1);

	constexpr SIZE_T NotFullyReplicatedArrayElementIndex = 4;
	FTestPropertyReplicationState_NotFullyReplicatedStruct NotFullyReplicatedArrayElementValue0 = {};
	NotFullyReplicatedArrayElementValue0.IntC = 1;
	FTestPropertyReplicationState_NotFullyReplicatedStruct NotFullyReplicatedArrayElementValue1;
	ReplicationState.SetPropertyValue(NotFullyReplicatedArrayElementIndex, &NotFullyReplicatedArrayElementValue0);
	ReplicationState.GetPropertyValue(NotFullyReplicatedArrayElementIndex, &NotFullyReplicatedArrayElementValue1);
	ReplicationState.SetPropertyValue(NotFullyReplicatedArrayElementIndex, &NotFullyReplicatedArrayElementValue1);

	ReplicationStateB.Set(ReplicationState);
	for (uint32 MemberIt = 0, MemberEndIt = Descriptor->MemberCount; MemberIt != MemberEndIt; ++MemberIt)
	{
		if (MemberIt == FullyReplicatedArrayElementIndex || (MemberIt == NotFullyReplicatedArrayElementIndex))
		{
			UE_NET_ASSERT_TRUE_MSG(ReplicationStateB.IsDirty(MemberIt), "Member '" << Descriptor->MemberProperties[MemberIt]->GetName() << "' index " << MemberIt);
		}
		else
		{
			UE_NET_ASSERT_FALSE_MSG(ReplicationStateB.IsDirty(MemberIt), "Member '" << Descriptor->MemberProperties[MemberIt]->GetName() << "' index " << MemberIt);
		}
	}
}

// Test OnRep behavior on TArray when unresolved references are involved.
UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, TestArrayOnRepWithUnresolvedReferences)
{
	IConsoleVariable* CVarbApplyPreviouslyReceivedState = IConsoleManager::Get().FindConsoleVariable(TEXT("net.Iris.DispatchUnresolvedPreviouslyReceivedChanges"), false);
	UE_NET_ASSERT_NE(CVarbApplyPreviouslyReceivedState, nullptr);
	UE_NET_ASSERT_TRUE(CVarbApplyPreviouslyReceivedState->IsVariableBool());

	const bool bOldApplyPreviouslyReceivedState = CVarbApplyPreviouslyReceivedState->GetBool();
	ON_SCOPE_EXIT { CVarbApplyPreviouslyReceivedState->Set(bOldApplyPreviouslyReceivedState, ECVF_SetByCode); };

	for (const bool bApplyPreviouslyReceivedState : {false, true})
	{
		CVarbApplyPreviouslyReceivedState->Set(bApplyPreviouslyReceivedState, ECVF_SetByCode);

		// Add a client
		FReplicationSystemTestClient* Client = CreateClient();

		// Spawn objects on server
		UTestPropertyReplicationState_TestClassWithTArray* ServerObject = Server->CreateObject<UTestPropertyReplicationState_TestClassWithTArray>();

		// Spawn objects to reference
		UReplicatedTestObject* ServerReferencedObjectA = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
		UReplicatedTestObject* ServerReferencedObjectB = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
		UReplicatedTestObject* ServerReferencedObjectC = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
		UReplicatedTestObject* ServerReferencedObjectD = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});

		// Prevent objects C and D from being replicated.
		FNetObjectGroupHandle NotReplicatedGroupHandle = Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup();
		Server->ReplicationSystem->AddToGroup(NotReplicatedGroupHandle, ServerReferencedObjectC->NetRefHandle);
		Server->ReplicationSystem->AddToGroup(NotReplicatedGroupHandle, ServerReferencedObjectD->NetRefHandle);

		UTestPropertyReplicationState_TestClassWithTArray* ClientObject = nullptr;

		// Step 1. Create a fully resolvable payload and make sure it got replicated properly.
		{
			ServerObject->ReferencedObjects.Add(ServerReferencedObjectA);

			// Send and make sure everything is as expected on the client
			Server->UpdateAndSend({ Client });

			ClientObject = Cast<UTestPropertyReplicationState_TestClassWithTArray>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
			UE_NET_ASSERT_NE(ClientObject, nullptr);
			UE_NET_ASSERT_FALSE(ClientObject->ReferencedObjects.IsEmpty());
			UE_NET_ASSERT_NE(ClientObject->ReferencedObjects[0].Get(), nullptr);
		}

		// Step 2. Modify the referenced objects array such that it contains both resolvable and unresolvable references.
		{
			// Add not yet replicated objects to the mix.
			ServerObject->ReferencedObjects.Add(ServerReferencedObjectC);
			ServerObject->ReferencedObjects.Add(ServerReferencedObjectD);

			// Send and make sure everything is as expected on the client
			Server->UpdateAndSend({ Client });

			ClientObject = Cast<UTestPropertyReplicationState_TestClassWithTArray>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
			UE_NET_ASSERT_EQ(ClientObject->ReferencedObjects.Num(), 3);
			UE_NET_ASSERT_EQ(ClientObject->ReferencedObjects[0].Get(), Client->GetReplicationBridge()->GetReplicatedObject(ServerReferencedObjectA->NetRefHandle));
			UE_NET_ASSERT_EQ(ClientObject->ReferencedObjects[1].Get(), nullptr);
			UE_NET_ASSERT_EQ(ClientObject->ReferencedObjects[2].Get(), nullptr);
		}

		// Step 3. Modify non-array property on server and null out reference on client. OnRep call depending on whether we're applying previously received state with unresolved references or not.
		{
			++ServerObject->ForceReplication;
			ClientObject->bOnRepWasCalled = false;
			ClientObject->ReferencedObjects[0] = nullptr;

			// Send and make sure everything is as expected on the client
			Server->UpdateAndSend({ Client });

			UE_NET_ASSERT_EQ(ClientObject->bOnRepWasCalled, bApplyPreviouslyReceivedState);
		}

		// Step 4. Allow object C to be replicated. Expecting OnRep.
		{
			Server->ReplicationSystem->RemoveFromGroup(NotReplicatedGroupHandle, ServerReferencedObjectC->NetRefHandle);

			ClientObject->bOnRepWasCalled = false;

			Server->UpdateAndSend({ Client });

			UE_NET_ASSERT_TRUE(ClientObject->bOnRepWasCalled);
			UE_NET_ASSERT_EQ(ClientObject->ReferencedObjects[1].Get(), Client->GetReplicationBridge()->GetReplicatedObject(ServerReferencedObjectC->NetRefHandle));
		}

		// Step 5. Allow object D to be replicated. Expecting OnRep.
		{
			Server->ReplicationSystem->RemoveFromGroup(NotReplicatedGroupHandle, ServerReferencedObjectD->NetRefHandle);

			ClientObject->bOnRepWasCalled = false;

			Server->UpdateAndSend({ Client });

			UE_NET_ASSERT_TRUE(ClientObject->bOnRepWasCalled);
			UE_NET_ASSERT_EQ(ClientObject->ReferencedObjects[2].Get(), Client->GetReplicationBridge()->GetReplicatedObject(ServerReferencedObjectD->NetRefHandle));
		}

		// Step 6. Resize array on server. Expecting OnRep.
		{
			ServerObject->ReferencedObjects.SetNum(1);

			ClientObject->bOnRepWasCalled = false;

			Server->UpdateAndSend({ Client });

			UE_NET_ASSERT_TRUE(ClientObject->bOnRepWasCalled);
		}

		// Step 7. Finally modify non-array property on server and null out reference on client and make sure we don't get an OnRep call.
		{
			++ServerObject->ForceReplication;

			ClientObject->bOnRepWasCalled = false;
			ClientObject->ReferencedObjects[0] = nullptr;

			// Send and make sure everything is as expected on the client
			Server->UpdateAndSend({ Client });

			UE_NET_ASSERT_FALSE(ClientObject->bOnRepWasCalled);
		}
	}
}

// Test that automatic fragment registration works for any class that doesn't implement RegisterReplicationFragments
UE_NET_TEST_FIXTURE(FTestPropertyReplicationStateContext, TestClassDefaultRegisterFragments)
{
	using namespace UE::Net;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object
	UTestPropertyReplicationState_NoRegisterFragments* ServerObject = Server->CreateObject<UTestPropertyReplicationState_NoRegisterFragments>();

	// Create replica on client
	Server->UpdateAndSend({ Client });

	// Find the replica
	const FNetRefHandle ServerHandle = Server->GetReplicationBridge()->GetReplicatedRefHandle(ServerObject);
	UTestPropertyReplicationState_NoRegisterFragments* ClientObject = Cast<UTestPropertyReplicationState_NoRegisterFragments>(Client->GetReplicationBridge()->GetReplicatedObject(ServerHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Dirty a replicated property
	ServerObject->IntA = 0xAA;

	// Update it for the client
	Server->UpdateAndSend({ Client });

	// Make sure it was received
	UE_NET_ASSERT_EQ(ClientObject->IntA, 0xAA);
}

}
