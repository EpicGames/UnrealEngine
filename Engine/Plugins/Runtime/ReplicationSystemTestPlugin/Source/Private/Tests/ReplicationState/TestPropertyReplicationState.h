// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkAutomationTest.h"
#include "UObject/ObjectMacros.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "TestPropertyReplicationState.generated.h"

UCLASS()
class UTestPropertyReplicationState_TestClass : public UObject
{
	GENERATED_BODY()

public:
	UTestPropertyReplicationState_TestClass() : UObject() {};

	UPROPERTY(Transient, Replicated)
	int IntA;

	UPROPERTY(Transient, Replicated)
	int IntB;

	UPROPERTY(Transient, Replicated)
	int8 IntC;
};

UCLASS()
class UTestPropertyReplicationState_TestClassWithRepNotify : public UObject
{
	GENERATED_BODY()
public:
	UTestPropertyReplicationState_TestClassWithRepNotify() : UObject() {};

	UFUNCTION()
	void OnRep_IntA(int32 OldInt);

	UFUNCTION()
	void OnRep_IntB(int32 OldInt);

	UPROPERTY(Transient, ReplicatedUsing=OnRep_IntA)
	int32 IntA;

	UPROPERTY(Transient, ReplicatedUsing=OnRep_IntB)
	int32 IntB;
};


USTRUCT()
struct FTestPropertyReplicationState_FullyReplicatedStruct
{
	GENERATED_BODY()

	UPROPERTY()
	int32 IntA;

	UPROPERTY()
	int32 IntB;

	bool operator==(const FTestPropertyReplicationState_FullyReplicatedStruct& Other) const
	{
		return (IntA == Other.IntA) & (IntB == Other.IntB);
	}
};

// Set WithIdenticalViaEquality trait on FTestPropertyReplicationState_FullyReplicatedStruct to optimize the equality test 
template<>
struct TStructOpsTypeTraits<FTestPropertyReplicationState_FullyReplicatedStruct> : public TStructOpsTypeTraitsBase2<FTestPropertyReplicationState_FullyReplicatedStruct>
{
	enum
	{
		WithIdenticalViaEquality = true
	};
};

USTRUCT()
struct FTestPropertyReplicationState_NotFullyReplicatedStruct
{
	GENERATED_BODY()

	UPROPERTY()
	int32 IntA;

	UPROPERTY(NotReplicated)
	int32 IntB;

	UPROPERTY()
	int32 IntC;
};

USTRUCT()
struct FTestPropertyReplicationState_StructWithArrayOfNotFullyReplicatedStruct
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FTestPropertyReplicationState_NotFullyReplicatedStruct> DynamicArrayOfNotFullyReplicatedStruct;
};

UCLASS()
class UTestPropertyReplicationState_TestClassWithInitAndCArrays : public UObject
{
	GENERATED_BODY()

public:
	UTestPropertyReplicationState_TestClassWithInitAndCArrays() : UObject() {}

	UPROPERTY(Replicated)
	FTestPropertyReplicationState_FullyReplicatedStruct InitArrayOfFullyReplicatedStruct[3];

	UPROPERTY(Replicated)
	FTestPropertyReplicationState_NotFullyReplicatedStruct InitArrayOfNotFullyReplicatedStruct[3];

	UPROPERTY(Replicated)
	FTestPropertyReplicationState_FullyReplicatedStruct ArrayOfFullyReplicatedStruct[3];

	UPROPERTY(Replicated)
	FTestPropertyReplicationState_NotFullyReplicatedStruct ArrayOfNotFullyReplicatedStruct[3];

	UPROPERTY(Replicated)
	FTestPropertyReplicationState_StructWithArrayOfNotFullyReplicatedStruct StructWithArrayOfNotFullyReplicatedStruct;
};

UCLASS()
class UTestPropertyReplicationState_TestClassWithTArray : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UTestPropertyReplicationState_TestClassWithTArray() : UReplicatedTestObject() {}

	UPROPERTY(ReplicatedUsing=OnRep_ReferencedObjects)
	TArray<TObjectPtr<UObject>> ReferencedObjects;

	UPROPERTY(Replicated)
	uint32 ForceReplication = 0;

	bool bOnRepWasCalled = false;

protected:
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	UFUNCTION()
	void OnRep_ReferencedObjects();

};


UCLASS()
class UTestPropertyReplicationState_NoRegisterFragments : public UReplicatedTestObject
{
	GENERATED_BODY()

public:

	UPROPERTY(Transient, Replicated)
	int IntA = 0;
};