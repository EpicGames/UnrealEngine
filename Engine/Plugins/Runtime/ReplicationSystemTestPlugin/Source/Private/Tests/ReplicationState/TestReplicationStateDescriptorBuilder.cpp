// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestReplicationStateDescriptorBuilder.h"
#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/ReplicationState/InternalReplicationStateDescriptorUtils.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/ReplicationState/InternalPropertyReplicationState.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateFwd.h"
#include "Iris/Serialization/InternalNetSerializers.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/RemoteObjectReferenceNetSerializer.h"
// UnrealNetwork.h needed for DOREPLIFETIME macros.
#include "Net/UnrealNetwork.h"

namespace UE::Net::Private
{

class FTestReplicationStateDescriptorBuilderFixture : public FNetworkAutomationTestSuiteFixture
{
public:
	FTestReplicationStateDescriptorBuilderFixture() : Descriptor(nullptr) {}

protected:
	virtual void TearDown() override
	{
		Descriptor = nullptr;
		Descriptors.Reset();
	}

	void InitDescriptorsFromClass(UClass* StaticClass, bool IncludeSuper = true)
	{		
		FReplicationStateDescriptorBuilder::FParameters Params;
		Params.IncludeSuper = IncludeSuper ? 1 : 0;

		FReplicationStateDescriptorBuilder::CreateDescriptorsForClass(Descriptors, StaticClass, Params);

		Descriptor = (Descriptors.Num() > 0 ? Descriptors[0] : nullptr);
	}	

	void InitDescriptorFromStruct(UStruct* StaticStruct)
	{
		TRefCountPtr<const FReplicationStateDescriptor> StructDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(StaticStruct);
		if (StructDescriptor.IsValid())
		{
			Descriptors.Add(StructDescriptor);

			Descriptor = StructDescriptor;
		}
	}

	void InitDescriptorFromFunction(const UFunction* Function)
	{
		TRefCountPtr<const FReplicationStateDescriptor> FunctionDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForFunction(Function);
		if (FunctionDescriptor.IsValid())
		{
			Descriptors.Add(FunctionDescriptor);
			Descriptor = FunctionDescriptor;
		}
	}

	const FReplicationStateDescriptor* GetDescriptorWithTraits(EReplicationStateTraits Traits) const
	{
		for (const TRefCountPtr<const FReplicationStateDescriptor>& Desc : Descriptors)
		{
			if (EnumHasAllFlags(Desc->Traits, Traits))
			{
				return Desc.GetReference();
			}
		}

		return nullptr;
	}

	void ValidateMemberDescriptors(const FReplicationStateDescriptor* Expected, const FReplicationStateDescriptor* Value)
	{
		UE_NET_ASSERT_EQ(Expected->MemberCount, Value->MemberCount);

		for (uint32 It = 0; It < Value->MemberCount; ++It)
		{
			UE_NET_ASSERT_EQ(Expected->MemberDescriptors[It].ExternalMemberOffset, Value->MemberDescriptors[It].ExternalMemberOffset);
			UE_NET_ASSERT_EQ(Expected->MemberDescriptors[It].InternalMemberOffset, Value->MemberDescriptors[It].InternalMemberOffset);
		}
	}

	void ValidateMemberSerializerDescriptors(const FReplicationStateDescriptor* Expected, const FReplicationStateDescriptor* Value)
	{
		UE_NET_ASSERT_EQ(Expected->MemberCount, Value->MemberCount);

		for (uint32 It = 0; It < Value->MemberCount; ++It)
		{
			UE_NET_ASSERT_EQ(Expected->MemberSerializerDescriptors[It].Serializer, Value->MemberSerializerDescriptors[It].Serializer);
			//UE_NET_ASSERT_EQ(Expected->MemberSerializerDescriptors[It].SerializerConfig, Value->MemberSerializerDescriptors[It].SerializerConfig);
		}
	}

	void ValidateMemberChangeMaskDescriptors(const FReplicationStateDescriptor* Expected, const FReplicationStateDescriptor* Value)
	{
		UE_NET_ASSERT_EQ(Expected->MemberCount, Value->MemberCount);
		UE_NET_ASSERT_EQ(Expected->ChangeMaskBitCount, Value->ChangeMaskBitCount);
		UE_NET_ASSERT_EQ(Expected->ChangeMasksExternalOffset, Value->ChangeMasksExternalOffset);

		if (Expected->MemberChangeMaskDescriptors)
		{
			for (uint32 It = 0; It < Value->MemberCount; ++It)
			{
				UE_NET_ASSERT_EQ(Expected->MemberChangeMaskDescriptors[It].BitCount, Value->MemberChangeMaskDescriptors[It].BitCount);
				UE_NET_ASSERT_EQ(Expected->MemberChangeMaskDescriptors[It].BitOffset, Value->MemberChangeMaskDescriptors[It].BitOffset);
			}
		}
		else
		{
			UE_NET_ASSERT_TRUE(Value->MemberChangeMaskDescriptors == nullptr);
		}
	}

	void ValidateReplicationStateDescriptor(const FReplicationStateDescriptor* Expected, const FReplicationStateDescriptor* Value)
	{
		UE_NET_ASSERT_EQ(Expected->ExternalSize, Value->ExternalSize);
		UE_NET_ASSERT_EQ(Expected->InternalSize, Value->InternalSize);
		UE_NET_ASSERT_EQ(Expected->ExternalAlignment, Value->ExternalAlignment);
		UE_NET_ASSERT_EQ(Expected->InternalAlignment, Value->InternalAlignment);
		UE_NET_ASSERT_EQ(Expected->MemberCount, Value->MemberCount);
		UE_NET_ASSERT_EQ(Expected->ChangeMaskBitCount, Value->ChangeMaskBitCount);
		UE_NET_ASSERT_EQ(Expected->ChangeMasksExternalOffset, Value->ChangeMasksExternalOffset);
		//UE_NET_ASSERT_EQ(Expected->DescriptorIdentifier, Value->DescriptorIdentifier);

		UE_NET_ASSERT_EQ(Expected->ConstructReplicationState, Value->ConstructReplicationState);
		UE_NET_ASSERT_EQ(Expected->DestructReplicationState, Value->DestructReplicationState);

		UE_NET_ASSERT_EQ(uint64(Expected->Traits), uint64(Value->Traits));

		ValidateMemberDescriptors(Expected, Value);
		ValidateMemberSerializerDescriptors(Expected, Value);
		ValidateMemberChangeMaskDescriptors(Expected, Value);
	}

protected:
	FReplicationStateDescriptorBuilder::FResult Descriptors;
	const FReplicationStateDescriptor* Descriptor;
};

namespace TestReplicationStateDescriptor_FakeDescriptorData
{
// Fake desc for FTestReplicationStateDescriptor_TestStruct
static FReplicationStateMemberDescriptor TestStructMemberDescriptors[] = 
{
	{0, 0},
	{2, 2},
	{4, 4},
	{8, 8},
	{16, 16},
	{16, 17},
	{17, 18},
};

// Fake member serializer, config is complicated to setup
static FReplicationStateMemberSerializerDescriptor TestStructMemberSerializerDescriptors[] =
{
	{ &UE_NET_GET_SERIALIZER(FInt8NetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FInt16NetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FPackedInt32NetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FInt64NetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FBitfieldNetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FBitfieldNetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FBoolNetSerializer), nullptr }
};

static FReplicationStateMemberTraitsDescriptor TestStructMemberTraitsDescriptors[] = 
{
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
};

static const FReplicationStateDescriptor TestStructReplictionStateDescriptor =
{
	TestStructMemberDescriptors,			// MemberDescriptors
	static_cast<const FReplicationStateMemberChangeMaskDescriptor*>(nullptr),	// MemberChangeMaskDescriptors
	TestStructMemberSerializerDescriptors,	// MemberSerializerDescriptors
	TestStructMemberTraitsDescriptors, // MemberTraitsDescriptors
	static_cast<const FReplicationStateMemberFunctionDescriptor*>(nullptr),
	static_cast<const FReplicationStateMemberTagDescriptor*>(nullptr),	// MemberTagDescriptors
	static_cast<const FReplicationStateMemberReferenceDescriptor*>(nullptr),	// MemberReferenceDescriptors
	static_cast<const FProperty**>(nullptr),	// MemberProperties
	static_cast<const FReplicationStateMemberPropertyDescriptor*>(nullptr), // MemberPropertyDescriptors
	static_cast<const FReplicationStateMemberLifetimeConditionDescriptor*>(nullptr),
	static_cast<const FReplicationStateMemberRepIndexToMemberIndexDescriptor*>(nullptr),
	static_cast<const UScriptStruct*>(nullptr), // BaseStruct
	static_cast<const FNetDebugName*>(nullptr), // DebugName
	static_cast<const FReplicationStateMemberDebugDescriptor*>(nullptr), // MemberDebugDescriptors
	24,			//uint32 ExternalSize;
	24,			//uint32 InternalSize;
	8,			//uint16 ExternalAlignment;
	8,			//uint16 InternalAlignment;
	7,			//uint16 MemberCount;
	0,			//uint16 FunctionCount;
	0,			//uint16 TagCount;
	0,			//uint16 ReferenceCount
	0,			//uint16 RepIndexCount
	0,			//uint16 ChangeMaskBitCount;			// How many bits do we need for our tracking of dirty states
	0,			//uint32 ChangeMaskExternalOffset;		// This is the offset to where we store our state mask data in the external state.
	{0},			//FReplicationStateIdentifier DescriptorIdentifier;
	ConstructPropertyReplicationState,	//ConstructReplicationStateFunc CreateReplicationState;
	DestructPropertyReplicationState,	//DestructReplicationStateFunc DestroyReplicationState;
	static_cast<CreateAndRegisterReplicationFragmentFunc>(nullptr),
	EReplicationStateTraits::NeedsRefCount | EReplicationStateTraits::AllMembersAreReplicated | EReplicationStateTraits::IsSourceTriviallyDestructible | EReplicationStateTraits::SupportsDeltaCompression,  // EReplicationStateTraits Traits;
	{}, // std::atomic<int32> RefCount
};

// Fake member serializer, config is complicated to setup
static FReplicationStateMemberSerializerDescriptor TestClassMemberSerializerDescriptors[] =
{
	{ &UE_NET_GET_SERIALIZER(FInt8NetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FInt16NetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FPackedInt32NetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FInt64NetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FBitfieldNetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FBitfieldNetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FBoolNetSerializer), nullptr }
};

static FReplicationStateMemberTraitsDescriptor TestClassMemberTraitsDescriptors[] = 
{
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
};

// 4 byte header + changemask + pollmask
constexpr SIZE_T TestClassMemberDescriptors_MemberOffset = 8 + sizeof(FReplicationStateHeader);
static FReplicationStateMemberDescriptor TestClassMemberDescriptors[] = 
{
	{TestClassMemberDescriptors_MemberOffset, 0},
	{TestClassMemberDescriptors_MemberOffset + 2, 2},
	{TestClassMemberDescriptors_MemberOffset + 4, 4},
	{TestClassMemberDescriptors_MemberOffset + 8, 8},
	{TestClassMemberDescriptors_MemberOffset + 16, 16},
	{TestClassMemberDescriptors_MemberOffset + 17, 17},
	{TestClassMemberDescriptors_MemberOffset + 18, 18}
};

static FReplicationStateMemberChangeMaskDescriptor TestClassMemberChangeMaskDescriptors[] =
{
	{0,1},
	{1,1},
	{2,1},
	{3,1},
	{4,1},
	{5,1},
	{6,1}
};

// Should be the same
static const FReplicationStateDescriptor TestClassReplictionStateDescriptor
{
	TestClassMemberDescriptors,			// MemberDescriptors
	TestClassMemberChangeMaskDescriptors,	// MemberChangeMaskDescriptors
	TestClassMemberSerializerDescriptors,	// MemberSerializerDescriptors
	TestClassMemberTraitsDescriptors, // MemberTraitsDescriptors
	static_cast<const FReplicationStateMemberFunctionDescriptor*>(nullptr),
	static_cast<const FReplicationStateMemberTagDescriptor*>(nullptr), // MemberTagDescriptors
	static_cast<const FReplicationStateMemberReferenceDescriptor*>(nullptr), // MemberReferenceDescriptors
	static_cast<const FProperty**>(nullptr), // MemberProperties
	static_cast<const FReplicationStateMemberPropertyDescriptor*>(nullptr), // MemberPropertyDescriptors
	static_cast<const FReplicationStateMemberLifetimeConditionDescriptor*>(nullptr),
	static_cast<const FReplicationStateMemberRepIndexToMemberIndexDescriptor*>(nullptr),
	static_cast<const UScriptStruct*>(nullptr), // BaseStruct
	static_cast<const FNetDebugName*>(nullptr), // DebugName
	static_cast<const FReplicationStateMemberDebugDescriptor*>(nullptr), // MemberDebugDescriptors
	Align<uint32>(32U + sizeof(FReplicationStateHeader), 8U),	//uint32 ExternalSize;
	24,			//uint32 InternalSize;
	8,			//uint16 ExternalAlignment;
	8,			//uint16 InternalAlignment;
	7,			//uint16 MemberCount;
	0,			//uint16 FunctionCount;
	0,			//uint16 TagCount;
	0,			//uint16 ReferenceCount
	0,			//uint16 RepIndexCount
	7,			//uint16 ChangeMaskBitCount;			// How many bits do we need for our tracking of dirty states
	sizeof(FReplicationStateHeader), //uint32 ChangeMaskExternalOffset;		// This is the offset to where we store our state mask data in the external state.
	{0},			//FReplicationStateIdentifier DescriptorIdentifier;
	ConstructPropertyReplicationState,	//ConstructReplicationStateFunc CreateReplicationState;
	DestructPropertyReplicationState,	//DestructReplicationStateFunc DestroyReplicationState;
	static_cast<CreateAndRegisterReplicationFragmentFunc>(nullptr),
	EReplicationStateTraits::NeedsRefCount | EReplicationStateTraits::SupportsDeltaCompression, // EReplicationStateTraits Traits;		
	{}, // std::atomic<int32> RefCount
};

// Should be the same
static const FReplicationStateDescriptor TestClassReplictionStateDescriptorNotReferenceCounted
{
	TestClassMemberDescriptors,			// MemberDescriptors
	TestClassMemberChangeMaskDescriptors,	// MemberChangeMaskDescriptors
	TestClassMemberSerializerDescriptors,	// MemberSerializerDescriptors
	TestClassMemberTraitsDescriptors, // MemberTraitsDescriptors
	static_cast<const FReplicationStateMemberFunctionDescriptor*>(nullptr),
	static_cast<const FReplicationStateMemberTagDescriptor*>(nullptr), // MemberTagDescriptors
	static_cast<const FReplicationStateMemberReferenceDescriptor*>(nullptr), // MemberReferenceDescriptors
	static_cast<const FProperty**>(nullptr), // MemberProperties
	static_cast<const FReplicationStateMemberPropertyDescriptor*>(nullptr), // MemberPropertyDescriptors
	static_cast<const FReplicationStateMemberLifetimeConditionDescriptor*>(nullptr),
	static_cast<const FReplicationStateMemberRepIndexToMemberIndexDescriptor*>(nullptr),
	static_cast<const UScriptStruct*>(nullptr), // BaseStruct
	static_cast<const FNetDebugName*>(nullptr), // DebugName
	static_cast<const FReplicationStateMemberDebugDescriptor*>(nullptr), // MemberDebugDescriptors
	Align<uint32>(32U + sizeof(FReplicationStateHeader), 8U),	//uint32 ExternalSize;
	24,			//uint32 InternalSize;
	8,			//uint16 ExternalAlignment;
	8,			//uint16 InternalAlignment;
	7,			//uint16 MemberCount;
	0,			//uint16 FunctionCount;
	0,			//uint16 TagCount;
	0,			//uint16 ReferenceCount;
	0,			//uint16 RepIndexCount
	7,			//uint16 ChangeMaskBitCount;			// How many bits do we need for our tracking of dirty states
	sizeof(FReplicationStateHeader), //uint32 ChangeMaskExternalOffset;		// This is the offset to where we store our state mask data in the external state.
	{0},			//FReplicationStateIdentifier DescriptorIdentifier;
	ConstructPropertyReplicationState,	//ConstructReplicationStateFunc CreateReplicationState;
	DestructPropertyReplicationState,	//DestructReplicationStateFunc DestroyReplicationState;
	static_cast<CreateAndRegisterReplicationFragmentFunc>(nullptr),
	EReplicationStateTraits::None, // EReplicationStateTraits Traits;		
	{}, // std::atomic<int32> RefCount
};

// Should be the same
static const FReplicationStateDescriptor TestClassWithNonReplicatedDataReplicationStateDescriptor =
{
	TestClassMemberDescriptors,			// MemberDescriptors
	TestClassMemberChangeMaskDescriptors,	// MemberChangeMaskDescriptors
	TestClassMemberSerializerDescriptors,	// MemberSerializerDescriptors
	TestClassMemberTraitsDescriptors,
	static_cast<const FReplicationStateMemberFunctionDescriptor*>(nullptr),
	static_cast<const FReplicationStateMemberTagDescriptor*>(nullptr), // MemberTagDescriptors
	static_cast<const FReplicationStateMemberReferenceDescriptor*>(nullptr), // MemberReferenceDescriptors
	static_cast<const FProperty**>(nullptr), // MemberProperties
	static_cast<const FReplicationStateMemberPropertyDescriptor*>(nullptr), // MemberPropertyDescriptors
	static_cast<const FReplicationStateMemberLifetimeConditionDescriptor*>(nullptr),
	static_cast<const FReplicationStateMemberRepIndexToMemberIndexDescriptor*>(nullptr),
	static_cast<const UScriptStruct*>(nullptr), // BaseStruct
	static_cast<const FNetDebugName*>(nullptr), // DebugName
	static_cast<const FReplicationStateMemberDebugDescriptor*>(nullptr), // MemberDebugDescriptors
	Align<uint32>(32U + sizeof(FReplicationStateHeader), 8U),	//uint32 ExternalSize;
	24,			//uint32 InternalSize;
	8,			//uint16 ExternalAlignment;
	8,			//uint16 InternalAlignment;
	7,			//uint16 MemberCount;
	0,			//uint16 FunctionCount;
	0,			//uint16 TagCount;
	0,			//uint16 ReferenceCount;
	0,			//uint16 RepIndexCount
	7,			//uint16 ChangeMaskBitCount;			// How many bits do we need for our tracking of dirty states
	sizeof(FReplicationStateHeader), //uint32 ChangeMaskExternalOffset;		// This is the offset to where we store our state mask data in the external state.
	{0},			//FReplicationStateIdentifier DescriptorIdentifier;
	ConstructPropertyReplicationState,	//ConstructReplicationStateFunc CreateReplicationState;
	DestructPropertyReplicationState,	//DestructReplicationStateFunc DestroyReplicationState;
	static_cast<CreateAndRegisterReplicationFragmentFunc>(nullptr),
	EReplicationStateTraits::NeedsRefCount | EReplicationStateTraits::SupportsDeltaCompression, // EReplicationStateTraits Traits;
	{}, // std::atomic<int32> RefCount
};

}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClass)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClass::StaticClass());
	ValidateReplicationStateDescriptor(&TestReplicationStateDescriptor_FakeDescriptorData::TestClassReplictionStateDescriptor, Descriptor);
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, FTestReplicationStateDescriptor_TestStruct)
{
	InitDescriptorFromStruct(FTestReplicationStateDescriptor_TestStruct::StaticStruct());

	ValidateReplicationStateDescriptor(&TestReplicationStateDescriptor_FakeDescriptorData::TestStructReplictionStateDescriptor, Descriptor);

	FString Temp;
	Temp.Reserve(1024);
	DescribeReplicationDescriptor(Temp, Descriptor);

	UE_NET_LOG(*Temp);
}


UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithNonReplicatedData)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithNonReplicatedData::StaticClass());
	ValidateReplicationStateDescriptor(&TestReplicationStateDescriptor_FakeDescriptorData::TestClassWithNonReplicatedDataReplicationStateDescriptor, Descriptor);
}

// TestClassWithInheritance
namespace TestReplicationStateDescriptor_FakeDescriptorData
{

// 4 byte header + changemask + pollmask
constexpr SIZE_T TestClassWithInheritanceMemberDescriptors_MemberOffset = 8 + sizeof(FReplicationStateHeader);
static FReplicationStateMemberDescriptor TestClassWithInheritanceMemberDescriptors[] = 
{
	{TestClassWithInheritanceMemberDescriptors_MemberOffset, 0},
	{TestClassWithInheritanceMemberDescriptors_MemberOffset + 2, 2},
	{TestClassWithInheritanceMemberDescriptors_MemberOffset + 4, 4},
	{TestClassWithInheritanceMemberDescriptors_MemberOffset + 8, 8},
	{TestClassWithInheritanceMemberDescriptors_MemberOffset + 16, 16},
	{TestClassWithInheritanceMemberDescriptors_MemberOffset + 17, 17},
	{TestClassWithInheritanceMemberDescriptors_MemberOffset + 18, 18},
	{TestClassWithInheritanceMemberDescriptors_MemberOffset + 19, 19},
	{TestClassWithInheritanceMemberDescriptors_MemberOffset + 20, 20}
};

// Fake member serializer, config is complicated to setup
static FReplicationStateMemberSerializerDescriptor TestClassWithInheritanceMemberSerializerDescriptors[] =
{
	{ &UE_NET_GET_SERIALIZER(FInt8NetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FInt16NetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FPackedInt32NetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FInt64NetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FBitfieldNetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FBitfieldNetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FBoolNetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FUint8NetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FUint16NetSerializer), nullptr },
};

static FReplicationStateMemberTraitsDescriptor TestClassWithInheritanceMemberTraitsDescriptors[]
{
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
};

static FReplicationStateMemberChangeMaskDescriptor TestClassWithInheritanceMemberChangeMaskDescriptors[] =
{
	{0,1},
	{1,1},
	{2,1},
	{3,1},
	{4,1},
	{5,1},
	{6,1},
	{7,1},
	{8,1}
};

static FReplicationStateDescriptor TestClassWithInheritanceReplictionStateDescriptor =
{
	TestClassWithInheritanceMemberDescriptors,
	TestClassWithInheritanceMemberChangeMaskDescriptors,
	TestClassWithInheritanceMemberSerializerDescriptors,
	TestClassWithInheritanceMemberTraitsDescriptors,
	static_cast<const FReplicationStateMemberFunctionDescriptor*>(nullptr),
	static_cast<const FReplicationStateMemberTagDescriptor*>(nullptr), // MemberTagDescriptors
	static_cast<const FReplicationStateMemberReferenceDescriptor*>(nullptr), // MemberReferenceDescriptors
	static_cast<const FProperty**>(nullptr), // MemberProperties
	static_cast<const FReplicationStateMemberPropertyDescriptor*>(nullptr), // MemberPropertyDescriptors
	static_cast<const FReplicationStateMemberLifetimeConditionDescriptor*>(nullptr),
	static_cast<const FReplicationStateMemberRepIndexToMemberIndexDescriptor*>(nullptr),
	static_cast<const UScriptStruct*>(nullptr), // BaseStruct
	static_cast<const FNetDebugName*>(nullptr), // DebugName
	static_cast<const FReplicationStateMemberDebugDescriptor*>(nullptr), // MemberDebugDescriptors
	Align<uint32>(32U + sizeof(FReplicationStateHeader), 8U),	//uint32 ExternalSize;
	24,	// InternalSize
	8,	// ExternalAlignment
	8,	// InternalAlignment
	9,	// MemberCount
	0,	// FunctionCount
	0,	// TagCount;
	0,	// ReferenceCount;
	0,	// RepIndexCount
	9,	// ChangeMaskBitCount
	sizeof(FReplicationStateHeader),	// ChangeMaskExternalOffset
	{0}, // FReplicationStateIdentifier DescriptorIdentifier
	ConstructPropertyReplicationState,	// ConstructReplicationStateFunc CreateReplicationState;
	DestructPropertyReplicationState,	// DestructReplicationStateFunc DestroyReplicationState;
	static_cast<CreateAndRegisterReplicationFragmentFunc>(nullptr),
	EReplicationStateTraits::NeedsRefCount | EReplicationStateTraits::SupportsDeltaCompression,
	{}, // std::atomic<int32> RefCount
};

}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithInheritance)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithInheritance::StaticClass());
	ValidateReplicationStateDescriptor(&TestReplicationStateDescriptor_FakeDescriptorData::TestClassWithInheritanceReplictionStateDescriptor, Descriptor);
}

// TestClassWithInheritanceNoSuper
namespace TestReplicationStateDescriptor_FakeDescriptorData
{

// Fake desc for FTestReplicationStateDescriptor_TestStruct
// 4 byte header + changemask + pollmask
constexpr SIZE_T TestClassWithInheritanceNoSuperMemberDescriptors_MemberOffset = 8 + sizeof(FReplicationStateHeader);
static FReplicationStateMemberDescriptor TestClassWithInheritanceNoSuperMemberDescriptors[] = 
{
	{TestClassWithInheritanceNoSuperMemberDescriptors_MemberOffset, 0},
	{TestClassWithInheritanceNoSuperMemberDescriptors_MemberOffset + 2, 2}
};

// Fake member serializer, config is complicated to setup
static FReplicationStateMemberSerializerDescriptor TestClassWithInheritanceNoSuperMemberSerializerDescriptors[] =
{
	{ &UE_NET_GET_SERIALIZER(FUint8NetSerializer), nullptr },
	{ &UE_NET_GET_SERIALIZER(FUint16NetSerializer), nullptr },
};

static FReplicationStateMemberTraitsDescriptor TestClassWithInheritanceNoSuperMemberTraitsDescriptors[]
{
	{ EReplicationStateMemberTraits::None },
	{ EReplicationStateMemberTraits::None },
};

static FReplicationStateMemberChangeMaskDescriptor TestClassWithInheritanceNoSuperMemberChangeMaskDescriptors[] =
{
	{0,1},
	{1,1}
};

static FReplicationStateDescriptor TestClassWithInheritanceNoSuperReplictionStateDescriptor =
{
	TestClassWithInheritanceNoSuperMemberDescriptors,
	TestClassWithInheritanceNoSuperMemberChangeMaskDescriptors,
	TestClassWithInheritanceNoSuperMemberSerializerDescriptors,
	TestClassWithInheritanceNoSuperMemberTraitsDescriptors,
	static_cast<const FReplicationStateMemberFunctionDescriptor*>(nullptr),
	static_cast<const FReplicationStateMemberTagDescriptor*>(nullptr), // MemberTagDescriptors
	static_cast<const FReplicationStateMemberReferenceDescriptor*>(nullptr), // MemberReferenceDescriptors
	static_cast<const FProperty**>(nullptr), // MemberProperties
	static_cast<const FReplicationStateMemberPropertyDescriptor*>(nullptr), // MemberPropertyDescriptors
	static_cast<const FReplicationStateMemberLifetimeConditionDescriptor*>(nullptr),
	static_cast<const FReplicationStateMemberRepIndexToMemberIndexDescriptor*>(nullptr),
	static_cast<const UScriptStruct*>(nullptr), // BaseStruct
	static_cast<const FNetDebugName*>(nullptr), // DebugName
	static_cast<const FReplicationStateMemberDebugDescriptor*>(nullptr), // MemberDebugDescriptors
	Align<uint32>(16U + sizeof(FReplicationStateHeader), 8U),	//uint32 ExternalSize;
	4,	// InternalSize
	8,	// ExternalAlignment
	2,	// InternalAlignment
	2,	// MemberCount
	0,	// FunctionCount
	0,	// TagCount;
	0,	// ReferenceCount;
	0,	// RepIndexCount
	2,	// ChangeMaskBitCount
	sizeof(FReplicationStateHeader),	// ChangeMaskExternalOffset
	{0}, // FReplicationStateIdentifier DescriptorIdentifier
	ConstructPropertyReplicationState,	// ConstructReplicationStateFunc CreateReplicationState;
	DestructPropertyReplicationState,	// DestructReplicationStateFunc DestroyReplicationState;
	static_cast<CreateAndRegisterReplicationFragmentFunc>(nullptr),
	EReplicationStateTraits::NeedsRefCount | EReplicationStateTraits::SupportsDeltaCompression,
	{}, // std::atomic<int32> RefCount
};
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithInheritanceNoSuper)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithInheritance::StaticClass(), false);
	ValidateReplicationStateDescriptor(&TestReplicationStateDescriptor_FakeDescriptorData::TestClassWithInheritanceNoSuperReplictionStateDescriptor, Descriptor);
}

// TestClassWithInheritanceNoSuper
namespace TestReplicationStateDescriptor_FakeDescriptorData
{

// Fake desc for FTestReplicationStateDescriptor_TestStruct
// 4 byte header + changemask + pollmask
constexpr SIZE_T TestClassWithReplicatedStructMemberDescriptors_MemberOffset = 8 + sizeof(FReplicationStateHeader);
static FReplicationStateMemberDescriptor TestClassWithReplicatedStructMemberDescriptors[] = 
{
	{TestClassWithReplicatedStructMemberDescriptors_MemberOffset, 0},
};

// Fake member serializer, config is complicated to setup
static FReplicationStateMemberSerializerDescriptor TestClassWithReplicatedStructMemberSerializerDescriptors[] =
{
	{ &UE_NET_GET_SERIALIZER(FStructNetSerializer), nullptr },
};

static FReplicationStateMemberTraitsDescriptor TestClassWithReplicatedStructMemberTraitsDescriptors[]
{
	{ EReplicationStateMemberTraits::None },
};

static FReplicationStateMemberChangeMaskDescriptor TestClassWithReplicatedStructMemberChangeMaskDescriptors[] =
{
	{0,1},
};

static FReplicationStateDescriptor TestClassWithReplicatedStructReplictionStateDescriptor =
{
	TestClassWithReplicatedStructMemberDescriptors,
	TestClassWithReplicatedStructMemberChangeMaskDescriptors,
	TestClassWithReplicatedStructMemberSerializerDescriptors,
	TestClassWithReplicatedStructMemberTraitsDescriptors,
	static_cast<const FReplicationStateMemberFunctionDescriptor*>(nullptr), // MemberTagDescriptors
	static_cast<const FReplicationStateMemberTagDescriptor*>(nullptr), // MemberTagDescriptors
	static_cast<const FReplicationStateMemberReferenceDescriptor*>(nullptr), // MemberReferenceDescriptors
	static_cast<const FProperty**>(nullptr), // MemberProperties
	static_cast<const FReplicationStateMemberPropertyDescriptor*>(nullptr), // MemberPropertyDescriptors
	static_cast<const FReplicationStateMemberLifetimeConditionDescriptor*>(nullptr),
	static_cast<const FReplicationStateMemberRepIndexToMemberIndexDescriptor*>(nullptr),
	static_cast<const UScriptStruct*>(nullptr), // BaseStruct
	static_cast<const FNetDebugName*>(nullptr), // DebugName
	static_cast<const FReplicationStateMemberDebugDescriptor*>(nullptr), // MemberDebugDescriptors
	Align<uint32>(32U + sizeof(FReplicationStateHeader), 8U),	//uint32 ExternalSize;
	24,	// InternalSize
	8,	// ExternalAlignment
	8,	// InternalAlignment
	1,	// MemberCount
	0,	// FunctionCount
	0,	// TagCount;
	0,	// ReferenceCount;
	0,	// RepIndexCount
	1,	// ChangeMaskBitCount
	sizeof(FReplicationStateHeader),	// ChangeMaskExternalOffset
	{0}, // FReplicationStateIdentifier DescriptorIdentifier
	ConstructPropertyReplicationState,	// ConstructReplicationStateFunc CreateReplicationState;
	DestructPropertyReplicationState,	// DestructReplicationStateFunc DestroyReplicationState;
	static_cast<CreateAndRegisterReplicationFragmentFunc>(nullptr),
	EReplicationStateTraits::NeedsRefCount | EReplicationStateTraits::SupportsDeltaCompression,
	{}, // std::atomic<int32> RefCount
};

}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithReplicatedStruct)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithReplicatedStruct::StaticClass());
	ValidateReplicationStateDescriptor(&TestReplicationStateDescriptor_FakeDescriptorData::TestClassWithReplicatedStructReplictionStateDescriptor, Descriptor);
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithTArray)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithTArray::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);
	UE_NET_ASSERT_EQ(Descriptor->MemberSerializerDescriptors[0].Serializer, &UE_NET_GET_SERIALIZER(FArrayPropertyNetSerializer));
	UE_NET_ASSERT_EQ(Descriptor->MemberSerializerDescriptors[1].Serializer, &UE_NET_GET_SERIALIZER(FArrayPropertyNetSerializer));
	UE_NET_ASSERT_EQ(uint32(Descriptor->MemberTraitsDescriptors[0].Traits), uint32(EReplicationStateMemberTraits::HasDynamicState));
	UE_NET_ASSERT_EQ(uint32(Descriptor->MemberTraitsDescriptors[1].Traits), uint32(EReplicationStateMemberTraits::HasDynamicState));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithStructWithTArray)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithStructWithTArray::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);
	UE_NET_ASSERT_EQ(uint32(Descriptor->MemberTraitsDescriptors[0].Traits), uint32(EReplicationStateMemberTraits::HasDynamicState));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithCArray)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithCArray::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	// C arrays should be flattened.
	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(2*UTestReplicationStateDescriptor_TestClassWithCArray::ArrayElementCount));
	// We do however expect each array element to have separate dirty state tracking.
	UE_NET_ASSERT_EQ(Descriptor->ChangeMaskBitCount, uint16(2*UTestReplicationStateDescriptor_TestClassWithCArray::ArrayElementCount));

	const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = Descriptor->MemberPropertyDescriptors;
	// We have two arrays with the same element count. So we expect array indices to be 0 .. ArrayElementCount and then 0 .. ArrayElementCount again.
	for (SIZE_T MemberIt = 0, MemberEndIt = 2 * UTestReplicationStateDescriptor_TestClassWithCArray::ArrayElementCount; MemberIt != MemberEndIt; ++MemberIt)
	{
		const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[MemberIt];
		const uint16 ExpectedArrayIndex = (MemberIt % UTestReplicationStateDescriptor_TestClassWithCArray::ArrayElementCount);
		UE_NET_ASSERT_EQ(MemberPropertyDescriptor.ArrayIndex, ExpectedArrayIndex);
	}

	// There should be a RepIndex per CArray element. There are two arrays with ArrayElementCount elements each.
	UE_NET_ASSERT_EQ(uint32(Descriptor->RepIndexCount), 2U*UTestReplicationStateDescriptor_TestClassWithCArray::ArrayElementCount);

	// Each rep index should point to an identical member index as the class only has C arrays
	{
		const FReplicationStateMemberRepIndexToMemberIndexDescriptor* RepIndexDescriptor = Descriptor->MemberRepIndexToMemberIndexDescriptors;
		UE_NET_ASSERT_NE(RepIndexDescriptor, nullptr);
		for (uint16 RepIndexIt = 0, RepIndexEndIt = 2*UTestReplicationStateDescriptor_TestClassWithCArray::ArrayElementCount; RepIndexIt != RepIndexEndIt; ++RepIndexIt)
		{
			UE_NET_ASSERT_EQ(RepIndexDescriptor[RepIndexIt].MemberIndex, RepIndexIt);
		}
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithStructWithCArray)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithStructWithCArray::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_GE(Descriptor->MemberCount, uint16(1));
	UE_NET_ASSERT_EQ(Descriptor->ChangeMaskBitCount, uint16(1));

	const FReplicationStateMemberSerializerDescriptor* SerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	UE_NET_ASSERT_TRUE(IsUsingStructNetSerializer(SerializerDescriptors[0]));

	const FReplicationStateDescriptor* StructDescriptor = static_cast<const FStructNetSerializerConfig*>(SerializerDescriptors[0].SerializerConfig)->StateDescriptor;
	UE_NET_ASSERT_EQ(StructDescriptor->MemberCount, uint16(FTestReplicationStateDescriptor_TestStructWithCArray::ArrayElementCount + 2U));
}

// UHT doesn't support C arrays of TArrays (or containers in general) which is why there's no test for it.

// Enum tests
UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithEnums)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithEnums::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(9));

	const FReplicationStateMemberSerializerDescriptor* SerializerDescriptors = Descriptor->MemberSerializerDescriptors;

	// ints
	UE_NET_ASSERT_EQ(SerializerDescriptors[0].Serializer, &UE_NET_GET_SERIALIZER(FEnumInt8NetSerializer));
	UE_NET_ASSERT_EQ(SerializerDescriptors[1].Serializer, &UE_NET_GET_SERIALIZER(FEnumInt16NetSerializer));
	UE_NET_ASSERT_EQ(SerializerDescriptors[2].Serializer, &UE_NET_GET_SERIALIZER(FEnumInt32NetSerializer));
	UE_NET_ASSERT_EQ(SerializerDescriptors[3].Serializer, &UE_NET_GET_SERIALIZER(FEnumInt64NetSerializer));

	// EnumAsByte
	UE_NET_ASSERT_EQ(SerializerDescriptors[4].Serializer, &UE_NET_GET_SERIALIZER(FEnumUint8NetSerializer));

	// uints
	UE_NET_ASSERT_EQ(SerializerDescriptors[5].Serializer, &UE_NET_GET_SERIALIZER(FEnumUint8NetSerializer));
	UE_NET_ASSERT_EQ(SerializerDescriptors[6].Serializer, &UE_NET_GET_SERIALIZER(FEnumUint16NetSerializer));
	UE_NET_ASSERT_EQ(SerializerDescriptors[7].Serializer, &UE_NET_GET_SERIALIZER(FEnumUint32NetSerializer));
	UE_NET_ASSERT_EQ(SerializerDescriptors[8].Serializer, &UE_NET_GET_SERIALIZER(FEnumUint64NetSerializer));
}

// ENetRole tests
UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestStructWithRoleAndRemoteRole)
{
	InitDescriptorFromStruct(StaticStruct<FTestReplicationStateDescriptor_TestStructWithRoleAndRemoteRole>());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(2));

	const FReplicationStateMemberSerializerDescriptor* SerializerDescriptors = Descriptor->MemberSerializerDescriptors;

	// A struct shouldn't be using the special NetRoleNetSerializer as it shouldn't be doing role swapping
	UE_NET_ASSERT_NE(SerializerDescriptors[0].Serializer, &UE_NET_GET_SERIALIZER(FNetRoleNetSerializer));
	UE_NET_ASSERT_NE(SerializerDescriptors[1].Serializer, &UE_NET_GET_SERIALIZER(FNetRoleNetSerializer));

	UE_NET_EXPECT_EQ(SerializerDescriptors[0].Serializer, &UE_NET_GET_SERIALIZER(FEnumUint8NetSerializer));
	UE_NET_EXPECT_EQ(SerializerDescriptors[1].Serializer, &UE_NET_GET_SERIALIZER(FEnumUint8NetSerializer));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithRoleAndRemoteRole)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithRoleAndRemoteRole::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(2));

	const FReplicationStateMemberSerializerDescriptor* SerializerDescriptors = Descriptor->MemberSerializerDescriptors;

	// A class with Role and RemoteRole should be set up to support role swapping
	UE_NET_ASSERT_EQ(SerializerDescriptors[0].Serializer, &UE_NET_GET_SERIALIZER(FNetRoleNetSerializer));
	UE_NET_ASSERT_EQ(SerializerDescriptors[1].Serializer, &UE_NET_GET_SERIALIZER(FNetRoleNetSerializer));

	// NetRoles should be swapped in classes. This can only happen if the offsets are setup properly
	const FNetRoleNetSerializerConfig* FirstRoleConfig = static_cast<const FNetRoleNetSerializerConfig*>(SerializerDescriptors[0].SerializerConfig);
	const FNetRoleNetSerializerConfig* SecondRoleConfig = static_cast<const FNetRoleNetSerializerConfig*>(SerializerDescriptors[1].SerializerConfig);

	UE_NET_ASSERT_NE(FirstRoleConfig->RelativeInternalOffsetToOtherRole, 0);
	UE_NET_ASSERT_NE(FirstRoleConfig->RelativeExternalOffsetToOtherRole, 0);

	UE_NET_ASSERT_NE(SecondRoleConfig->RelativeInternalOffsetToOtherRole, 0);
	UE_NET_ASSERT_NE(SecondRoleConfig->RelativeExternalOffsetToOtherRole, 0);

	UE_NET_ASSERT_EQ_MSG(FirstRoleConfig->RelativeInternalOffsetToOtherRole, -SecondRoleConfig->RelativeInternalOffsetToOtherRole, "Roles aren't referring to each other");
	UE_NET_ASSERT_EQ_MSG(FirstRoleConfig->RelativeExternalOffsetToOtherRole, -SecondRoleConfig->RelativeExternalOffsetToOtherRole, "Roles aren't referring to each other");
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithManyRoles)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithManyRoles::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(3));

	const FReplicationStateMemberSerializerDescriptor* SerializerDescriptors = Descriptor->MemberSerializerDescriptors;

	// Only one role is named such that it should be using the NetRoleNetSerializer
	UE_NET_ASSERT_EQ(SerializerDescriptors[0].Serializer, &UE_NET_GET_SERIALIZER(FNetRoleNetSerializer));
	UE_NET_ASSERT_NE(SerializerDescriptors[1].Serializer, &UE_NET_GET_SERIALIZER(FNetRoleNetSerializer));
	UE_NET_ASSERT_NE(SerializerDescriptors[2].Serializer, &UE_NET_GET_SERIALIZER(FNetRoleNetSerializer));

	const FNetRoleNetSerializerConfig* RoleConfig = static_cast<const FNetRoleNetSerializerConfig*>(SerializerDescriptors[0].SerializerConfig);

	UE_NET_ASSERT_EQ(RoleConfig->RelativeInternalOffsetToOtherRole, 0);
	UE_NET_ASSERT_EQ(RoleConfig->RelativeExternalOffsetToOtherRole, 0);
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithRPCs)
{

	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithRPCs::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 2);

	UE_NET_ASSERT_EQ(Descriptor->FunctionCount, uint16(5));

	// Rudimentary checks
	for (const FReplicationStateMemberFunctionDescriptor& FunctionDescriptor : MakeArrayView(Descriptor->MemberFunctionDescriptors, Descriptor->FunctionCount))
	{
		UE_NET_ASSERT_NE(FunctionDescriptor.Function, nullptr);
		UE_NET_ASSERT_NE(FunctionDescriptor.Descriptor, nullptr);

		UE_NET_ASSERT_GE(FunctionDescriptor.Descriptor->ExternalSize, uint32(FunctionDescriptor.Function->ParmsSize));
		UE_NET_ASSERT_GE(FunctionDescriptor.Descriptor->ExternalAlignment, uint16(FunctionDescriptor.Function->GetMinAlignment()));
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_InheritedTestClassWithRPCs)
{

	InitDescriptorsFromClass(UTestReplicationStateDescriptor_InheritedTestClassWithRPCs::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 2);

	UE_NET_ASSERT_EQ(Descriptor->FunctionCount, uint16(6));

	// Rudimentary checks
	for (const FReplicationStateMemberFunctionDescriptor& FunctionDescriptor : MakeArrayView(Descriptor->MemberFunctionDescriptors, Descriptor->FunctionCount))
	{
		UE_NET_ASSERT_NE(FunctionDescriptor.Function, nullptr);
		UE_NET_ASSERT_NE(FunctionDescriptor.Descriptor, nullptr);

		UE_NET_ASSERT_GE(FunctionDescriptor.Descriptor->ExternalSize, uint32(FunctionDescriptor.Function->ParmsSize));
		UE_NET_ASSERT_GE(FunctionDescriptor.Descriptor->ExternalAlignment, uint16(FunctionDescriptor.Function->GetMinAlignment()));
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestFunctionWithPODParameters)
{
	const UFunction* Function = *TFieldIterator<const UFunction>(UTestReplicationStateDescriptor_TestFunctionWithPODParameters::StaticClass(), EFieldIteratorFlags::ExcludeSuper);
	InitDescriptorFromFunction(Function);

	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(3));

	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::IsSourceTriviallyDestructible));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestFunctionWithNonPODParameters)
{
	const UFunction* Function = *TFieldIterator<const UFunction>(UTestReplicationStateDescriptor_TestFunctionWithNonPODParameters::StaticClass(), EFieldIteratorFlags::ExcludeSuper);
	InitDescriptorFromFunction(Function);

	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(4));

	UE_NET_ASSERT_FALSE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::IsSourceTriviallyDestructible));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestFunctionWithNotReplicatedNonPODParameters)
{
	const UFunction* Function = *TFieldIterator<const UFunction>(UTestReplicationStateDescriptor_TestFunctionWithNotReplicatedNonPODParameters::StaticClass(), EFieldIteratorFlags::ExcludeSuper);
	InitDescriptorFromFunction(Function);

	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(3));

	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::IsSourceTriviallyDestructible));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithRef)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithRef::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->ObjectReferenceCount, uint16(1));

	for (const FReplicationStateMemberReferenceDescriptor& ReferenceDescriptor : MakeArrayView(Descriptor->MemberReferenceDescriptors, Descriptor->ObjectReferenceCount))
	{
		UE_NET_ASSERT_TRUE(ReferenceDescriptor.Info.ResolveType == FNetReferenceInfo::EResolveType::ResolveOnClient);
		UE_NET_ASSERT_EQ(ReferenceDescriptor.MemberIndex, uint16(0U));
		UE_NET_ASSERT_EQ(ReferenceDescriptor.InnerReferenceIndex, MAX_uint16);
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithRefInStruct)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithRefInStruct::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->ObjectReferenceCount, uint16(1));

	for (const FReplicationStateMemberReferenceDescriptor& ReferenceDescriptor : MakeArrayView(Descriptor->MemberReferenceDescriptors, Descriptor->ObjectReferenceCount))
	{
		UE_NET_ASSERT_TRUE(ReferenceDescriptor.Info.ResolveType == FNetReferenceInfo::EResolveType::ResolveOnClient);
		UE_NET_ASSERT_EQ(ReferenceDescriptor.MemberIndex, uint16(0U));
		UE_NET_ASSERT_EQ(ReferenceDescriptor.InnerReferenceIndex, uint16(0U));
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithRefInStructWithNestedCArray)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithRefInStructWithNestedCArray::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->ObjectReferenceCount, uint16(3));

	uint16 ExpectedInnerReferenceIndex = 0;
	for (const FReplicationStateMemberReferenceDescriptor& ReferenceDescriptor : MakeArrayView(Descriptor->MemberReferenceDescriptors, Descriptor->ObjectReferenceCount))
	{
		UE_NET_ASSERT_TRUE(ReferenceDescriptor.Info.ResolveType == FNetReferenceInfo::EResolveType::ResolveOnClient);
		UE_NET_ASSERT_EQ(ReferenceDescriptor.MemberIndex, uint16(0U));
		UE_NET_ASSERT_EQ(ReferenceDescriptor.InnerReferenceIndex, uint16(ExpectedInnerReferenceIndex));

		++ExpectedInnerReferenceIndex;
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithRefInStructWithNestedTArray)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithRefInStructWithNestedTArray::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->ObjectReferenceCount, uint16(1));

	for (const FReplicationStateMemberReferenceDescriptor& ReferenceDescriptor : MakeArrayView(Descriptor->MemberReferenceDescriptors, Descriptor->ObjectReferenceCount))
	{
		UE_NET_ASSERT_TRUE(ReferenceDescriptor.Info.ResolveType == FNetReferenceInfo::EResolveType::Invalid);
		UE_NET_ASSERT_EQ(ReferenceDescriptor.MemberIndex, uint16(0U));
		UE_NET_ASSERT_EQ(ReferenceDescriptor.InnerReferenceIndex, uint16(0));
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithRefInCArray)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithRefInCArray::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->ObjectReferenceCount, uint16(3));

	uint16 ExpectedMemberIndex = 0;
	for (const FReplicationStateMemberReferenceDescriptor& ReferenceDescriptor : MakeArrayView(Descriptor->MemberReferenceDescriptors, Descriptor->ObjectReferenceCount))
	{
		UE_NET_ASSERT_TRUE(ReferenceDescriptor.Info.ResolveType == FNetReferenceInfo::EResolveType::ResolveOnClient);
		UE_NET_ASSERT_EQ(ReferenceDescriptor.MemberIndex, ExpectedMemberIndex);
		UE_NET_ASSERT_EQ(ReferenceDescriptor.InnerReferenceIndex, MAX_uint16);

		++ExpectedMemberIndex;
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithRefInTArray)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithRefInTArray::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->ObjectReferenceCount, uint16(1));

	// Rudimentary checks
	for (const FReplicationStateMemberReferenceDescriptor& ReferenceDescriptor : MakeArrayView(Descriptor->MemberReferenceDescriptors, Descriptor->ObjectReferenceCount))
	{
		UE_NET_ASSERT_TRUE(ReferenceDescriptor.Info.ResolveType == FNetReferenceInfo::EResolveType::Invalid);
		UE_NET_ASSERT_EQ(ReferenceDescriptor.MemberIndex, uint16(0U));
		UE_NET_ASSERT_EQ(ReferenceDescriptor.InnerReferenceIndex, MAX_uint16);
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithRefInStructCArray)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithRefInStructCArray::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->ObjectReferenceCount, uint16(3));

	uint16 ExpectedMemberIndex = 0;
	for (const FReplicationStateMemberReferenceDescriptor& ReferenceDescriptor : MakeArrayView(Descriptor->MemberReferenceDescriptors, Descriptor->ObjectReferenceCount))
	{
		UE_NET_ASSERT_TRUE(ReferenceDescriptor.Info.ResolveType == FNetReferenceInfo::EResolveType::ResolveOnClient);
		UE_NET_ASSERT_EQ(ReferenceDescriptor.MemberIndex, ExpectedMemberIndex);
		UE_NET_ASSERT_EQ(ReferenceDescriptor.InnerReferenceIndex, uint16(0));

		++ExpectedMemberIndex;
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithRefInStructTArray)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithRefInStructTArray::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->ObjectReferenceCount, uint16(1));

	uint16 ExpectedMemberIndex = 0;
	for (const FReplicationStateMemberReferenceDescriptor& ReferenceDescriptor : MakeArrayView(Descriptor->MemberReferenceDescriptors, Descriptor->ObjectReferenceCount))
	{
		UE_NET_ASSERT_TRUE(ReferenceDescriptor.Info.ResolveType == FNetReferenceInfo::EResolveType::Invalid);
		UE_NET_ASSERT_EQ(ReferenceDescriptor.MemberIndex, ExpectedMemberIndex);
		UE_NET_ASSERT_EQ(ReferenceDescriptor.InnerReferenceIndex, MAX_uint16);

		++ExpectedMemberIndex;
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithCondition_InitialOnly)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithCondition_InitialOnly::StaticClass());
	UE_NET_ASSERT_GE(Descriptors.Num(), 2);

	const FReplicationStateDescriptor* InitOnlyDesc = GetDescriptorWithTraits(EReplicationStateTraits::InitOnly);
	UE_NET_ASSERT_NE(InitOnlyDesc, nullptr);
	UE_NET_ASSERT_TRUE(EnumHasAllFlags(InitOnlyDesc->Traits, EReplicationStateTraits::InitOnly));
	UE_NET_ASSERT_FALSE(EnumHasAnyFlags(InitOnlyDesc->Traits, EReplicationStateTraits::HasLifetimeConditionals));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithCondition_InitialOrOwner)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithCondition_InitialOrOwner::StaticClass());
	UE_NET_ASSERT_GE(Descriptors.Num(), 2);

	//const FReplicationStateDescriptor* InitOnlyDesc = GetDescriptorWithTraits(EReplicationStateTraits::InitOnly);
	//UE_NET_ASSERT_NE(InitOnlyDesc, nullptr);
	//UE_NET_ASSERT_TRUE(EnumHasAllFlags(InitOnlyDesc->Traits, EReplicationStateTraits::InitOnly));
	//UE_NET_ASSERT_FALSE(EnumHasAnyFlags(InitOnlyDesc->Traits, EReplicationStateTraits::HasLifetimeConditionals));

	const FReplicationStateDescriptor* LifetimeConditionalsDesc = GetDescriptorWithTraits(EReplicationStateTraits::HasLifetimeConditionals);
	UE_NET_ASSERT_NE(LifetimeConditionalsDesc, nullptr);
	UE_NET_ASSERT_TRUE(EnumHasAllFlags(LifetimeConditionalsDesc->Traits, EReplicationStateTraits::HasLifetimeConditionals));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithCondition_ToOwner)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithCondition_ToOwner::StaticClass());
	UE_NET_ASSERT_GE(Descriptors.Num(), 2);

	const FReplicationStateDescriptor* ToOwnerDesc = GetDescriptorWithTraits(EReplicationStateTraits::HasLifetimeConditionals);
	UE_NET_ASSERT_NE(ToOwnerDesc, nullptr);
	UE_NET_ASSERT_TRUE(EnumHasAllFlags(ToOwnerDesc->Traits, EReplicationStateTraits::HasLifetimeConditionals));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithCondition_SkipOwner)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithCondition_SkipOwner::StaticClass());
	UE_NET_ASSERT_GE(Descriptors.Num(), 2);

	const FReplicationStateDescriptor* SkipOwnerDesc = GetDescriptorWithTraits(EReplicationStateTraits::HasLifetimeConditionals);
	UE_NET_ASSERT_NE(SkipOwnerDesc, nullptr);
	UE_NET_ASSERT_TRUE(EnumHasAllFlags(SkipOwnerDesc->Traits, EReplicationStateTraits::HasLifetimeConditionals));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithCondition_LifetimeConditionals)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithCondition_LifetimeConditionals::StaticClass());
	UE_NET_ASSERT_GE(Descriptors.Num(), 2);

	const FReplicationStateDescriptor* LifetimeConditionalsDesc = GetDescriptorWithTraits(EReplicationStateTraits::HasLifetimeConditionals);
	UE_NET_ASSERT_NE(LifetimeConditionalsDesc, nullptr);
	UE_NET_ASSERT_TRUE(EnumHasAllFlags(LifetimeConditionalsDesc->Traits, EReplicationStateTraits::HasLifetimeConditionals));

	UE_NET_ASSERT_EQ(LifetimeConditionalsDesc->MemberCount, uint16(5));

	UE_NET_ASSERT_NE(LifetimeConditionalsDesc->MemberLifetimeConditionDescriptors, nullptr);
	for (const FReplicationStateMemberLifetimeConditionDescriptor& ConditionDesc : MakeArrayView(LifetimeConditionalsDesc->MemberLifetimeConditionDescriptors, LifetimeConditionalsDesc->MemberCount))
	{
		const SIZE_T MemberIndex = &ConditionDesc - LifetimeConditionalsDesc->MemberLifetimeConditionDescriptors;
		UE_NET_ASSERT_NE_MSG(ConditionDesc.Condition, int8(COND_None), "Unexpected lifetime condition for property " << LifetimeConditionalsDesc->MemberProperties[MemberIndex]->GetName());
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithCondition_CustomConditionals)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithCondition_CustomConditionals::StaticClass());
	UE_NET_ASSERT_GE(Descriptors.Num(), 1);

	const FReplicationStateDescriptor* CustomConditionalsDesc = GetDescriptorWithTraits(EReplicationStateTraits::HasLifetimeConditionals);
	UE_NET_ASSERT_NE(CustomConditionalsDesc, nullptr);
	UE_NET_ASSERT_TRUE(EnumHasAllFlags(CustomConditionalsDesc->Traits, EReplicationStateTraits::HasLifetimeConditionals));

	UE_NET_ASSERT_EQ(CustomConditionalsDesc->MemberCount, uint16(1));

	// The fourth member (RepIndex expected to be 3) is the property with custom condition.
	UE_NET_ASSERT_GE(CustomConditionalsDesc->RepIndexCount, uint16(4));
	UE_NET_ASSERT_NE(CustomConditionalsDesc->MemberRepIndexToMemberIndexDescriptors, nullptr);

	// Validate members with custom conditions are present in RepIndexToMemberIndex
	uint32 MemberWithCustomConditionCount = 0;
	for (const FReplicationStateMemberLifetimeConditionDescriptor& MemberLifetimeConditionDescriptor : MakeArrayView(CustomConditionalsDesc->MemberLifetimeConditionDescriptors, CustomConditionalsDesc->MemberCount))
	{
		if (MemberLifetimeConditionDescriptor.Condition != COND_Custom)
		{
			continue;
		}

		++MemberWithCustomConditionCount;

		const SIZE_T MemberIndex = &MemberLifetimeConditionDescriptor - CustomConditionalsDesc->MemberLifetimeConditionDescriptors;
		const uint32 RepIndex = CustomConditionalsDesc->MemberProperties[MemberIndex]->RepIndex;
		const FReplicationStateMemberRepIndexToMemberIndexDescriptor& RepIndexToMemberIndexDesc = CustomConditionalsDesc->MemberRepIndexToMemberIndexDescriptors[RepIndex];
		UE_NET_ASSERT_EQ((SIZE_T)RepIndexToMemberIndexDesc.MemberIndex, MemberIndex);
	}

	UE_NET_ASSERT_EQ(MemberWithCustomConditionCount, 1U);
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithCondition_Never)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithCondition_Never::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);
	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(1));
	UE_NET_ASSERT_EQ(Descriptor->MemberProperties[0]->RepIndex, uint16(0));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, FTestReplicationStateDescriptor_NotFullyReplicatedStruct)
{
	InitDescriptorFromStruct(StaticStruct<FTestReplicationStateDescriptor_NotFullyReplicatedStruct>());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);
	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(2));
	UE_NET_ASSERT_FALSE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, FTestReplicationStateDescriptor_FullyReplicatedStruct)
{
	InitDescriptorFromStruct(StaticStruct<FTestReplicationStateDescriptor_FullyReplicatedStruct>());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);
	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(3));
	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, FTestReplicationStateDescriptor_StructWithCustomSerializerWithApply)
{
	InitDescriptorFromStruct(StaticStruct<FTestReplicationStateDescriptor_StructWithCustomSerializerWithApply>());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);
	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(1));
	UE_NET_ASSERT_FALSE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, FTestReplicationStateDescriptor_StructWithArrayOfNotFullyReplicatedStruct)
{
	InitDescriptorFromStruct(StaticStruct<FTestReplicationStateDescriptor_StructWithArrayOfNotFullyReplicatedStruct>());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);
	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(1));
	UE_NET_ASSERT_FALSE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, FTestReplicationStateDescriptor_StructWithArrayOfFullyReplicatedStruct)
{
	InitDescriptorFromStruct(StaticStruct<FTestReplicationStateDescriptor_StructWithArrayOfFullyReplicatedStruct>());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);
	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(1));
	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, FTestReplicationStateDescriptor_StructWithArrayOfStructWithCustomSerializerWithApply)
{
	InitDescriptorFromStruct(StaticStruct<FTestReplicationStateDescriptor_StructWithArrayOfStructWithCustomSerializerWithApply>());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);
	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(1));
	UE_NET_ASSERT_FALSE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithNotFullyReplicatedStructAndArrays)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithNotFullyReplicatedStructAndArrays::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(2));
	UE_NET_ASSERT_FALSE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithFullyReplicatedStructAndArrays)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithFullyReplicatedStructAndArrays::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(2));

	// For the time being we do not propagate this to ReplicationStateDescriptors generated for classes
	UE_NET_ASSERT_FALSE(EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithFieldPathProperty)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithFieldPathProperty::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(1));

	const FReplicationStateMemberSerializerDescriptor* SerializerDescriptors = Descriptor->MemberSerializerDescriptors;

	UE_NET_ASSERT_EQ(SerializerDescriptors[0].Serializer, &UE_NET_GET_SERIALIZER(FFieldPathNetSerializer));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithRemoteObjectReferenceProperty)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithRemoteObjectReferenceProperty::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	UE_NET_ASSERT_EQ(Descriptor->MemberCount, uint16(1));

	const FReplicationStateMemberSerializerDescriptor* SerializerDescriptors = Descriptor->MemberSerializerDescriptors;

	UE_NET_ASSERT_EQ(SerializerDescriptors[0].Serializer, &UE_NET_GET_SERIALIZER(FRemoteObjectReferenceNetSerializer));
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, TestClassWithOnlyPushBasedPropertiesHasPushBasedDirtinessTrait)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithOnlyPushBasedProperties::StaticClass());
	UE_NET_ASSERT_GE(Descriptors.Num(), 1);

	for (const FReplicationStateDescriptor* TheDescriptor : Descriptors)
	{
		UE_NET_ASSERT_TRUE(EnumHasAnyFlags(TheDescriptor->Traits, EReplicationStateTraits::HasPushBasedDirtiness));
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, TestClassWithNeverReplicatePropertiesHasPushBasedDirtinessTrait)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithOnlyPropertiesThatNeverReplicates::StaticClass());

	// If a descriptor is created at all it is expected to have the HasPushBasedDirtiness trait 
	for (const FReplicationStateDescriptor* TheDescriptor : Descriptors)
	{
		UE_NET_ASSERT_TRUE(EnumHasAnyFlags(TheDescriptor->Traits, EReplicationStateTraits::HasPushBasedDirtiness));
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, TestClassWithPushBasedPropertiesAndNeverReplicatedPropertiesHasFullPushBasedDirtinessTrait)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithPushBasedPropertiesAndNeverReplicatedProperties::StaticClass());
	UE_NET_ASSERT_GE(Descriptors.Num(), 1);

	for (const FReplicationStateDescriptor* TheDescriptor : Descriptors)
	{
		UE_NET_ASSERT_TRUE(EnumHasAnyFlags(TheDescriptor->Traits, EReplicationStateTraits::HasPushBasedDirtiness));
		UE_NET_ASSERT_TRUE(EnumHasAnyFlags(TheDescriptor->Traits, EReplicationStateTraits::HasFullPushBasedDirtiness));
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, TestClassWithPushBasedPropertiesAndNeverReplicatedPropertiesAndFunctionsHasFullPushBasedDirtinessTrait)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithPushBasedPropertiesAndNeverReplicatedPropertiesAndFunctions::StaticClass());
	UE_NET_ASSERT_GE(Descriptors.Num(), 2);

	for (const FReplicationStateDescriptor* TheDescriptor : Descriptors)
	{
		UE_NET_ASSERT_TRUE(EnumHasAnyFlags(TheDescriptor->Traits, EReplicationStateTraits::HasPushBasedDirtiness));
		UE_NET_ASSERT_TRUE(EnumHasAnyFlags(TheDescriptor->Traits, EReplicationStateTraits::HasFullPushBasedDirtiness));
	}
}

UE_NET_TEST_FIXTURE(FTestReplicationStateDescriptorBuilderFixture, UTestReplicationStateDescriptor_TestClassWithPushBasedAndNonPushBasedPropertiesDoesNotHavePushBasedDirtinessTrait)
{
	InitDescriptorsFromClass(UTestReplicationStateDescriptor_TestClassWithPushBasedAndNonPushBasedProperties::StaticClass());
	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);

	for (const FReplicationStateDescriptor* TheDescriptor : Descriptors)
	{
		UE_NET_ASSERT_TRUE(EnumHasAnyFlags(TheDescriptor->Traits, EReplicationStateTraits::HasPushBasedDirtiness));
		UE_NET_ASSERT_FALSE(EnumHasAnyFlags(TheDescriptor->Traits, EReplicationStateTraits::HasFullPushBasedDirtiness));
	}
}

}

// Function definitions
void UTestReplicationStateDescriptor_TestClass::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	DOREPLIFETIME(ThisClass, Int8);
	DOREPLIFETIME(ThisClass, Int16);
	DOREPLIFETIME(ThisClass, Int32);
	DOREPLIFETIME(ThisClass, Int64);
	DOREPLIFETIME(ThisClass, bBitFieldBoolA);
	DOREPLIFETIME(ThisClass, bBitFieldBoolB);
	DOREPLIFETIME(ThisClass, bNativeBool);
}

void UTestReplicationStateDescriptor_TestClassWithNonReplicatedData::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	DOREPLIFETIME(ThisClass, Int8);
	DOREPLIFETIME(ThisClass, Int16);
	DOREPLIFETIME(ThisClass, Int32);
	DOREPLIFETIME(ThisClass, Int64);
	DOREPLIFETIME(ThisClass, bBitFieldBoolA);
	DOREPLIFETIME(ThisClass, bBitFieldBoolB);
	DOREPLIFETIME(ThisClass, bNativeBool);
}

void UTestReplicationStateDescriptor_TestClassWithInheritance::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(ThisClass, UInt8);
	DOREPLIFETIME(ThisClass, UInt16);
}

void UTestReplicationStateDescriptor_TestClassWithReplicatedStruct::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	DOREPLIFETIME(ThisClass, ReplicatedStruct);
}

void UTestReplicationStateDescriptor_TestClassWithTArray::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	DOREPLIFETIME(ThisClass, ArrayWithStruct);
	DOREPLIFETIME(ThisClass, ArrayWithPrimitiveType);
}

void UTestReplicationStateDescriptor_TestClassWithStructWithTArray::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	DOREPLIFETIME(ThisClass, StructWithTArray);
}

void UTestReplicationStateDescriptor_TestClassWithCArray::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, ArrayOfStruct);
	DOREPLIFETIME(ThisClass, ArrayWithPrimitiveType);
}

void UTestReplicationStateDescriptor_TestClassWithStructWithCArray::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, StructWithCArray);
}

void UTestReplicationStateDescriptor_TestClassWithEnums::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, Int8Enum);
	DOREPLIFETIME(ThisClass, Int16Enum);
	DOREPLIFETIME(ThisClass, Int32Enum);
	DOREPLIFETIME(ThisClass, Int64Enum);
	DOREPLIFETIME(ThisClass, EnumAsByteEnum);
	DOREPLIFETIME(ThisClass, Uint8Enum);
	DOREPLIFETIME(ThisClass, Uint16Enum);
	DOREPLIFETIME(ThisClass, Uint32Enum);
	DOREPLIFETIME(ThisClass, Uint64Enum);
}

void UTestReplicationStateDescriptor_TestClassWithRoleAndRemoteRole::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, Role);
	DOREPLIFETIME(ThisClass, RemoteRole);
}

void UTestReplicationStateDescriptor_TestClassWithManyRoles::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, Role);
	DOREPLIFETIME(ThisClass, OtherRole);
	DOREPLIFETIME(ThisClass, YetAnotherRole);
}

void UTestReplicationStateDescriptor_TestClassWithRPCs::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, Int);
}

void UTestReplicationStateDescriptor_TestClassWithRef::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, ObjectRef);
}

void UTestReplicationStateDescriptor_TestClassWithRefInStruct::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, StructWithRef);
}

void UTestReplicationStateDescriptor_TestClassWithRefInStructWithNestedCArray::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, StructWithRef);
}

void UTestReplicationStateDescriptor_TestClassWithRefInStructWithNestedTArray::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, StructWithRef);
}

void UTestReplicationStateDescriptor_TestClassWithRefInCArray::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, ObjectRef);
}
void UTestReplicationStateDescriptor_TestClassWithRefInTArray::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, ObjectRefInArray);
}
void UTestReplicationStateDescriptor_TestClassWithRefInStructCArray::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, StructWithRef);
}
void UTestReplicationStateDescriptor_TestClassWithRefInStructTArray::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, StructWithRef);
}

void UTestReplicationStateDescriptor_TestClassWithRPCs::UnreliableRPCOnServerWithPrimitiveArgs_Implementation(bool bBool, int InInt)
{
}

void UTestReplicationStateDescriptor_TestClassWithRPCs::UnreliableRPCOnClientWithPrimitiveArgs_Implementation(bool bBool, FVector Vector)
{
}

void UTestReplicationStateDescriptor_TestClassWithRPCs::UnreliableRPCOnServerWithComplexArgs_Implementation(const TArray<bool>& Array, bool bMessingWithAlignment, const FTestReplicationStateDescriptor_TestStructWithTArray& OutStruct)
{
}

void UTestReplicationStateDescriptor_TestClassWithRPCs::ReliableMulticastRPCWithComplexArgs_Implementation(bool bMessingWithAlignment, const FTestReplicationStateDescriptor_TestStructWithTArray& OutStruct) const
{
}

void UTestReplicationStateDescriptor_TestClassWithRPCs::UnreliableVirtualFunction_Implementation()
{
}

void UTestReplicationStateDescriptor_InheritedTestClassWithRPCs::UnreliableVirtualFunction_Implementation()
{
}

void UTestReplicationStateDescriptor_InheritedTestClassWithRPCs::AnotherRPC_Implementation()
{
}

void UTestReplicationStateDescriptor_TestFunctionWithPODParameters::FunctionWithPODParameters_Implementation(int Param0, bool Param1, int Param2)
{
}

void UTestReplicationStateDescriptor_TestFunctionWithNonPODParameters::FunctionWithNonPODParameters_Implementation(int Param0, bool Param1, int Param2, const TArray<FTestReplicationStateDescriptor_TestStructWithRefCArray>& Param3)
{
}

void UTestReplicationStateDescriptor_TestFunctionWithNotReplicatedNonPODParameters::FunctionWithNotReplicatedNonPODParameters_Implementation(int Param0, bool Param1, int Param2, const TArray<FTestReplicationStateDescriptor_TestStructWithRefCArray>& Param3)
{
}

void UTestReplicationStateDescriptor_TestClassWithCondition_InitialOnly::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, RegularInt);
	DOREPLIFETIME_CONDITION(ThisClass, InitialOnlyInt, COND_InitialOnly);
}

void UTestReplicationStateDescriptor_TestClassWithCondition_ToOwner::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, RegularInt);
	DOREPLIFETIME_CONDITION(ThisClass, ToOwnerInt, COND_OwnerOnly);
}

void UTestReplicationStateDescriptor_TestClassWithCondition_SkipOwner::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, RegularInt);
	DOREPLIFETIME_CONDITION(ThisClass, SkipOwnerInt, COND_SkipOwner);
}

void UTestReplicationStateDescriptor_TestClassWithCondition_InitialOrOwner::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, RegularInt);
	DOREPLIFETIME_CONDITION(ThisClass, InitialOrOwnerInt, COND_InitialOrOwner);
}

void UTestReplicationStateDescriptor_TestClassWithCondition_LifetimeConditionals::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, RegularInt);
	DOREPLIFETIME_CONDITION(ThisClass, SimulatedOnlyInt, COND_SimulatedOnly);
	DOREPLIFETIME_CONDITION(ThisClass, AutonomousOnlyInt, COND_AutonomousOnly);
	DOREPLIFETIME_CONDITION(ThisClass, SimulatedOrPhysicsInt, COND_SimulatedOrPhysics);
	DOREPLIFETIME_CONDITION(ThisClass, SimulatedOnlyNoReplayInt, COND_SimulatedOnlyNoReplay);
	DOREPLIFETIME_CONDITION(ThisClass, SimulatedOrPhysicsNoReplayInt, COND_SimulatedOrPhysicsNoReplay);
}

void UTestReplicationStateDescriptor_TestClassWithCondition_CustomConditionals::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, RegularInt1);
	DOREPLIFETIME(ThisClass, RegularInt2);
	DOREPLIFETIME(ThisClass, RegularInt3);
	DOREPLIFETIME_CONDITION(ThisClass, CustomConditionInt, COND_Custom);
	DOREPLIFETIME(ThisClass, RegularInt4);
}

void UTestReplicationStateDescriptor_TestClassWithCondition_Never::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, RegularInt1);
	DISABLE_REPLICATED_PROPERTY(ThisClass, NeverReplicateInt);
}

void UTestReplicationStateDescriptor_TestClassWithNotFullyReplicatedStructAndArrays::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, TArrayOfNotFullyReplicatedStruct);
	DOREPLIFETIME(ThisClass, StructWithArrayOfNotFullyReplicatedStruct);
}

void UTestReplicationStateDescriptor_TestClassWithFullyReplicatedStructAndArrays::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, TArrayOfFullyReplicatedStruct);
	DOREPLIFETIME(ThisClass, StructWithArrayOfFullyReplicatedStruct);
}

void UTestReplicationStateDescriptor_TestClassWithFieldPathProperty::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty> & OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, FieldPathProperty);
}

void UTestReplicationStateDescriptor_TestClassWithRemoteObjectReferenceProperty::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty>& OutLifetimeProps) const
{
 	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
}

void UTestReplicationStateDescriptor_TestClassWithOnlyPushBasedProperties::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams LifetimeParams;
	LifetimeParams.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntPushBasedA, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntPushBasedB, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntPushBasedC, LifetimeParams);
}

void UTestReplicationStateDescriptor_TestClassWithOnlyPropertiesThatNeverReplicates::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams LifetimeParams;
	LifetimeParams.bIsPushBased = false;
	LifetimeParams.Condition = COND_Never;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntNeverReplicatesA, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntNeverReplicatesB, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntNeverReplicatesC, LifetimeParams);
}

void UTestReplicationStateDescriptor_TestClassWithPushBasedPropertiesAndNeverReplicatedProperties::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams LifetimeParams;
	LifetimeParams.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntPushBasedA, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntPushBasedB, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntPushBasedC, LifetimeParams);


	LifetimeParams.bIsPushBased = false;
	LifetimeParams.Condition = COND_Never;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntNeverReplicatesA, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntNeverReplicatesB, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntNeverReplicatesC, LifetimeParams);
}

void UTestReplicationStateDescriptor_TestClassWithPushBasedPropertiesAndNeverReplicatedPropertiesAndFunctions::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams LifetimeParams;
	LifetimeParams.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntPushBasedA, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntPushBasedB, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntPushBasedC, LifetimeParams);


	LifetimeParams.bIsPushBased = false;
	LifetimeParams.Condition = COND_Never;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntNeverReplicatesA, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntNeverReplicatesB, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntNeverReplicatesC, LifetimeParams);
}

void UTestReplicationStateDescriptor_TestClassWithPushBasedPropertiesAndNeverReplicatedPropertiesAndFunctions::ClientUnrealiableFunction_Implementation(int32 Param)
{
}

void UTestReplicationStateDescriptor_TestClassWithPushBasedPropertiesAndNeverReplicatedPropertiesAndFunctions::ServerReliableFunction_Implementation(int32 Param)
{
}

void UTestReplicationStateDescriptor_TestClassWithPushBasedAndNonPushBasedProperties::GetLifetimeReplicatedProps(TArray<FLifetimeProperty> & OutLifetimeProps) const
{
	FDoRepLifetimeParams LifetimeParams;
	LifetimeParams.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntPushBasedA, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntPushBasedB, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntPushBasedC, LifetimeParams);


	LifetimeParams.bIsPushBased = false;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntNotPushBasedA, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntNotPushBasedB, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntNotPushBasedC, LifetimeParams);
}


namespace UE::Net
{

struct FTestReplicationStateDescriptor_StructWithCustomSerializerWithApplyNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Types
	typedef FTestReplicationStateDescriptor_StructWithCustomSerializerWithApply SourceType;
	typedef int QuantizedType;
	typedef FNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	// 
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&)
	{
	}

	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&)
	{
	}

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&)
	{
	}

	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&)
	{
	}

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&)
	{
		return true;
	}

	static void Apply(FNetSerializationContext&, const FNetApplyArgs&)
	{
	}

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;

		inline static const FName TestReplicationStateDescriptor_StructWithCustomSerializerWithApply = FName("TestReplicationStateDescriptor_StructWithCustomSerializerWithApply");
		UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(TestReplicationStateDescriptor_StructWithCustomSerializerWithApply, FTestReplicationStateDescriptor_StructWithCustomSerializerWithApplyNetSerializer);
	};

	inline static FTestReplicationStateDescriptor_StructWithCustomSerializerWithApplyNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
};
UE_NET_IMPLEMENT_SERIALIZER(FTestReplicationStateDescriptor_StructWithCustomSerializerWithApplyNetSerializer);

FTestReplicationStateDescriptor_StructWithCustomSerializerWithApplyNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(TestReplicationStateDescriptor_StructWithCustomSerializerWithApply);
}

void FTestReplicationStateDescriptor_StructWithCustomSerializerWithApplyNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(TestReplicationStateDescriptor_StructWithCustomSerializerWithApply);
}

}
