// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkAutomationTest.h"
#include "UObject/ObjectMacros.h"
#include "UObject/RemoteObjectTransfer.h"
#include "Engine/EngineTypes.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Tests/Serialization/EnumTestTypes.h"
#include "TestReplicationStateDescriptorBuilder.generated.h"


namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FTestReplicationStateDescriptor_StructWithCustomSerializerWithApplyNetSerializer, REPLICATIONSYSTEMTESTPLUGIN_API);

}

// N.B. Must use alignas(8) to get 8-byte aligned int64 on 32-bit, due to compiler options to pack structs with 4-byte alignment.

USTRUCT()
struct alignas(8) FTestReplicationStateDescriptor_TestStruct
{

	GENERATED_BODY()
	
	UPROPERTY(Transient)
	int8 Int8;					// 0

	UPROPERTY(Transient)
	int16 Int16;				// 2

	UPROPERTY(Transient)
	int Int32;					// 4

	UPROPERTY(Transient)
	int64 Int64;				// 8

	UPROPERTY(Transient)
	uint8 bBitFieldBoolA : 1;	// 16

	UPROPERTY(Transient)
	uint8 bBitFieldBoolB : 1;	// 16

	UPROPERTY(Transient)		// 17
	bool bNativeBool;
};

USTRUCT()
struct FTestReplicationStateDescriptor_TestStructWithTArray
{
	GENERATED_BODY()
	
	alignas(32) int8 MessWithAlignment;

	UPROPERTY(Transient)
	int8 Int8;						// 0

	UPROPERTY(Transient)
	TArray<uint64> Uint64Array;		// 1
};

USTRUCT()
struct FTestReplicationStateDescriptor_TestStructWithCArray
{
	enum Constants : uint32
	{
		ArrayElementCount = 3,
	};

	GENERATED_BODY()
	
	UPROPERTY(Transient)
	int8 Int8;						// 0

	UPROPERTY(Transient)
	uint64 Uint64Array[ArrayElementCount];	// 1, 2, 3

	UPROPERTY(Transient)
	uint64 Sentinel = 0x8181818181818181ULL;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClass : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClass() : UObject() {};

	UPROPERTY(Transient, Replicated)
	int8 Int8;

	UPROPERTY(Transient, Replicated)
	int16 Int16;

	UPROPERTY(Transient, Replicated)
	int Int32;

	UPROPERTY(Transient, Replicated)
	int64 Int64;

	UPROPERTY(Transient, Replicated)
	uint8 bBitFieldBoolA : 1;

	UPROPERTY(Transient, Replicated)
	uint8 bBitFieldBoolB : 1;

	UPROPERTY(Transient, Replicated)
	bool bNativeBool;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithInheritance : public UTestReplicationStateDescriptor_TestClass
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithInheritance() : UTestReplicationStateDescriptor_TestClass() {};

	UPROPERTY(Transient, Replicated)
	uint8 UInt8;

	UPROPERTY(Transient, Replicated)
	uint16 UInt16;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithNonReplicatedData : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithNonReplicatedData() : UObject() {};

	UPROPERTY(Transient, Replicated)
	int8 Int8;

	UPROPERTY(Transient, Replicated)
	int16 Int16;

	UPROPERTY(Transient, Replicated)
	int Int32;

	UPROPERTY(Transient)
	int Int32NoRep;

	UPROPERTY(Transient, Replicated)
	int64 Int64;

	UPROPERTY(Transient, Replicated)
	uint8 bBitFieldBoolA : 1;

	UPROPERTY(Transient, Replicated)
	uint8 bBitFieldBoolB : 1;

	UPROPERTY(Transient, Replicated)
	bool bNativeBool;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithReplicatedStruct : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithReplicatedStruct() : UObject() {};

	UPROPERTY(Transient, Replicated)
	FTestReplicationStateDescriptor_TestStruct ReplicatedStruct;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithTArray : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithTArray() : UObject() {}

	UPROPERTY(Transient, Replicated)
	TArray<FTestReplicationStateDescriptor_TestStruct> ArrayWithStruct;

	UPROPERTY(Transient, Replicated)
	TArray<uint64> ArrayWithPrimitiveType;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithStructWithTArray : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithStructWithTArray() : UObject() {}

	UPROPERTY(Transient, Replicated)
	FTestReplicationStateDescriptor_TestStructWithTArray StructWithTArray;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithCArray : public UObject
{
public:
	enum Constants : uint32
	{
		ArrayElementCount = 3,
	};

	UTestReplicationStateDescriptor_TestClassWithCArray() : UObject() {}

	UPROPERTY(Transient, Replicated)
	FTestReplicationStateDescriptor_TestStruct ArrayOfStruct[ArrayElementCount];

	UPROPERTY(Transient)
	uint64 Sentinel0 = 0x8181818181818181ULL;

	UPROPERTY(Transient, Replicated)
	uint64 ArrayWithPrimitiveType[ArrayElementCount];

	UPROPERTY(Transient)
	uint64 Sentinel1 = 0x8181818181818181ULL;

private:
	GENERATED_BODY()
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithStructWithCArray : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithStructWithCArray() : UObject() {}

	UPROPERTY(Transient, Replicated)
	FTestReplicationStateDescriptor_TestStructWithCArray StructWithCArray;
};

UENUM()
enum ETestEnumAsByte : int32
{
	ETestEnumAsByte_Value,
	ETestEnumAsByte_AnotherValue = 5,
	ETestEnumAsByte_YetAnotherValue = 6,
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithEnums : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithEnums() : UObject() {}

	// Re-using enums from the enum serializer tests
	UPROPERTY(Transient, Replicated)
	ETestInt8Enum Int8Enum;
	UPROPERTY(Transient, Replicated)
	ETestInt16Enum Int16Enum;
	UPROPERTY(Transient, Replicated)
	ETestInt32Enum Int32Enum;
	UPROPERTY(Transient, Replicated)
	ETestInt64Enum Int64Enum;

	UPROPERTY(Transient, Replicated)
	TEnumAsByte<ETestEnumAsByte> EnumAsByteEnum;

	UPROPERTY(Transient, Replicated)
	ETestUint8Enum Uint8Enum;
	UPROPERTY(Transient, Replicated)
	ETestUint16Enum Uint16Enum;
	UPROPERTY(Transient, Replicated)
	ETestUint32Enum Uint32Enum;
	UPROPERTY(Transient, Replicated)
	ETestUint64Enum Uint64Enum;
};

USTRUCT()
struct FTestReplicationStateDescriptor_TestStructWithRoleAndRemoteRole
{
	GENERATED_BODY();

	UPROPERTY()
	TEnumAsByte<ENetRole> Role;

	UPROPERTY()
	TEnumAsByte<ENetRole> RemoteRole;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithRoleAndRemoteRole : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithRoleAndRemoteRole() : UObject() {}

	UPROPERTY(Replicated)
	TEnumAsByte<ENetRole> Role;

	UPROPERTY(Replicated)
	TEnumAsByte<ENetRole> RemoteRole;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithManyRoles : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithManyRoles() : UObject() {}

	UPROPERTY(Replicated)
	TEnumAsByte<ENetRole> Role;

	UPROPERTY(Replicated)
	TEnumAsByte<ENetRole> OtherRole;

	UPROPERTY(Replicated)
	TEnumAsByte<ENetRole> YetAnotherRole;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithRPCs : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithRPCs() : UObject() {}

	UPROPERTY(Replicated)
	int Int;

	UFUNCTION(Unreliable, Server)
	void UnreliableRPCOnServerWithPrimitiveArgs(bool bBool, int InInt);
	void UnreliableRPCOnServerWithPrimitiveArgs_Implementation(bool bBool, int InInt);

	UFUNCTION(Unreliable, Client)
	void UnreliableRPCOnClientWithPrimitiveArgs(bool bBool, FVector Vector);
	void UnreliableRPCOnClientWithPrimitiveArgs_Implementation(bool bBool, FVector Vector);

	UFUNCTION(Unreliable, Server)
	void UnreliableRPCOnServerWithComplexArgs(const TArray<bool>& Array, bool bMessingWithAlignment, const FTestReplicationStateDescriptor_TestStructWithTArray& Struct);
	void UnreliableRPCOnServerWithComplexArgs_Implementation(const TArray<bool>& Array, bool bMessingWithAlignment, const FTestReplicationStateDescriptor_TestStructWithTArray& Struct);

	UFUNCTION(Reliable, NetMulticast)
	void ReliableMulticastRPCWithComplexArgs(bool bMessingWithAlignment, const FTestReplicationStateDescriptor_TestStructWithTArray& Struct) const;
	void ReliableMulticastRPCWithComplexArgs_Implementation(bool bMessingWithAlignment, const FTestReplicationStateDescriptor_TestStructWithTArray& Struct) const;

	UFUNCTION(Unreliable, Client)
	void UnreliableVirtualFunction();
	virtual void UnreliableVirtualFunction_Implementation();
};

UCLASS()
class UTestReplicationStateDescriptor_InheritedTestClassWithRPCs : public UTestReplicationStateDescriptor_TestClassWithRPCs
{
	GENERATED_BODY()

public:
	virtual void UnreliableVirtualFunction_Implementation() override;

	UFUNCTION(Unreliable, Client)
	void AnotherRPC();
	void AnotherRPC_Implementation();
};

USTRUCT()
struct FTestReplicationStateDescriptor_TestStructWithRef
{
	GENERATED_BODY();

	UPROPERTY()
	TObjectPtr<UObject> ObjectRef;
};

USTRUCT()
struct FTestReplicationStateDescriptor_TestStructWithRefCArray
{
	enum Constants : uint32
	{
		ArrayElementCount = 3,
	};

	GENERATED_BODY();

	UPROPERTY()
	TObjectPtr<UObject> ObjectRef[ArrayElementCount];
};

USTRUCT()
struct FTestReplicationStateDescriptor_TestStructWithRefTArray
{
	GENERATED_BODY();

	UPROPERTY()
	TArray<TObjectPtr<UObject>> ObjectRef;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithRef : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithRef() : UObject() {}

	UPROPERTY(Transient, Replicated)
	TObjectPtr<UObject> ObjectRef;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithRefInStruct : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithRefInStruct() : UObject() {}

	UPROPERTY(Transient, Replicated)
	FTestReplicationStateDescriptor_TestStructWithRef StructWithRef;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithRefInStructWithNestedCArray : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithRefInStructWithNestedCArray() : UObject() {}

	UPROPERTY(Transient, Replicated)
	FTestReplicationStateDescriptor_TestStructWithRefCArray StructWithRef;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithRefInStructWithNestedTArray : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithRefInStructWithNestedTArray() : UObject() {}

	UPROPERTY(Transient, Replicated)
	FTestReplicationStateDescriptor_TestStructWithRefTArray StructWithRef;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithRefInCArray : public UObject
{
	enum Constants : uint32
	{
		ArrayElementCount = 3,
	};

	GENERATED_BODY()
public:
	UTestReplicationStateDescriptor_TestClassWithRefInCArray() : UObject() {}

	UPROPERTY(Transient, Replicated)
	TObjectPtr<UObject> ObjectRef[ArrayElementCount];
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithRefInTArray : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithRefInTArray() : UObject() {}

	UPROPERTY(Transient, Replicated)
	TArray<TObjectPtr<UObject>> ObjectRefInArray;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithRefInStructCArray : public UObject
{
	enum Constants : uint32
	{
		ArrayElementCount = 3,
	};

	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithRefInStructCArray() : UObject() {}

	UPROPERTY(Transient, Replicated)
	FTestReplicationStateDescriptor_TestStructWithRef StructWithRef[ArrayElementCount];
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithRefInStructTArray : public UObject
{
	enum Constants : uint32
	{
		ArrayElementCount = 3,
	};

	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithRefInStructTArray() : UObject() {}

	UPROPERTY(Transient, Replicated)
	TArray<FTestReplicationStateDescriptor_TestStructWithRef> StructWithRef;
};

UCLASS()
class UTestReplicationStateDescriptor_TestFunctionWithPODParameters : public UObject
{
	GENERATED_BODY()

protected:
	UFUNCTION(Client, unreliable)
	void FunctionWithPODParameters(int Param0, bool Param1, int Param2);
	void FunctionWithPODParameters_Implementation(int Param0, bool Param1, int Param2);
};

UCLASS()
class UTestReplicationStateDescriptor_TestFunctionWithNonPODParameters : public UObject
{
	GENERATED_BODY()

protected:
	UFUNCTION(Client, unreliable)
	void FunctionWithNonPODParameters(int Param0, bool Param1, int Param2, const TArray<FTestReplicationStateDescriptor_TestStructWithRefCArray>& Param3);
	void FunctionWithNonPODParameters_Implementation(int Param0, bool Param1, int Param2, const TArray<FTestReplicationStateDescriptor_TestStructWithRefCArray>& Param3);
};

UCLASS()
class UTestReplicationStateDescriptor_TestFunctionWithNotReplicatedNonPODParameters : public UObject
{
	GENERATED_BODY()

protected:
	// Currently some features such as not replicating all parameters isn't allowed on regular RPCs
	UFUNCTION(ServiceRequest(Iris))
	void FunctionWithNotReplicatedNonPODParameters(int Param0, bool Param1, int Param2, UPARAM(NotReplicated) const TArray<FTestReplicationStateDescriptor_TestStructWithRefCArray>& NotReplicatedParam3);
	void FunctionWithNotReplicatedNonPODParameters_Implementation(int Param0, bool Param1, int Param2, UPARAM(NotReplicated) const TArray<FTestReplicationStateDescriptor_TestStructWithRefCArray>& NotReplicatedParam3);
};

// Conditions and filtering
UCLASS()
class UTestReplicationStateDescriptor_TestClassWithCondition_InitialOnly : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithCondition_InitialOnly() : UObject() {}

	UPROPERTY(Replicated, Transient)
	int32 RegularInt;

	UPROPERTY(Replicated, Transient)
	int32 InitialOnlyInt;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithCondition_ToOwner : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithCondition_ToOwner() : UObject() {}

	UPROPERTY(Replicated, Transient)
	int32 RegularInt;

	UPROPERTY(Replicated, Transient)
	int32 ToOwnerInt;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithCondition_SkipOwner : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithCondition_SkipOwner() : UObject() {}

	UPROPERTY(Replicated, Transient)
	int32 RegularInt;

	UPROPERTY(Replicated, Transient)
	int32 SkipOwnerInt;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithCondition_InitialOrOwner : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithCondition_InitialOrOwner() : UObject() {}

	UPROPERTY(Replicated, Transient)
	int32 RegularInt;

	UPROPERTY(Replicated, Transient)
	int32 InitialOrOwnerInt;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithCondition_LifetimeConditionals : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithCondition_LifetimeConditionals() : UObject() {}

	UPROPERTY(Replicated, Transient)
	int32 RegularInt;

	UPROPERTY(Replicated, Transient)
	int32 SimulatedOnlyInt;

	UPROPERTY(Replicated, Transient)
	int32 AutonomousOnlyInt;

	UPROPERTY(Replicated, Transient)
	int32 SimulatedOrPhysicsInt;

	UPROPERTY(Replicated, Transient)
	int32 SimulatedOnlyNoReplayInt;

	UPROPERTY(Replicated, Transient)
	int32 SimulatedOrPhysicsNoReplayInt;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithCondition_CustomConditionals : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithCondition_CustomConditionals() : UObject() {}

	UPROPERTY(Replicated, Transient)
	int32 RegularInt1;

	UPROPERTY(Replicated, Transient)
	int32 RegularInt2;

	UPROPERTY(Replicated, Transient)
	int32 RegularInt3;

	UPROPERTY(Replicated, Transient)
	int32 CustomConditionInt;

	UPROPERTY(Replicated, Transient)
	int32 RegularInt4;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithCondition_Never : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithCondition_Never() : UObject() {}

	UPROPERTY(Replicated, Transient)
	int32 RegularInt1;

	UPROPERTY(Replicated, Transient)
	int32 NeverReplicateInt;
};

USTRUCT()
struct FTestReplicationStateDescriptor_NotFullyReplicatedStruct
{
	GENERATED_BODY()

	UPROPERTY()
	int32 IntA;

	UPROPERTY(NotReplicated)
	int32 IntB;

	UPROPERTY()
	int32 IntC;
};

USTRUCT()
struct FTestReplicationStateDescriptor_FullyReplicatedStruct
{
	GENERATED_BODY()

	UPROPERTY()
	int32 IntA;

	UPROPERTY()
	int32 IntB;

	UPROPERTY()
	int32 IntC;
};

USTRUCT()
struct FTestReplicationStateDescriptor_StructWithArrayOfNotFullyReplicatedStruct
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FTestReplicationStateDescriptor_NotFullyReplicatedStruct> TArrayOfNotFullyReplicatedStruct;
};

USTRUCT()
struct FTestReplicationStateDescriptor_StructWithArrayOfFullyReplicatedStruct
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FTestReplicationStateDescriptor_FullyReplicatedStruct> TArrayOfFullyReplicatedStruct;
};

USTRUCT()
struct FTestReplicationStateDescriptor_StructWithCustomSerializerWithApply
{
	GENERATED_BODY()

	UPROPERTY()
	int32 IntA = 0;
};

USTRUCT()
struct FTestReplicationStateDescriptor_StructWithArrayOfStructWithCustomSerializerWithApply
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FTestReplicationStateDescriptor_StructWithCustomSerializerWithApply> ArrayOfStructWithCustomSerializerWithApply;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithNotFullyReplicatedStructAndArrays : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithNotFullyReplicatedStructAndArrays() : UObject() {}

	UPROPERTY(Replicated)
	TArray<FTestReplicationStateDescriptor_FullyReplicatedStruct> TArrayOfNotFullyReplicatedStruct;

	UPROPERTY(Replicated)
	FTestReplicationStateDescriptor_StructWithArrayOfNotFullyReplicatedStruct StructWithArrayOfNotFullyReplicatedStruct;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithFullyReplicatedStructAndArrays : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithFullyReplicatedStructAndArrays() : UObject() {}

	UPROPERTY(Replicated)
	TArray<FTestReplicationStateDescriptor_FullyReplicatedStruct> TArrayOfFullyReplicatedStruct;

	UPROPERTY(Replicated)
	FTestReplicationStateDescriptor_StructWithArrayOfFullyReplicatedStruct StructWithArrayOfFullyReplicatedStruct;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithFieldPathProperty : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithFieldPathProperty() : UObject() {}

	UPROPERTY(Replicated)
	TFieldPath<FProperty> FieldPathProperty;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithRemoteObjectReferenceProperty : public UObject
{
	GENERATED_BODY()

public:
	UTestReplicationStateDescriptor_TestClassWithRemoteObjectReferenceProperty() : UObject() {}

	void GetLifetimeReplicatedProps(TArray<class FLifetimeProperty>& OutLifetimeProps) const;

	UPROPERTY(Replicated)
	FRemoteObjectReference RemoteObjectReferenceProperty;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithOnlyPushBasedProperties : public UObject
{
	GENERATED_BODY()

private:
	UPROPERTY(Replicated)
	int32 IntPushBasedA;

	UPROPERTY(Replicated)
	int32 IntPushBasedB;

	UPROPERTY(Replicated)
	int32 IntPushBasedC;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithOnlyPropertiesThatNeverReplicates : public UObject
{
	GENERATED_BODY()

private:
	UPROPERTY(Replicated)
	int32 IntNeverReplicatesA;

	UPROPERTY(Replicated)
	int32 IntNeverReplicatesB;

	UPROPERTY(Replicated)
	int32 IntNeverReplicatesC;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithPushBasedPropertiesAndNeverReplicatedProperties : public UObject
{
	GENERATED_BODY()

private:
	UPROPERTY(Replicated)
	int32 IntPushBasedA;

	UPROPERTY(Replicated)
	int32 IntPushBasedB;

	UPROPERTY(Replicated)
	int32 IntPushBasedC;

	UPROPERTY(Replicated)
	int32 IntNeverReplicatesA;

	UPROPERTY(Replicated)
	int32 IntNeverReplicatesB;

	UPROPERTY(Replicated)
	int32 IntNeverReplicatesC;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithPushBasedPropertiesAndNeverReplicatedPropertiesAndFunctions : public UObject
{
	GENERATED_BODY()

protected:
	UFUNCTION(Client, unreliable)
	void ClientUnrealiableFunction(int32 Param);

	UFUNCTION(Server, reliable)
	void ServerReliableFunction(int32 Param);

private:
	UPROPERTY(Replicated)
	int32 IntPushBasedA;

	UPROPERTY(Replicated)
	int32 IntPushBasedB;

	UPROPERTY(Replicated)
	int32 IntPushBasedC;

	UPROPERTY(Replicated)
	int32 IntNeverReplicatesA;

	UPROPERTY(Replicated)
	int32 IntNeverReplicatesB;

	UPROPERTY(Replicated)
	int32 IntNeverReplicatesC;
};

UCLASS()
class UTestReplicationStateDescriptor_TestClassWithPushBasedAndNonPushBasedProperties : public UObject
{
	GENERATED_BODY()

private:
	UPROPERTY(Replicated)
	int32 IntPushBasedA;

	UPROPERTY(Replicated)
	int32 IntPushBasedB;

	UPROPERTY(Replicated)
	int32 IntPushBasedC;

	UPROPERTY(Replicated)
	int32 IntNotPushBasedA;

	UPROPERTY(Replicated)
	int32 IntNotPushBasedB;

	UPROPERTY(Replicated)
	int32 IntNotPushBasedC;
};
