// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/CoreNetTypes.h"

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/Conditionals/ReplicationCondition.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"

namespace UE::Net::Private
{

class FTestConditionalsFixture : public FReplicationSystemServerClientTestFixture
{
protected:
};

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, ToOwnerStateIsReplicatedToOwner)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Set some values in ToOwner only state
	ServerObject->ConnectionFilteredComponents[0]->ToOwnerA = 13;
	ServerObject->ConnectionFilteredComponents[0]->ToOwnerB = 37;
	ServerObject->ConnectionFilteredComponents[0]->ReplayOrOwner = 47;

	// Set owner
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the To Owner members have the same values as on the sending side.
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->ToOwnerA, ServerObject->ConnectionFilteredComponents[0]->ToOwnerA);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->ToOwnerB, ServerObject->ConnectionFilteredComponents[0]->ToOwnerB);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->ReplayOrOwner, ServerObject->ConnectionFilteredComponents[0]->ReplayOrOwner);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, ToOwnerStateIsNotReplicatedToNonOwner)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Set some values in ToOwner only state
	ServerObject->ConnectionFilteredComponents[0]->ToOwnerA = 13;
	ServerObject->ConnectionFilteredComponents[0]->ToOwnerB = 37;
	ServerObject->ConnectionFilteredComponents[0]->ReplayOrOwner = 47;

	// Do not set owner. Optionally one could set it to some connection other than the client's.

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the To Owner members haven't been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->ToOwnerA, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->ToOwnerB, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->ReplayOrOwner, 0);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, SkipOwnerStateIsNotReplicatedToOwner)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Set some values in SkipOwner only state
	ServerObject->ConnectionFilteredComponents[0]->SkipOwnerA = 13;
	ServerObject->ConnectionFilteredComponents[0]->SkipOwnerB = 37;

	// Set owner
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the SkipOwner members haven't been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SkipOwnerA, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SkipOwnerB, 0);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, SkipOwnerStateIsReplicatedToNonOwner)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Set some values in SkipOwner only state
	ServerObject->ConnectionFilteredComponents[0]->SkipOwnerA = 13;
	ServerObject->ConnectionFilteredComponents[0]->SkipOwnerB = 37;

	// Do not set owner. Optionally one could set it to some connection other than the client's.

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the SkipOwner members have been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SkipOwnerA, ServerObject->ConnectionFilteredComponents[0]->SkipOwnerA);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SkipOwnerB, ServerObject->ConnectionFilteredComponents[0]->SkipOwnerB);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, SimulatedIsReplicatedToSimulated)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Set values that should be replicated for simulated objects
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt = 13;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt = 37;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt = 47;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt = 11;

	// Could set autonomous condition to be some connection other than the create client, or not set at all as is done now.

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the various Simulated members have been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, SimulatedIsNotReplicatedToAutonomous)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Set values that should be replicated for simulated objects
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt = 13;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt = 37;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt = 47;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt = 11;

	// Set this connection to be autonomous.
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, true);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the various Simulated members have not been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt, 0);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, AutonomousIsReplicatedToAutonomous)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Set values that should be replicated for autonomous objects
	ServerObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt = 13;

	// Set this connection to be autonomous.
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, true);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the Autonomous member has been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt, ServerObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, AutonomousIsNotReplicatedToSimulated)
{
	// Add clients
	FReplicationSystemTestClient* Client = CreateClient();
	// Adding a second client just to guarantee the connection ID for the autonomous owner is valid
	FReplicationSystemTestClient* AutonomousClient = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Set values that should be replicated for autonomous objects
	ServerObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt = 13;

	// Set other connection to be autonomous.
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, AutonomousClient->ConnectionIdOnServer, true);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the Autonomous member has not been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt, 0);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, SimulatedOrPhysicsIsReplicatedWhenPhysicsIsEnabled)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Set values that should be replicated for simulated objects
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt = 37;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt = 11;

	// Set this connection to be autonomous so we know that only the Physics condition is at play
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, true);
	// Enable ReplicatePhysics condition
	Server->ReplicationSystem->SetReplicationCondition(ServerObject->NetRefHandle, EReplicationCondition::ReplicatePhysics, true);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the various Physics members have been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, SimulatedOrPhysicsIsNotReplicatedWhenPhysicsIsNotEnabled)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Set values that should be replicated for simulated objects
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt = 37;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt = 11;

	// Set this connection to be autonomous so we know that only the Physics condition is at play
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, true);
	// Disable ReplicatePhysics condition, though that is expected to be the default
	Server->ReplicationSystem->SetReplicationCondition(ServerObject->NetRefHandle, EReplicationCondition::ReplicatePhysics, false);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the various Physics members have not been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt, 0);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, CanToggleAutonomous)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Set values that should be replicated for autonomous objects
	ServerObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt = 13;

	// Set this connection to be autonomous.
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, true);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the Autonomous member has been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt, ServerObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt);

	// Set this connection to not be autonomous (one could also pass ConnectionId 0)
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, false);

	// Set values that should be replicated for autonomous objects
	ServerObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt = 37;

	// 
	const int32 ExpectedAutonomousOnlyInt = ClientObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Check that the Autonomous member has not been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt, ExpectedAutonomousOnlyInt);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, CanToggleSimulated)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Could set autonomous condition to be some connection other than the create client, or not set at all as is done now.

	// Set values that should be replicated for simulated objects
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt = 13;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt = 37;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt = 47;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt = 11;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the various Simulated members have been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt);

	// Set this connection to be autonomous.
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, true);

	// Set values that should be replicated for simulated objects
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt = 47;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt = 11;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt = 13;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt = 37;

	// 
	const int32 ExpectedSimulatedOnlyInt = ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt;
	const int32 ExpectedSimulatedOrPhysicsInt = ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt;
	const int32 ExpectedSimulatedOnlyNoReplayInt = ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt;
	const int32 ExpectedSimulatedOrPhysicsNoReplayInt = ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Check that the Simulated members have not been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt, ExpectedSimulatedOnlyInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, ExpectedSimulatedOrPhysicsInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt, ExpectedSimulatedOnlyNoReplayInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt, ExpectedSimulatedOrPhysicsNoReplayInt);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, CanMixConditions)
{
	// Add clients
	FReplicationSystemTestClient* ClientArray[] = {CreateClient(), CreateClient()};
	constexpr uint32 AutonomousClientIndex = 1;
	const uint32 AutonomousClientConnectionId = ClientArray[AutonomousClientIndex]->ConnectionIdOnServer;

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Set Autonomous connection
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, AutonomousClientConnectionId, true);
	// Enable ReplicatePhysics condition
	Server->ReplicationSystem->SetReplicationCondition(ServerObject->NetRefHandle, EReplicationCondition::ReplicatePhysics, true);

	// Set all the values
	ServerObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt = 1;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt = 13;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt = 37;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt = 47;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt = 11;
	
	// Send and deliver packets
	Server->NetUpdate();
	for (FReplicationSystemTestClient* Client : ClientArray)
	{
		Server->SendAndDeliverTo(Client, DeliverPacket);
	}
	Server->PostSendUpdate();
	
	// Object should have been created on the clients
	const UTestReplicatedIrisObject* ClientObjectArray[sizeof(ClientArray)/sizeof(ClientArray[0])];
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		const SIZE_T ClientIndex = &Client - ClientArray;
		ClientObjectArray[ClientIndex] = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObjectArray[ClientIndex], nullptr);
	}

	// Validate all the members
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		const SIZE_T ClientIndex = &Client - ClientArray;
		const UTestReplicatedIrisObject* ClientObject = ClientObjectArray[ClientIndex];

		// Everybody should get the physics members
		UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt);
		UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt);
		if (Client->ConnectionIdOnServer == AutonomousClientConnectionId)
		{
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt, ServerObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt);
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt, 0);
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt, 0);
		}
		else
		{
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt, 0);
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt);
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt);
		}
	}
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, CanSwitchAutonmousConnection)
{
	// Add clients
	FReplicationSystemTestClient* ClientArray[] = {CreateClient(), CreateClient()};
	uint32 AutonomousClientIndex = 1;
	uint32 AutonomousClientConnectionId = ClientArray[AutonomousClientIndex]->ConnectionIdOnServer;

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Set Autonomous connection
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, AutonomousClientConnectionId, true);

	// Set all the values
	ServerObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt = 1;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt = 13;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt = 37;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt = 47;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt = 11;

	// Send and deliver packets
	Server->NetUpdate();
	for (FReplicationSystemTestClient* Client : ClientArray)
	{
		Server->SendAndDeliverTo(Client, DeliverPacket);
	}
	Server->PostSendUpdate();
	
	// Object should have been created on the clients
	const UTestReplicatedIrisObject* ClientObjectArray[sizeof(ClientArray)/sizeof(ClientArray[0])];
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		const SIZE_T ClientIndex = &Client - ClientArray;
		ClientObjectArray[ClientIndex] = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObjectArray[ClientIndex], nullptr);
	}

	// Switch Autonomous connection
	AutonomousClientIndex = (AutonomousClientIndex + 1)  % UE_ARRAY_COUNT(ClientArray);
	AutonomousClientConnectionId = ClientArray[AutonomousClientIndex]->ConnectionIdOnServer;
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, AutonomousClientConnectionId, true);

	// Set all the values
	ServerObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt = 1+1;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt = 13+1;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt = 37+1;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt = 47+1;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt = 11+1;

	// Send and deliver packets
	Server->NetUpdate();
	for (FReplicationSystemTestClient* Client : ClientArray)
	{
		Server->SendAndDeliverTo(Client, DeliverPacket);
	}
	Server->PostSendUpdate();

	// Validate all the members
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		const SIZE_T ClientIndex = &Client - ClientArray;
		const UTestReplicatedIrisObject* ClientObject = ClientObjectArray[ClientIndex];

		if (Client->ConnectionIdOnServer == AutonomousClientConnectionId)
		{
			// This should have been updated
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt, ServerObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt);

			// These should have the previously set values
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt - 1);
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt - 1);
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt - 1);
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt - 1);
		}
		else
		{
			// This should have the previously set value
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt, ServerObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt - 1);

			// These should have been updated
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt);
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt);
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt);
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt);
		}
	}
}


UE_NET_TEST_FIXTURE(FTestConditionalsFixture, SubObjectConditionsMatchesParentObject)
{
	// Add clients
	FReplicationSystemTestClient* ClientArray[] = {CreateClient(), CreateClient()};
	constexpr uint32 AutonomousClientIndex = 1;
	const uint32 AutonomousClientConnectionId = ClientArray[AutonomousClientIndex]->ConnectionIdOnServer;

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, ObjectComponents);


	// Set Autonomous connection
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, AutonomousClientConnectionId, true);
	// Enable ReplicatePhysics condition
	Server->ReplicationSystem->SetReplicationCondition(ServerObject->NetRefHandle, EReplicationCondition::ReplicatePhysics, true);

	// Set all the values
	for (UTestReplicatedIrisObject* Object : {ServerObject, ServerSubObject})
	{
		Object->ConnectionFilteredComponents[0]->AutonomousOnlyInt = 1;
		Object->ConnectionFilteredComponents[0]->SimulatedOnlyInt = 13;
		Object->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt = 37;
		Object->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt = 47;
		Object->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt = 11;
	}
	
	// Send and deliver packets
	Server->NetUpdate();
	for (FReplicationSystemTestClient* Client : ClientArray)
	{
		Server->SendAndDeliverTo(Client, DeliverPacket);
	}
	Server->PostSendUpdate();
	
	// Object and subobject should have been created on the clients
	const UTestReplicatedIrisObject* ClientObjectArray[sizeof(ClientArray)/sizeof(ClientArray[0])];
	const UTestReplicatedIrisObject* ClientSubObjectArray[sizeof(ClientArray)/sizeof(ClientArray[0])];
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		const SIZE_T ClientIndex = &Client - ClientArray;
		ClientObjectArray[ClientIndex] = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObjectArray[ClientIndex], nullptr);
		ClientSubObjectArray[ClientIndex] = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientSubObjectArray[ClientIndex], nullptr);
	}

	// Validate all the members
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		const SIZE_T ClientIndex = &Client - ClientArray;

		for (const UTestReplicatedIrisObject* ClientObject : {ClientObjectArray[ClientIndex], ClientSubObjectArray[ClientIndex]})
		{
			// Everybody should get the physics members
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt);
			UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt);

			if (Client->ConnectionIdOnServer == AutonomousClientConnectionId)
			{
				UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt, ServerObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt);
				UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt, 0);
				UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt, 0);
			}
			else
			{
				UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->AutonomousOnlyInt, 0);
				UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt);
				UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt);
			}
		}
	}
}

// Test set subobject condition api
// Test restored after removing subobject
// Test Filters out as expected
UE_NET_TEST_FIXTURE(FTestConditionalsFixture, WithSkipOwnerSubObjectIsNotReplicatedToOwner)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object + subobject on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_SkipOwner);

	// Set owner
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Set some values
	ServerSubObject->IntA = 13; 

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// SubObject should not have been created on client
	const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_EQ(ClientSubObject, nullptr);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, WithSkipOwnerSubObjectReplicatedToOther)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();
	FReplicationSystemTestClient* OtherClient = CreateClient();

	// Spawn object + subobject on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_SkipOwner);

	// Set owner
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Set some values
	ServerSubObject->IntA = 13; 

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(OtherClient, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Object should have been created on the client
	const UTestReplicatedIrisObject* OtherClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(OtherClientObject, nullptr);

	// SubObject should not ahve been created on client
	const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_EQ(ClientSubObject, nullptr);

	// SubObject should not ahve been created on client
	const UTestReplicatedIrisObject* OtherClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_EQ(OtherClientSubObject, nullptr);
}

 UE_NET_TEST_FIXTURE(FTestConditionalsFixture, WithSkipOwnerSubObjectCanBeChanged)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object + subobject on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_SkipOwner);

	// Set owner
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Set some values
	ServerSubObject->IntA = 13; 

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// SubObject should not have been created on client
	{
		const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
		UE_NET_ASSERT_EQ(ClientSubObject, nullptr);
	}

	// Remove condition
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_None);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// SubObject should now have been created on client
	{
		const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientSubObject, nullptr);

		UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObject->IntA);
	}
}

// Test case where we ended up trying to send subobjects waiting for creation confirmation causing server check/crash
UE_NET_TEST_FIXTURE(FTestConditionalsFixture, WithSkipOwnerSubObjectOwnerCanBeChangedWithDataInFlight)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object + subobject on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_SkipOwner);

	// Set owner
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Set some values
	ServerSubObject->IntA = 13; 

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// SubObject should not have been created on client
	{
		const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
		UE_NET_ASSERT_EQ(ClientSubObject, nullptr);
	}

	// Change owner
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, 0);

	// Send and delay packet
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Change owner again, just to dirty things and cause a conditional change with subobject pending create in-flight
	// before a fix in ReplicationWriter this would cause an invalid state assert/check
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, 0);

	// Send and delay packet
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	Server->DeliverTo(Client, true);
	Server->DeliverTo(Client, true);

	// SubObject should now have been created on client
	{
		const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientSubObject, nullptr);

		UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObject->IntA);
	}
}


UE_NET_TEST_FIXTURE(FTestConditionalsFixture, HierarchicalSubObjectIsNotReplicatedToOwnerUnlessParentIs)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object + subobjects on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	UTestReplicatedIrisObject* ServerSubSubObject = Server->CreateSubObjectWithParams<UTestReplicatedIrisObject>(
		{ 
			.RootObjectHandle = ServerObject->NetRefHandle, 
			.ParentSubObjectHandle = ServerSubObject->NetRefHandle,
		});

	// Set condition to only replicate if root is the owning connection
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_SkipOwner);

	// Set owner
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Set some values
	ServerSubObject->IntA = 13; 

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// SubObject and SubSubObject should not have been created on client
	{
		const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
		UE_NET_ASSERT_EQ(ClientSubObject, nullptr);

		const UTestReplicatedIrisObject* ClientSubSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubSubObject->NetRefHandle));
		UE_NET_ASSERT_EQ(ClientSubSubObject, nullptr);
	}

	// Remove condition on ParentSubObject
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_None);
	// Add condtion on SubSubObject
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubSubObject->NetRefHandle, ELifetimeCondition::COND_SkipOwner);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// SubObject should now be replicated while SubSubObject should not
	{
		const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientSubObject, nullptr);

		const UTestReplicatedIrisObject* ClientSubSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubSubObject->NetRefHandle));
		UE_NET_ASSERT_EQ(ClientSubSubObject, nullptr);
	}

	// Remove condition on SubSubObject
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubSubObject->NetRefHandle, ELifetimeCondition::COND_None);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// SubObject and SubSubObject should now both be replicated
	{
		const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientSubObject, nullptr);

		const UTestReplicatedIrisObject* ClientSubSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubSubObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientSubSubObject, nullptr);
	}

}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, SubObjectWithCustomOrderGetsCorrectConditional)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object + subobjects on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);
	UTestReplicatedIrisObject* ServerSubObject0 = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	// Set condition on first subobject to not replicate
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject0->NetRefHandle, ELifetimeCondition::COND_Never);

	// Add another subobject
	UTestReplicatedIrisObject* ServerSubObject1 = Server->CreateSubObjectWithParams<UTestReplicatedIrisObject>(
		{ 
			.RootObjectHandle = ServerObject->NetRefHandle, 
			.InsertionOrder = ESubObjectInsertionOrder::InsertAtStart
		});

	// Set some values
	ServerSubObject0->IntA = 1; 
	ServerSubObject1->IntA = 2; 

	// Send and deliver packet
	Server->UpdateAndSend({Client});
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// SubObject0 should not have been created, 
	const UTestReplicatedIrisObject* ClientSubObject0 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject0->NetRefHandle));
	UE_NET_ASSERT_EQ(ClientSubObject0, nullptr);

	// SubObject1 should have been created, 
	const UTestReplicatedIrisObject* ClientSubObject1 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject1->NetRefHandle));
	UE_NET_ASSERT_NE(ClientSubObject1, nullptr);	
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, NoneIsReplicated)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Set values that should be replicated
	ServerObject->ConnectionFilteredComponents[0]->NoneInt = 13;

	// Could set autonomous condition to be some connection other than the create client, or not set at all as is done now.

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the COND_None member has been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->NoneInt, ServerObject->ConnectionFilteredComponents[0]->NoneInt);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, NeverIsNotReplicated)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	const int32 ExpectedClientValue = ServerObject->ConnectionFilteredComponents[0]->NeverInt;

	// Set value
	ServerObject->ConnectionFilteredComponents[0]->NeverInt = 13;

	// Could set autonomous condition to be some connection other than the create client, or not set at all as is done now.

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the COND_Never member remains unmodified.
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->NeverInt, ExpectedClientValue);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, SkipReplayIsReplicatedToRegularConnection)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	// Set value
	ServerObject->ConnectionFilteredComponents[0]->SkipReplayInt = 13;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the COND_SkipReplay member has been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SkipReplayInt, ServerObject->ConnectionFilteredComponents[0]->SkipReplayInt);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, ReplayOnlyIsNotReplicatedToRegularConnection)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);

	const int32 ExpectedClientValue = ServerObject->ConnectionFilteredComponents[0]->NeverInt;

	// Set value
	ServerObject->ConnectionFilteredComponents[0]->ReplayOnlyInt = 13;

	// Could set autonomous condition to be some connection other than the create client, or not set at all as is done now.

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the COND_SkipReplay member has been modified
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->ReplayOnlyInt, ExpectedClientValue);
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, PushBased_COND_InitialOrOwner)
{
	// Add clients
	FReplicationSystemTestClient* SomeClients[] = { CreateClient(), CreateClient() };
	constexpr SIZE_T NonOwnerClientIndex = 0;
	constexpr SIZE_T OwnerClientIndex = 1;

	UTestReplicatedIrisPushModelObject* ServerObject = Server->CreateObject<UTestReplicatedIrisPushModelObject>();

	// Set value
	ServerObject->SetInitialOrOwnerInt(13);

	// Set owning connection
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, SomeClients[OwnerClientIndex]->ConnectionIdOnServer);

	// Send and deliver packets
	for (auto Client : SomeClients)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
	}
	
	// Check value on owning client
	{
		const UTestReplicatedIrisPushModelObject* ClientObject = Cast<UTestReplicatedIrisPushModelObject>(SomeClients[OwnerClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the value is the same as on the server
		UE_NET_ASSERT_EQ(ClientObject->GetInitialOrOwnerInt(), ServerObject->GetInitialOrOwnerInt());
	}

	// Check value on non-owning client
	{
		const UTestReplicatedIrisPushModelObject* ClientObject = Cast<UTestReplicatedIrisPushModelObject>(SomeClients[NonOwnerClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the value is the same as on the server
		UE_NET_ASSERT_EQ(ClientObject->GetInitialOrOwnerInt(), ServerObject->GetInitialOrOwnerInt());
	}

	// Modify value
	const int32 PrevValue = ServerObject->GetInitialOrOwnerInt();
	ServerObject->SetInitialOrOwnerInt(PrevValue + 13);

	// Send and deliver packets
	for (auto Client : SomeClients)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
	}

	// Check value on owning client. It should've been updated.
	{
		const UTestReplicatedIrisPushModelObject* ClientObject = Cast<UTestReplicatedIrisPushModelObject>(SomeClients[OwnerClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the value the same as on the server
		UE_NET_ASSERT_EQ(ClientObject->GetInitialOrOwnerInt(), ServerObject->GetInitialOrOwnerInt());
	}

	// Check value on non-owning client. It should remain the same as the previous value.
	{
		const UTestReplicatedIrisPushModelObject* ClientObject = Cast<UTestReplicatedIrisPushModelObject>(SomeClients[NonOwnerClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the value is unmodified.
		UE_NET_ASSERT_EQ(ClientObject->GetInitialOrOwnerInt(), PrevValue);
	}
}

UE_NET_TEST_FIXTURE(FTestConditionalsFixture, PushBased_COND_InitialOnly)
{
	// Create ClientA
	FReplicationSystemTestClient* ClientA =  CreateClient();

	UTestReplicatedIrisPushModelObject* ServerObject = Server->CreateObject<UTestReplicatedIrisPushModelObject>();

	// Set value
	ServerObject->SetInitialOnlyInt(13);

	// Send and deliver to clientA
	Server->UpdateAndSend({ ClientA });
	
	// Check value on ClientA
	{
		const UTestReplicatedIrisPushModelObject* ClientObject = Cast<UTestReplicatedIrisPushModelObject>(ClientA->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the value is the same as on the server
		UE_NET_ASSERT_EQ(ClientObject->GetInitialOnlyInt(), ServerObject->GetInitialOnlyInt());
	}

	// Modify value
	const int32 PrevValue = ServerObject->GetInitialOnlyInt();
	ServerObject->SetInitialOnlyInt(PrevValue + 13);

	// Create ClientB
	FReplicationSystemTestClient* ClientB =  CreateClient();

	// Send and deliver to clientA
	Server->UpdateAndSend({ ClientA, ClientB});

	// Check value on ClientA, should still be the PrevValue
	{
		const UTestReplicatedIrisPushModelObject* ClientObject = Cast<UTestReplicatedIrisPushModelObject>(ClientA->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the value is the same as on the server
		UE_NET_ASSERT_EQ(ClientObject->GetInitialOnlyInt(), PrevValue);
	}

	// Check value on ClientB, it should be the updated value
	{
		const UTestReplicatedIrisPushModelObject* ClientObject = Cast<UTestReplicatedIrisPushModelObject>(ClientB->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the value is the same as on the server
		UE_NET_ASSERT_EQ(ClientObject->GetInitialOnlyInt(), ServerObject->GetInitialOnlyInt());
	}
}

}
