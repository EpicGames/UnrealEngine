// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/Conditionals/ReplicationCondition.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"

namespace UE::Net::Private
{

class FTestDynamicConditionFixture : public FReplicationSystemServerClientTestFixture
{
protected:
};


UE_NET_TEST_FIXTURE(FTestDynamicConditionFixture, DynamicConditionIsReplicatedByDefault)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObjectWithDynamicCondition* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithDynamicCondition>();

	// Set value
	ServerObject->DynamicConditionInt ^= 13;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the dynamic condition member has been modified
	UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, ServerObject->DynamicConditionInt);
}

UE_NET_TEST_FIXTURE(FTestDynamicConditionFixture, DynamicConditionSupports_COND_Never)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObjectWithDynamicCondition* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithDynamicCondition>();

	// Set value
	const int32 OriginalValue = ServerObject->DynamicConditionInt;
	ServerObject->DynamicConditionInt ^= 13;

	// Set condition to COND_Never
	ServerObject->SetDynamicCondition(COND_Never);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the dynamic condition member is unmodified
	UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, OriginalValue);
}

UE_NET_TEST_FIXTURE(FTestDynamicConditionFixture, DynamicConditionSupports_COND_AutonomousOnly)
{
	// Add clients
	FReplicationSystemTestClient* SomeClients[] = { CreateClient(), CreateClient() };
	constexpr SIZE_T ClientIndex = 0;
	constexpr SIZE_T AutonomousClientIndex = 1;

	UTestReplicatedIrisObjectWithDynamicCondition* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithDynamicCondition>();

	// Set value
	const int32 OriginalValue = ServerObject->DynamicConditionInt;
	ServerObject->DynamicConditionInt ^= 13;

	// Set condition to COND_AutonomousOnly
	ServerObject->SetDynamicCondition(COND_AutonomousOnly);

	// Set other connection to be autonomous.
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, SomeClients[AutonomousClientIndex]->ConnectionIdOnServer, true);

	// Send and deliver packets
	for (auto Client : SomeClients)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
	}
	
	// Check value on client
	{
		const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(SomeClients[ClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the dynamic condition member is unmodified
		UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, OriginalValue);
	}

	// Check value on autonomous client
	{
		const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(SomeClients[AutonomousClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the dynamic condition member is the same as on the server
		UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, ServerObject->DynamicConditionInt);
	}
}

UE_NET_TEST_FIXTURE(FTestDynamicConditionFixture, DynamicConditionSupports_COND_SimulatedOnly)
{
	// Add clients
	FReplicationSystemTestClient* SomeClients[] = { CreateClient(), CreateClient() };
	constexpr SIZE_T SimulatedClientIndex = 0;
	constexpr SIZE_T AutonomousClientIndex = 1;

	UTestReplicatedIrisObjectWithDynamicCondition* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithDynamicCondition>();

	// Set value
	const int32 OriginalValue = ServerObject->DynamicConditionInt;
	ServerObject->DynamicConditionInt ^= 13;

	// Set condition to COND_SimulatedOnly
	ServerObject->SetDynamicCondition(COND_SimulatedOnly);

	// Set one connection to be autonomous.
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, SomeClients[AutonomousClientIndex]->ConnectionIdOnServer, true);

	// Send and deliver packets
	for (auto Client : SomeClients)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
	}
	
	// Check value on simulated client
	{
		const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(SomeClients[SimulatedClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the dynamic condition member is the same as on the server
		UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, ServerObject->DynamicConditionInt);
	}

	// Check value on autonomous client
	{
		const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(SomeClients[AutonomousClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the dynamic condition member is unmodified
		UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, OriginalValue);
	}
}

UE_NET_TEST_FIXTURE(FTestDynamicConditionFixture, DynamicConditionSupports_COND_SkipOwner)
{
	// Add clients
	FReplicationSystemTestClient* SomeClients[] = { CreateClient(), CreateClient() };
	constexpr SIZE_T NonOwnerClientIndex = 0;
	constexpr SIZE_T OwnerClientIndex = 1;

	UTestReplicatedIrisObjectWithDynamicCondition* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithDynamicCondition>();

	// Set value
	const int32 OriginalValue = ServerObject->DynamicConditionInt;
	ServerObject->DynamicConditionInt ^= 13;

	// Set condition to COND_SkipOwner
	ServerObject->SetDynamicCondition(COND_SkipOwner);

	// Set owning connection
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, SomeClients[OwnerClientIndex]->ConnectionIdOnServer);

	// Send and deliver packets
	for (auto Client : SomeClients)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
	}
	
	// Check value on owning client
	{
		const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(SomeClients[OwnerClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the dynamic condition member is unmodified
		UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, OriginalValue);
	}

	// Check value on non-owning client
	{
		const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(SomeClients[NonOwnerClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the dynamic condition member is the same as on the server
		UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, ServerObject->DynamicConditionInt);
	}
}

UE_NET_TEST_FIXTURE(FTestDynamicConditionFixture, DynamicConditionSupports_COND_OwnerOnly)
{
	// Add clients
	FReplicationSystemTestClient* SomeClients[] = { CreateClient(), CreateClient() };
	constexpr SIZE_T NonOwnerClientIndex = 0;
	constexpr SIZE_T OwnerClientIndex = 1;

	UTestReplicatedIrisObjectWithDynamicCondition* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithDynamicCondition>();

	// Set value
	const int32 OriginalValue = ServerObject->DynamicConditionInt;
	ServerObject->DynamicConditionInt ^= 13;

	// Set condition to COND_OwnerOnly
	ServerObject->SetDynamicCondition(COND_OwnerOnly);

	// Set owning connection
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, SomeClients[OwnerClientIndex]->ConnectionIdOnServer);

	// Send and deliver packets
	for (auto Client : SomeClients)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
	}
	
	// Check value on owning client
	{
		const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(SomeClients[OwnerClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the dynamic condition member is the same as on the server
		UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, ServerObject->DynamicConditionInt);
	}

	// Check value on non-owning client
	{
		const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(SomeClients[NonOwnerClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the dynamic condition member is unmodified
		UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, OriginalValue);
	}
}

UE_NET_TEST_FIXTURE(FTestDynamicConditionFixture, DynamicConditionSupports_COND_InitialOnly)
{
	// Add clients
	FReplicationSystemTestClient* Client = CreateClient();

	UTestReplicatedIrisObjectWithDynamicCondition* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithDynamicCondition>();

	// Set value
	ServerObject->DynamicConditionInt ^= 13;

	// Set condition to COND_InitialOnly
	ServerObject->SetDynamicCondition(COND_InitialOnly);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the dynamic condition member is the same as on the server
	UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, ServerObject->DynamicConditionInt);

	// Modify value
	const int32 PrevValue = ServerObject->DynamicConditionInt;
	ServerObject->DynamicConditionInt += 13;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Check that the dynamic condition member has not been modified
	UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, PrevValue);
}

UE_NET_TEST_FIXTURE(FTestDynamicConditionFixture, DynamicConditionSupports_COND_InitialOrOwner)
{
	// Add clients
	FReplicationSystemTestClient* SomeClients[] = { CreateClient(), CreateClient() };
	constexpr SIZE_T NonOwnerClientIndex = 0;
	constexpr SIZE_T OwnerClientIndex = 1;

	UTestReplicatedIrisObjectWithDynamicCondition* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithDynamicCondition>();

	// Set value
	ServerObject->DynamicConditionInt ^= 13;

	// Set condition to COND_OwnerOnly
	ServerObject->SetDynamicCondition(COND_InitialOrOwner);

	// Set owning connection
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, SomeClients[OwnerClientIndex]->ConnectionIdOnServer);

	// Send and deliver packets
	for (auto Client : SomeClients)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
	}
	
	// Check value on owning client
	{
		const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(SomeClients[OwnerClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the dynamic condition member is the same as on the server
		UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, ServerObject->DynamicConditionInt);
	}

	// Check value on non-owning client
	{
		const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(SomeClients[NonOwnerClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the dynamic condition member is the same as on the server
		UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, ServerObject->DynamicConditionInt);
	}

	// Modify value
	const int32 PrevValue = ServerObject->DynamicConditionInt;
	ServerObject->DynamicConditionInt += 13;

	// Send and deliver packets
	for (auto Client : SomeClients)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
	}

	// Check value on owning client. It should've been updated.
	{
		const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(SomeClients[OwnerClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the dynamic condition member is the same as on the server
		UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, ServerObject->DynamicConditionInt);
	}

	// Check value on non-owning client. It should remain the same as the previous value.
	{
		const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(SomeClients[NonOwnerClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Check that the dynamic condition member is unmodified.
		UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, PrevValue);
	}
}

UE_NET_TEST_FIXTURE(FTestDynamicConditionFixture, DynamicConditionSupports_COND_Custom)
{
	// Add clients
	FReplicationSystemTestClient* Client = CreateClient();

	UTestReplicatedIrisObjectWithDynamicCondition* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithDynamicCondition>();

	// Set value
	ServerObject->DynamicConditionInt ^= 13;

	// Set condition to COND_Custom. Technically speaking this isn't required as all propertys with a real condition (not COND_Never/None) can be enabled/disabled at will.
	ServerObject->SetDynamicCondition(COND_Custom);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the dynamic condition member is the same as on the server
	UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, ServerObject->DynamicConditionInt);

	// Disable the custom condition
	ServerObject->SetDynamicConditionCustomCondition(false);

	// Modify value
	const int32 PrevValue = ServerObject->DynamicConditionInt;
	ServerObject->DynamicConditionInt += 13;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Check that the dynamic condition member has not been modified
	UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, PrevValue);
}

UE_NET_TEST_FIXTURE(FTestDynamicConditionFixture, DynamicConditionSupports_COND_SimulatedOrPhysics)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	UTestReplicatedIrisObjectWithDynamicCondition* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithDynamicCondition>();

	// Set the connection to be autonomous so we know that only the Physics condition is at play
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, true);

	// Enable ReplicatePhysics condition
	Server->ReplicationSystem->SetReplicationCondition(ServerObject->NetRefHandle, EReplicationCondition::ReplicatePhysics, true);

	// Set value
	ServerObject->DynamicConditionInt ^= 13;

	// Set condition to COND_SimulatedOrPhysics. 
	ServerObject->SetDynamicCondition(COND_SimulatedOrPhysics);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the dynamic condition member is the same as on the server
	UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, ServerObject->DynamicConditionInt);

	// Disable ReplicatePhysics condition
	Server->ReplicationSystem->SetReplicationCondition(ServerObject->NetRefHandle, EReplicationCondition::ReplicatePhysics, false);

	// Modify value
	const int32 PrevValue = ServerObject->DynamicConditionInt;
	ServerObject->DynamicConditionInt += 13;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Check that the dynamic condition member is unmodified
	UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, PrevValue);

	// Set the connection to no longer be autonomous, thus simulated
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, false);

	// Modify value
	ServerObject->DynamicConditionInt += 17;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Check that the dynamic condition member is the same as on the server
	UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, ServerObject->DynamicConditionInt);
}

UE_NET_TEST_FIXTURE(FTestDynamicConditionFixture, DynamicConditionSupportsGoingFromNotReplicatedToReplicated)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObjectWithDynamicCondition* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithDynamicCondition>();

	// Set value
	const int32 OriginalValue = ServerObject->DynamicConditionInt;
	ServerObject->DynamicConditionInt ^= 13;

	// Set condition to COND_Never
	ServerObject->SetDynamicCondition(COND_Never);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Object should have been created on the client
	const UTestReplicatedIrisObjectWithDynamicCondition* ClientObject = Cast<UTestReplicatedIrisObjectWithDynamicCondition>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the dynamic condition member is unmodified
	UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, OriginalValue);

	// Set condition to COND_None
	ServerObject->SetDynamicCondition(COND_None);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Check that the dynamic condition member is the same as on the server
	UE_NET_ASSERT_EQ(ClientObject->DynamicConditionInt, ServerObject->DynamicConditionInt);
}

}
