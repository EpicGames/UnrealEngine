// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockNetObjectFilter.h"
#include "Containers/ArrayView.h"
#include "TestFilteringObject.h"

#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"

UMockNetObjectFilter::UMockNetObjectFilter()
: CallStatus({})
, AddedCount(0U)
{
}

void UMockNetObjectFilter::OnInit(const FNetObjectFilterInitParams& Params)
{
	AddFilterTraits(ENetFilterTraits::NeedsUpdate);

	++CallStatus.CallCounts.Init;

	CallStatus.SuccessfulCallCounts.Init += Cast<UMockNetObjectFilterConfig>(Params.Config) != nullptr;

	AddedObjectIndices.Init(Params.CurrentMaxInternalIndex);
	AddedConnectionIndices.Init(Params.MaxConnectionCount + 1);
}

void UMockNetObjectFilter::OnDeinit()
{
}

void UMockNetObjectFilter::OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex)
{
	AddedObjectIndices.SetNumBits(NewMaxInternalIndex);
}

void UMockNetObjectFilter::AddConnection(uint32 ConnectionId)
{
	++CallStatus.CallCounts.AddConnection;

	CallStatus.SuccessfulCallCounts.AddConnection += ConnectionId != 0U;

	AddedConnectionIndices.SetBit(ConnectionId);
}

void UMockNetObjectFilter::RemoveConnection(uint32 ConnectionId)
{
	++CallStatus.CallCounts.RemoveConnection;

	CallStatus.SuccessfulCallCounts.RemoveConnection += AddedConnectionIndices.GetBit(ConnectionId);
	AddedConnectionIndices.ClearBit(ConnectionId);
}

bool UMockNetObjectFilter::AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectFilterAddObjectParams& Params)
{
	++CallStatus.CallCounts.AddObject;
	if (CallSetup.AddObject.bReturnValue)
	{
		bool bIsProperCall = true;

		// Validate the passed info is zeroed
		bIsProperCall &= Params.OutInfo.Data[0] == 0;
		bIsProperCall &= Params.OutInfo.Data[1] == 0;
		bIsProperCall &= Params.OutInfo.Data[2] == 0;
		bIsProperCall &= Params.OutInfo.Data[3] == 0;
		
		CallStatus.SuccessfulCallCounts.AddObject += bIsProperCall;

		// Store object index in info so we can check it's passed to RemoveObject
		Params.OutInfo.Data[0] = uint16(ObjectIndex);
		Params.OutInfo.Data[1] = uint16(ObjectIndex);
		Params.OutInfo.Data[2] = uint16(ObjectIndex);
		Params.OutInfo.Data[3] = uint16(ObjectIndex);

		AddedObjectIndices.SetBit(ObjectIndex);
		++AddedCount;
	}

	return CallSetup.AddObject.bReturnValue;
}

void UMockNetObjectFilter::RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectFilteringInfo& Info)
{
	++CallStatus.CallCounts.RemoveObject;

	// If this object wasn't added to us it shouldn't be removed either.
	if (AddedObjectIndices.GetBit(ObjectIndex))
	{
		bool bIsProperCall = true;

		// Validate the passed info is as we set it
		bIsProperCall &= Info.Data[0] == uint16(ObjectIndex);
		bIsProperCall &= Info.Data[1] == uint16(ObjectIndex);
		bIsProperCall &= Info.Data[2] == uint16(ObjectIndex);
		bIsProperCall &= Info.Data[3] == uint16(ObjectIndex);

		CallStatus.SuccessfulCallCounts.RemoveObject += bIsProperCall;

		AddedObjectIndices.ClearBit(ObjectIndex);
		--AddedCount;
	}
}

void UMockNetObjectFilter::UpdateObjects(FNetObjectFilterUpdateParams& Params)
{
	++CallStatus.CallCounts.UpdateObjects;

	bool bIsProperCall = Params.ObjectCount <= AddedCount;
	for (UE::Net::FInternalNetRefIndex ObjectIndex : MakeArrayView(Params.ObjectIndices, Params.ObjectCount))
	{
		bIsProperCall = bIsProperCall && AddedObjectIndices.GetBit(ObjectIndex);
	}

	CallStatus.SuccessfulCallCounts.UpdateObjects += bIsProperCall;
}

void UMockNetObjectFilter::PreFilter(FNetObjectPreFilteringParams& Params)
{
	++CallStatus.CallCounts.PreFilter;

	constexpr bool bIsProperCall = true;
	CallStatus.SuccessfulCallCounts.PreFilter += bIsProperCall;
}

void UMockNetObjectFilter::Filter(FNetObjectFilteringParams& Params)
{
	++CallStatus.CallCounts.Filter;

	UE::Net::FNetBitArrayView AddedObjectsView = MakeNetBitArrayView(AddedObjectIndices);

	bool bIsProperCall = true;
	UE::Net::FNetBitArrayView::ForAllExclusiveBits(GetFilteredObjects(), AddedObjectsView, [](...) {}, [&bIsProperCall](...) {bIsProperCall = false; });
	CallStatus.SuccessfulCallCounts.Filter += bIsProperCall;

	if (CallSetup.Filter.bFilterOutByDefault)
	{
		Params.OutAllowedObjects.ClearAllBits();
	}
	else
	{
		Params.OutAllowedObjects.Copy(AddedObjectsView);
	}
}

void UMockNetObjectFilter::PostFilter(FNetObjectPostFilteringParams& Params)
{
	++CallStatus.CallCounts.PostFilter;

	constexpr bool bIsProperCall = true;
	CallStatus.SuccessfulCallCounts.PostFilter += bIsProperCall;
}

//**************************************************************************************************
// UMockNetObjectFilterWithCondition
//**************************************************************************************************

void UMockNetObjectFilterWithCondition::OnInit(const FNetObjectFilterInitParams& Params)
{
	Super::OnInit(Params);

	ReplicationSystem = Params.ReplicationSystem.Get();
}

bool UMockNetObjectFilterWithCondition::AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectFilterAddObjectParams& Params)
{
	bool bResult = Super::AddObject(ObjectIndex, Params);

	// Make sure the object is of the proper class
	UObject* ReplicatedObject = ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager().GetReplicatedObjectInstance(ObjectIndex);
	UTestFilteringObject* FilterObject = CastChecked<UTestFilteringObject>(ReplicatedObject);

	return bResult;
}

void UMockNetObjectFilterWithCondition::Filter(FNetObjectFilteringParams& Params)
{
	++CallStatus.CallCounts.Filter;

	bool bIsProperCall = true;

	GetFilteredObjects().ForAllSetBits([&](UE::Net::FInternalNetRefIndex ObjectIndex)
	{
		UTestFilteringObject* FilterObject = CastChecked<UTestFilteringObject>(ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager().GetReplicatedObjectInstance(ObjectIndex));
		
		const bool bIsFilteredOut = FilterObject->GetFilterOut();

		Params.OutAllowedObjects.SetBitValue(ObjectIndex, !bIsFilteredOut);
	});
	
	CallStatus.SuccessfulCallCounts.Filter += bIsProperCall;
}