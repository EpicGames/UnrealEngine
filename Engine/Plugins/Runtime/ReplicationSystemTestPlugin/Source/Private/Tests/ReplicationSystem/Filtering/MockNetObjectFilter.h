// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Iris/ReplicationSystem/RepTag.h"
#include "Net/Core/NetBitArray.h"
#include "MockNetObjectFilter.generated.h"

namespace UE::Net
{
	typedef uint32 FInternalNetRefIndex;
}

const UE::Net::FRepTag RepTag_NetTest_FilterOut = 0x521855F5DFA298B7ULL;

UCLASS()
class UMockNetObjectFilterConfig : public UNetObjectFilterConfig
{
	GENERATED_BODY()

};

UCLASS()
class UMockNetObjectFilter : public UNetObjectFilter
{
	GENERATED_BODY()

public:
	struct FFunctionCallStatus
	{
		struct
		{
			uint32 Init;
			uint32 AddConnection;
			uint32 RemoveConnection;
			uint32 AddObject;
			uint32 RemoveObject;
			uint32 UpdateObjects;
			uint32 PreFilter;
			uint32 Filter;
			uint32 PostFilter;
		} CallCounts, SuccessfulCallCounts;
	};

	struct FFunctionCallSetup
	{
		// AddObject
		struct FAddObject
		{
			bool bReturnValue = false;
		};

		// Filter
		struct FFilter
		{
			bool bFilterOutByDefault = false;
		};

		FAddObject AddObject;
		FFilter Filter;
	};

	inline void SetFunctionCallSetup(const FFunctionCallSetup& Setup) { CallSetup = Setup; }
	inline const FFunctionCallStatus& GetFunctionCallStatus() const { return CallStatus; }
	inline void ResetFunctionCallStatus() { CallStatus = FFunctionCallStatus({}); }

protected:
	virtual void OnInit(const FNetObjectFilterInitParams&) override;
	virtual void OnDeinit() override;
	virtual void OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex) override;
	virtual void AddConnection(uint32 ConnectionId) override;
	virtual void RemoveConnection(uint32 ConnectionId) override;
	virtual bool AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectFilterAddObjectParams&) override;
	virtual void RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectFilteringInfo&) override;
	virtual void UpdateObjects(FNetObjectFilterUpdateParams&) override;
	virtual void PreFilter(FNetObjectPreFilteringParams&) override;
	virtual void Filter(FNetObjectFilteringParams&) override;
	virtual void PostFilter(FNetObjectPostFilteringParams&) override;

protected:
	UMockNetObjectFilter();

	FFunctionCallStatus CallStatus;
	FFunctionCallSetup CallSetup;
	UE::Net::FNetBitArray AddedObjectIndices;
	UE::Net::FNetBitArray AddedConnectionIndices;
	SIZE_T AddedCount;

	static constexpr float DefaultPriority = 1.0f;
};

/**
 * Filter that checks object data to decide to filter out an object.
 */
UCLASS()
class UMockNetObjectFilterWithCondition : public UMockNetObjectFilter
{
	GENERATED_BODY()

protected:

	virtual void OnInit(const FNetObjectFilterInitParams&) override;
	virtual bool AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectFilterAddObjectParams&) override;
	virtual void Filter(FNetObjectFilteringParams&) override;

private:

	const UReplicationSystem* ReplicationSystem = nullptr;
};