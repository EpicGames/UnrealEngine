// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "IrisTestMessageStreamOperators.h"
#include "Iris/ReplicationSystem/Filtering/SharedConnectionFilterStatus.h"

namespace UE::Net::Private
{

class FConnectionHandleTestFixture : public FNetworkAutomationTestSuiteFixture
{
};

UE_NET_TEST_FIXTURE(FConnectionHandleTestFixture, ConnectionHandleIsInvalidByDefault)
{
	FConnectionHandle ConnHandle;
	UE_NET_ASSERT_FALSE(ConnHandle.IsValid());
}

UE_NET_TEST_FIXTURE(FConnectionHandleTestFixture, ConnectionHandleWithParentConnectioIdIsValid)
{
	constexpr uint32 ParentConnId = 1U;
	FConnectionHandle ConnHandle(ParentConnId);
	UE_NET_ASSERT_TRUE(ConnHandle.IsValid());
}

UE_NET_TEST_FIXTURE(FConnectionHandleTestFixture, ConnectionHandleReturnsExpectedParentConnectionId)
{
	constexpr uint32 ParentConnId = 4U;
	FConnectionHandle ConnHandle(ParentConnId);
	UE_NET_ASSERT_EQ(ConnHandle.GetParentConnectionId(), ParentConnId);
}

UE_NET_TEST_FIXTURE(FConnectionHandleTestFixture, ConnectionHandleReturnsExpectedChildConnectionId)
{
	constexpr uint32 ParentConnId = 4U;
	constexpr uint32 ChildConnId = 7U;
	FConnectionHandle ConnHandle(ParentConnId, ChildConnId);
	UE_NET_ASSERT_EQ(ConnHandle.GetChildConnectionId(), ChildConnId);
}

UE_NET_TEST_FIXTURE(FConnectionHandleTestFixture, ConnectionHandleWithOnlyValidChildConnectioIdIsInvalid)
{
	constexpr uint32 ParentConnId = UE::Net::InvalidConnectionId;
	constexpr uint32 ChildConnId = 15U;
	FConnectionHandle ConnHandle(ParentConnId, ChildConnId);
	UE_NET_ASSERT_FALSE(ConnHandle.IsValid());
}

}
