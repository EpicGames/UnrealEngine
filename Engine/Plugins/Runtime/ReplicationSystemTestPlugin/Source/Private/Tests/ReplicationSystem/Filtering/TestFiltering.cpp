// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"

#include "UObject/CoreNetTypes.h"

#include "Iris/IrisConstants.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilterDefinitions.h"
#include "Iris/ReplicationSystem/Filtering/ReplicationFilteringConfig.h"
#include "Tests/ReplicationSystem/Filtering/MockNetObjectFilter.h"
#include "Tests/ReplicationSystem/Filtering/TestFilteringObject.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"

namespace UE::Net::Private
{

class FTestFilteringFixture : public FReplicationSystemServerClientTestFixture
{
protected:
	virtual void SetUp() override
	{
		InitNetObjectFilterDefinitions();
		InitObjectScopeHysteresisProfiles();
		FReplicationSystemServerClientTestFixture::SetUp();
		InitFilterHandles();
	}

	virtual void TearDown() override
	{
		FReplicationSystemServerClientTestFixture::TearDown();
		RestoreObjectScopeHysteresisProfiles();
		RestoreFilterDefinitions();
	}

	virtual FName GetMockFilterName() const { return FName("MockFilter"); }
	virtual FName GetMockFilterClassName() const { return FName("/Script/ReplicationSystemTestPlugin.MockNetObjectFilter"); }

	void SetDynamicFilterStatus(ENetFilterStatus FilterStatus)
	{
		UMockNetObjectFilter::FFunctionCallSetup CallSetup;
		CallSetup.AddObject.bReturnValue = true;
		CallSetup.Filter.bFilterOutByDefault = FilterStatus == ENetFilterStatus::Disallow;

		MockNetObjectFilter->SetFunctionCallSetup(CallSetup);
		MockNetObjectFilter->ResetFunctionCallStatus();
	}

	uint32 GetHysteresisFrameCount(const char* ProfileName) const
	{
		const UReplicationFilteringConfig* Config = GetDefault<UReplicationFilteringConfig>();
		if (const FObjectScopeHysteresisProfile* Profile = Config->GetHysteresisProfiles().FindByKey(ProfileName))
		{
			return Profile->HysteresisFrameCount;
		}

		return Config->GetDefaultHysteresisFrameCount();
	}

	class FScopedDefaultHysteresisFrameCount
	{
	public:
		FScopedDefaultHysteresisFrameCount(uint8 DefaultHysteresisFrameCount)
		{
			const UClass* NetObjectFilteringConfiglass = UReplicationFilteringConfig::StaticClass();
			if (const FProperty* Property = NetObjectFilteringConfiglass->FindPropertyByName("DefaultHysteresisFrameCount"))
			{
				UReplicationFilteringConfig* FilteringConfig = GetMutableDefault<UReplicationFilteringConfig>();
				Property->CopyCompleteValue(&PrevValue, (void*)(UPTRINT(FilteringConfig) + Property->GetOffset_ForInternal()));
				Property->CopyCompleteValue((void*)(UPTRINT(FilteringConfig) + Property->GetOffset_ForInternal()), &DefaultHysteresisFrameCount);
				bPrevValueIsValid = true;
			}
		}

		~FScopedDefaultHysteresisFrameCount()
		{
			if (!bPrevValueIsValid)
			{
				return;
			}

			const UClass* NetObjectFilteringConfiglass = UReplicationFilteringConfig::StaticClass();
			if (const FProperty* Property = NetObjectFilteringConfiglass->FindPropertyByName("DefaultHysteresisFrameCount"))
			{
				UReplicationFilteringConfig* FilteringConfig = GetMutableDefault<UReplicationFilteringConfig>();
				Property->CopyCompleteValue((void*)(UPTRINT(FilteringConfig) + Property->GetOffset_ForInternal()), &PrevValue);
				bPrevValueIsValid = true;
			}
		}

		uint8 PrevValue = 0;
		bool bPrevValueIsValid = false;
	};

	class FScopedHysteresisUpdateConnectionThrottling
	{
	public:
		FScopedHysteresisUpdateConnectionThrottling(uint8 HysteresisUpdateConnectionThrottling)
		{
			const UClass* NetObjectFilteringConfiglass = UReplicationFilteringConfig::StaticClass();
			if (const FProperty* Property = NetObjectFilteringConfiglass->FindPropertyByName("HysteresisUpdateConnectionThrottling"))
			{
				UReplicationFilteringConfig* FilteringConfig = GetMutableDefault<UReplicationFilteringConfig>();
				Property->CopyCompleteValue(&PrevValue, (void*)(UPTRINT(FilteringConfig) + Property->GetOffset_ForInternal()));
				Property->CopyCompleteValue((void*)(UPTRINT(FilteringConfig) + Property->GetOffset_ForInternal()), &HysteresisUpdateConnectionThrottling);
				bPrevValueIsValid = true;
			}
		}

		~FScopedHysteresisUpdateConnectionThrottling()
		{
			if (!bPrevValueIsValid)
			{
				return;
			}

			const UClass* NetObjectFilteringConfiglass = UReplicationFilteringConfig::StaticClass();
			if (const FProperty* Property = NetObjectFilteringConfiglass->FindPropertyByName("HysteresisUpdateConnectionThrottling"))
			{
				UReplicationFilteringConfig* FilteringConfig = GetMutableDefault<UReplicationFilteringConfig>();
				Property->CopyCompleteValue((void*)(UPTRINT(FilteringConfig) + Property->GetOffset_ForInternal()), &PrevValue);
				bPrevValueIsValid = true;
			}
		}

		uint8 PrevValue = 0;
		bool bPrevValueIsValid = false;
	};

private:
	void InitNetObjectFilterDefinitions()
	{
		const UClass* NetObjectFilterDefinitionsClass = UNetObjectFilterDefinitions::StaticClass();
		const FProperty* DefinitionsProperty = NetObjectFilterDefinitionsClass->FindPropertyByName("NetObjectFilterDefinitions");
		check(DefinitionsProperty != nullptr);

		// Save CDO state.
		UNetObjectFilterDefinitions* FilterDefinitions = GetMutableDefault<UNetObjectFilterDefinitions>();
		DefinitionsProperty->CopyCompleteValue(&OriginalFilterDefinitions, (void*)(UPTRINT(FilterDefinitions) + DefinitionsProperty->GetOffset_ForInternal()));
		
		// Modify definitions to only include our filters. Ugly... 
		TArray<FNetObjectFilterDefinition> NewFilterDefinitions;
		{
			FNetObjectFilterDefinition& MockDefinition = NewFilterDefinitions.Emplace_GetRef();
			MockDefinition.FilterName = GetMockFilterName();
			MockDefinition.ClassName = GetMockFilterClassName();
			MockDefinition.ConfigClassName = "/Script/ReplicationSystemTestPlugin.MockNetObjectFilterConfig";
		}
		{
			FNetObjectFilterDefinition& NotRoutedDefinition = NewFilterDefinitions.Emplace_GetRef();
			NotRoutedDefinition.FilterName = "NotRouted";
			NotRoutedDefinition.ClassName = "/Script/IrisCore.FilterOutNetObjectFilter";
			NotRoutedDefinition.ConfigClassName = "/Script/IrisCore.NetObjectGridFilterConfig";
		}

		DefinitionsProperty->CopyCompleteValue((void*)(UPTRINT(FilterDefinitions) + DefinitionsProperty->GetOffset_ForInternal()), &NewFilterDefinitions);
	}

	void RestoreFilterDefinitions()
	{
		// Restore CDO state from the saved state.
		const UClass* NetObjectFilterDefinitionsClass = UNetObjectFilterDefinitions::StaticClass();
		const FProperty* DefinitionsProperty = NetObjectFilterDefinitionsClass->FindPropertyByName("NetObjectFilterDefinitions");
		UNetObjectFilterDefinitions* FilterDefinitions = GetMutableDefault<UNetObjectFilterDefinitions>();
		DefinitionsProperty->CopyCompleteValue((void*)(UPTRINT(FilterDefinitions) + DefinitionsProperty->GetOffset_ForInternal()), &OriginalFilterDefinitions);
		OriginalFilterDefinitions.Empty();

		MockNetObjectFilter = nullptr;
		MockFilterHandle = InvalidNetObjectFilterHandle;
		NotRoutedFilterHandle = InvalidNetObjectFilterHandle;
	}

	void InitObjectScopeHysteresisProfiles()
	{
		const UClass* ReplicationFilteringConfigClass = UReplicationFilteringConfig::StaticClass();
		const FProperty* ProfilesProperty = ReplicationFilteringConfigClass->FindPropertyByName("HysteresisProfiles");
		check(ProfilesProperty != nullptr);
		const FProperty* EnableObjectScopeHysteresisProperty = ReplicationFilteringConfigClass->FindPropertyByName("bEnableObjectScopeHysteresis");
		check(EnableObjectScopeHysteresisProperty != nullptr);

		// Save CDO state.
		UReplicationFilteringConfig* FilteringConfig = GetMutableDefault<UReplicationFilteringConfig>();
		ProfilesProperty->CopyCompleteValue(&OriginalObjectScopeHysteresisProfiles, (void*)(UPTRINT(FilteringConfig) + ProfilesProperty->GetOffset_ForInternal()));

		// Modify profiles to what we need
		TArray<FObjectScopeHysteresisProfile> NewProfiles;
		{
			{
				FObjectScopeHysteresisProfile& Profile = NewProfiles.Emplace_GetRef();
				Profile.FilterProfileName = "FiveFrames";
				Profile.HysteresisFrameCount = 5;
			}

			{
				FObjectScopeHysteresisProfile& Profile = NewProfiles.Emplace_GetRef();
				Profile.FilterProfileName = "OneFrame";
				Profile.HysteresisFrameCount = 1;
			}

			{
				FObjectScopeHysteresisProfile& Profile = NewProfiles.Emplace_GetRef();
				Profile.FilterProfileName = "ZeroFrames";
				Profile.HysteresisFrameCount = 0;
			}

			{
				FObjectScopeHysteresisProfile& Profile = NewProfiles.Emplace_GetRef();
				Profile.FilterProfileName = "VeryManyFrames";
				Profile.HysteresisFrameCount = 255;
			}
		}

		ProfilesProperty->CopyCompleteValue((void*)(UPTRINT(FilteringConfig) + ProfilesProperty->GetOffset_ForInternal()), &NewProfiles);

		bOriginalIsObjectScopeHysteresisEnabled = FilteringConfig->IsObjectScopeHysteresisEnabled();
		const bool bEnableObjectScopeHysteresis = true;
		EnableObjectScopeHysteresisProperty->CopyCompleteValue((void*)(UPTRINT(FilteringConfig) + EnableObjectScopeHysteresisProperty->GetOffset_ForInternal()), &bEnableObjectScopeHysteresis);
	}

	void RestoreObjectScopeHysteresisProfiles()
	{
		// Restore CDO state from the saved state.
		const UClass* ReplicationFilteringConfigClass = UReplicationFilteringConfig::StaticClass();
		const FProperty* ProfilesProperty = ReplicationFilteringConfigClass->FindPropertyByName("HysteresisProfiles");
		const FProperty* EnableObjectScopeHysteresisProperty = ReplicationFilteringConfigClass->FindPropertyByName("bEnableObjectScopeHysteresis");

		UReplicationFilteringConfig* FilteringConfig = GetMutableDefault<UReplicationFilteringConfig>();
		ProfilesProperty->CopyCompleteValue((void*)(UPTRINT(FilteringConfig) + ProfilesProperty->GetOffset_ForInternal()), &OriginalObjectScopeHysteresisProfiles);
		OriginalObjectScopeHysteresisProfiles.Empty();
		EnableObjectScopeHysteresisProperty->CopyCompleteValue((void*)(UPTRINT(FilteringConfig) + EnableObjectScopeHysteresisProperty->GetOffset_ForInternal()), &bOriginalIsObjectScopeHysteresisEnabled);
	}

	void InitFilterHandles()
	{
		MockNetObjectFilter = CastChecked<UMockNetObjectFilter>(Server->GetReplicationSystem()->GetFilter(GetMockFilterName()));
		MockFilterHandle = Server->GetReplicationSystem()->GetFilterHandle(GetMockFilterName());
		NotRoutedFilterHandle = Server->GetReplicationSystem()->GetFilterHandle("NotRouted");
	}

protected:
	UMockNetObjectFilter* MockNetObjectFilter = nullptr;
	FNetObjectFilterHandle MockFilterHandle;
	FNetObjectFilterHandle NotRoutedFilterHandle;
	FName ObjectHysteresisProfileName;

private:
	TArray<FNetObjectFilterDefinition> OriginalFilterDefinitions;
	TArray<FObjectScopeHysteresisProfile> OriginalObjectScopeHysteresisProfiles;
	bool bOriginalIsObjectScopeHysteresisEnabled = false;
};

class FTestFilteringWithConditionFixture : public FTestFilteringFixture
{
protected:

	virtual FName GetMockFilterName() const { return FName("MockFilterWithCondition"); }
	virtual FName GetMockFilterClassName() const { return FName("/Script/ReplicationSystemTestPlugin.MockNetObjectFilterWithCondition"); }
};

// Owner filtering tests
UE_NET_TEST_FIXTURE(FTestFilteringFixture, OwnerFilterPreventsObjectFromReplicatingToNonOwner)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

	// Apply owner filter
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, ToOwnerFilterHandle);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Object should not have been created on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	Server->DestroyObject(ServerObject);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, OwnerFilterAllowsObjectToReplicateToOwner)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, ToOwnerFilterHandle);
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Object should have been created on the client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Destroy object on server and client
	Server->DestroyObject(ServerObject);

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, OwnerFilterReplicatesOnlyToOwningConnection)
{
	// Add clients
	FReplicationSystemTestClient* ClientArray[] = {CreateClient(), CreateClient(), CreateClient()};
	constexpr SIZE_T LastClientIndex = UE_ARRAY_COUNT(ClientArray) - 1;

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, ToOwnerFilterHandle);
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, ClientArray[LastClientIndex]->ConnectionIdOnServer);

	// Send and deliver packets
	Server->NetUpdate();
	for (FReplicationSystemTestClient* Client : ClientArray)
	{
		Server->SendAndDeliverTo(Client, DeliverPacket);
	}
	Server->PostSendUpdate();

	// Object should only have been created on the last client
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		const SIZE_T ClientIndex = &Client - &ClientArray[0];
		if (ClientIndex == LastClientIndex)
		{
			UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
		}
		else
		{
			UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
		}
	}

	Server->DestroyObject(ServerObject);
	Server->NetUpdate();
	for (FReplicationSystemTestClient* Client : ClientArray)
	{
		Server->SendAndDeliverTo(Client, DeliverPacket);
	}
	Server->PostSendUpdate();
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, CanChangeOwningConnection)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Object should now exist on client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Turn on owner filter
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, ToOwnerFilterHandle);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// As object is now filtered it should be deleted on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Set the client as owner
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// The object should have been created again
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Finally, remove the owning connection
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, 0);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// The client is no longer owning the object so it should be deleted
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	Server->DestroyObject(ServerObject);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, CanChangeOwnerFilter)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Object should now exist on client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Turn on owner filter
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, ToOwnerFilterHandle);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// As object is now filtered it should be deleted on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Remove the owner filter
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, InvalidNetObjectFilterHandle);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// The object should have been created again
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Destroy the object
	Server->DestroyObject(ServerObject);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, LateAddedSubObjectGetsOwnerPropagated)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	TArray<UReplicatedTestObject*> ServerObjects;
	{
		constexpr SIZE_T ObjectCount = 64;
		ServerObjects.Reserve(ObjectCount);
		for (SIZE_T It = 0, EndIt = ObjectCount; It != EndIt; ++It)
		{
			UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
			ServerObjects.Add(ServerObject);
			Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);
		}
	}

	// Net update
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Create subobject to arbitrary object
	UReplicatedTestObject* ArbitraryServerObject = ServerObjects[3];
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ArbitraryServerObject->NetRefHandle, 1, 1);

	// Net update
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Verify subobject owner is as expected
	UE_NET_ASSERT_EQ(Server->ReplicationSystem->GetOwningNetConnection(ServerSubObject->NetRefHandle), Client->ConnectionIdOnServer);
}

// Owner filtering tests
UE_NET_TEST_FIXTURE(FTestFilteringFixture, GroupFilterPreventsObjectFromReplicating)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle);

	// Filter out objects in group
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Object should not have been created on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, GroupFilterAllowsObjectToReplicate)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle);

	// Filter out objects in group
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Object should not have been created on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Allow replication again
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Object should now have been created on the client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, CanGetDynamicFilter)
{
	UE_NET_ASSERT_NE(MockNetObjectFilter, nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, CanGetDynamicFilterHandle)
{
	UE_NET_ASSERT_NE(MockFilterHandle, InvalidNetObjectFilterHandle);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, DynamicFilterInitIsCalled)
{
	const UMockNetObjectFilter::FFunctionCallStatus& FunctionCallStatus = MockNetObjectFilter->GetFunctionCallStatus();
	UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.Init, 1U);
	UE_NET_ASSERT_EQ(FunctionCallStatus.SuccessfulCallCounts.Init, 1U);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, DynamicFilterAddObjectAndRemoveObjectIsCalledWhenObjectIsDeleted)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	Server->NetUpdate();
	// Filter needs to be set now.
	{
		const UMockNetObjectFilter::FFunctionCallStatus& CallStatus = MockNetObjectFilter->GetFunctionCallStatus();

		UE_NET_ASSERT_EQ(CallStatus.CallCounts.AddObject, 1U);
		UE_NET_ASSERT_EQ(CallStatus.SuccessfulCallCounts.AddObject, 1U);
		UE_NET_ASSERT_EQ(CallStatus.CallCounts.RemoveObject, 0U);

		MockNetObjectFilter->ResetFunctionCallStatus();
	}
	Server->PostSendUpdate();

	Server->DestroyObject(ServerObject);

	// Late join a client and verify that everything works as expected.
	FReplicationSystemTestClient* Client = CreateClient();

	Server->NetUpdate();
	// Filter needs to be cleared now.
	{
		const UMockNetObjectFilter::FFunctionCallStatus& CallStatus = MockNetObjectFilter->GetFunctionCallStatus();

		UE_NET_ASSERT_EQ(CallStatus.CallCounts.RemoveObject, 1U);
		UE_NET_ASSERT_EQ(CallStatus.SuccessfulCallCounts.RemoveObject, 1U);

		MockNetObjectFilter->ResetFunctionCallStatus();
	}
	Server->PostSendUpdate();
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, DynamicFilterCanAllowObjectToReplicate)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server and set filter
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Object should now exist on client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, DynamicFilterCanDisallowObjectToReplicate)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server and set filter
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Object should not exist on client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, SwitchingFiltersCallsRemoveObjectOnPreviousFilter)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Spawn object on server and set filter
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Make sure filter is set
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Check RemoveObject is called when switching filters.
	{
		MockNetObjectFilter->ResetFunctionCallStatus();

		Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, ToOwnerFilterHandle);

		Server->NetUpdate();
		Server->PostSendUpdate();

		const UMockNetObjectFilter::FFunctionCallStatus& FunctionCallStatus = MockNetObjectFilter->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.RemoveObject, 1U);
		UE_NET_ASSERT_EQ(FunctionCallStatus.SuccessfulCallCounts.RemoveObject, 1U);
	}
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, SubObjectsAreReplicatedWhenOwnerDynamicFilterAllows)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server and set filter
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Check that both the object and subobject exist.
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, SubObjectsAreNotReplicatedWhenOwnerDynamicFilterDisallows)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server and set filter
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Check that neither the object nor the subobject exist.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, DependentObjectIsUnaffectedByDynamicFilter)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server and set filter on dependent object
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	UReplicatedTestObject* ServerFutureDependentObject = Server->CreateObject(0, 0);
	Server->ReplicationSystem->SetFilter(ServerFutureDependentObject->NetRefHandle, MockFilterHandle);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// We expect the object to exist and the future dependent object not to exist
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerFutureDependentObject->NetRefHandle), nullptr);

	// Make dependent object and make sure it's now replicated.
	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerFutureDependentObject->NetRefHandle);
	UReplicatedTestObject* ServerDependentObject = ServerFutureDependentObject;

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// We now expect the dependent object to exist
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);

	// Remove dependency and make sure the formerly dependent object is removed from the client
	Server->ReplicationBridge->RemoveDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);
	UReplicatedTestObject* ServerFormerDependentObject = ServerDependentObject;

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// We expect the former dependent object not to exist
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerFormerDependentObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, NestedDependentObjectIsFilteredAsParentsOrIndependent)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server and set filter on dependent objects
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	UReplicatedTestObject* ServerFutureDependentObject = Server->CreateObject(0, 0);
	UReplicatedTestObject* ServerFutureNestedDependentObject = Server->CreateObject(0, 0);

	Server->ReplicationSystem->SetFilter(ServerFutureDependentObject->NetRefHandle, MockFilterHandle);
	Server->ReplicationSystem->SetFilter(ServerFutureNestedDependentObject->NetRefHandle, MockFilterHandle);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// We expect the object to exist and the future dependent and future nested dependent objects not to exist
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerFutureDependentObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerFutureNestedDependentObject->NetRefHandle), nullptr);

	// Make dependent objects and make sure they're now replicated.
	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerFutureDependentObject->NetRefHandle);
	Server->ReplicationBridge->AddDependentObject(ServerFutureDependentObject->NetRefHandle, ServerFutureNestedDependentObject->NetRefHandle);

	UReplicatedTestObject* ServerDependentObject = ServerFutureDependentObject;
	UReplicatedTestObject* ServerNestedDependentObject = ServerFutureNestedDependentObject;

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// We now expect the dependent objects to exist
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerNestedDependentObject->NetRefHandle), nullptr);

	// Remove dependency on root and make sure the formerly dependent object is removed from the client thanks to the filter.
	Server->ReplicationBridge->RemoveDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);
	UReplicatedTestObject* ServerFormerDependentObject = ServerDependentObject;

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// We expect the former dependent object to not to exist, thanks to the filter
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerFormerDependentObject->NetRefHandle), nullptr);

	// As the former dependent object is filtered out it's ok for the nested dependent object to be filtered out.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerNestedDependentObject->NetRefHandle), nullptr);

	// Remove filter on the nested dependent object
	Server->ReplicationSystem->SetFilter(ServerNestedDependentObject->NetRefHandle, InvalidNetObjectFilterHandle);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that dependent object no longer is filtered out even though its parent is
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerFormerDependentObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerNestedDependentObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, DynamicFilteredOutSubObjectsAreResetWhenIndexIsReused)
{
	// Setup dynamic filters for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server and set filter
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Create and destroy subobject
	{
		UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
		Server->DestroyObject(ServerSubObject);
	}

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Object should not exist on client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Create new object which should get the same internal index as the destroyed SubObject
	UReplicatedTestObject* ServerObject2 = Server->CreateObject(0, 0);

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Object should exist on client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject2->NetRefHandle), nullptr);
}


UE_NET_TEST_FIXTURE(FTestFilteringWithConditionFixture, TestCulledDirtyActors)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	const uint32 RepSystemID = Server->GetReplicationSystemId();

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Create multiple filtered objects
	UTestFilteringObject* ServerObjectA = Server->CreateObject<UTestFilteringObject>();
	Server->ReplicationSystem->SetFilter(ServerObjectA->NetRefHandle, MockFilterHandle);

	UTestFilteringObject* ServerObjectB = Server->CreateObject<UTestFilteringObject>();
	Server->ReplicationSystem->SetFilter(ServerObjectB->NetRefHandle, MockFilterHandle);

	UTestFilteringObject* ServerObjectC = Server->CreateObject<UTestFilteringObject>();
	Server->ReplicationSystem->SetFilter(ServerObjectC->NetRefHandle, MockFilterHandle);

	// Create a non-filtered object
	UTestFilteringObject* ServerObjectNoFilter = Server->CreateObject<UTestFilteringObject>();

	// Filter them in
	{	
		constexpr bool bFilterIn = false;
		ServerObjectA->SetFilterOut(bFilterIn);
		ServerObjectB->SetFilterOut(bFilterIn);
		ServerObjectC->SetFilterOut(bFilterIn);

		// Send and deliver packets
		Server->UpdateAndSend({Client});

		// Check that the filtered object do exist on the client.
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectA->NetRefHandle), nullptr);
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectB->NetRefHandle), nullptr);
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectC->NetRefHandle), nullptr);
	}

	// Now filter them out
	{
		constexpr bool bFilterOut = true;
		ServerObjectA->SetFilterOut(bFilterOut);
		ServerObjectB->SetFilterOut(bFilterOut);
		ServerObjectC->SetFilterOut(bFilterOut);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// Check that the filtered object do not exist on the client.
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectA->NetRefHandle), nullptr);
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectB->NetRefHandle), nullptr);
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectC->NetRefHandle), nullptr);
	}

	// Mark objects dirty
	{
		ServerObjectA->ReplicatedCounter = 0x01;
		ServerObjectB->ReplicatedCounter = 0x01;
		ServerObjectC->ReplicatedCounter = 0x01;
		Server->GetReplicationSystem()->MarkDirty(ServerObjectA->NetRefHandle);
		Server->GetReplicationSystem()->MarkDirty(ServerObjectB->NetRefHandle);
		Server->GetReplicationSystem()->MarkDirty(ServerObjectC->NetRefHandle);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// Should still not exist on the client.
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectA->NetRefHandle), nullptr);
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectB->NetRefHandle), nullptr);
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectC->NetRefHandle), nullptr);
	}

	// Put one of them back in the scope
	{
		constexpr bool bFilterIn = false;
		ServerObjectA->SetFilterOut(bFilterIn);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// This one exists
		UTestFilteringObject* ClientObjectA = Cast<UTestFilteringObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectA->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObjectA, nullptr);
		UE_NET_ASSERT_TRUE(ClientObjectA->ReplicatedCounter == 0x01);

		// These don't
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectB->NetRefHandle), nullptr);
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectC->NetRefHandle), nullptr);
	}

	// Add another one back in the scope
	{
		constexpr bool bFilterIn = false;
		ServerObjectB->SetFilterOut(bFilterIn);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// This one exists
		UTestFilteringObject* ClientObjectB = Cast<UTestFilteringObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectB->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObjectB, nullptr);
		UE_NET_ASSERT_TRUE(ClientObjectB->ReplicatedCounter == 0x01);

		// These don't
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectC->NetRefHandle), nullptr);
	}
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, InclusionGroupDoesNotFilterOutObject)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	
	// Setup inclusion group filter
	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(GroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Allow);

	// Send and deliver packets
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Object should have been created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Change filter to not allow replication. As it's an inclusion filter this should not change things.
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Disallow);

	// Send and deliver packets
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Object should not have been destroyed
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, InclusionGroupDoesNotOverrideOwnerFilter)
{
	// Add clients
	constexpr uint32 OwningClientIndex = 0;
	constexpr uint32 NonOwningClientIndex = 1;

	FReplicationSystemTestClient* ClientArray[2];
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		Client = CreateClient();
	}

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, ToOwnerFilterHandle);
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, ClientArray[OwningClientIndex]->ConnectionIdOnServer);

	// Setup inclusion group filter
	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(GroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend(ClientArray, DeliverPacket);

	// Object should have been created on the owning client
	UE_NET_ASSERT_NE(Clients[OwningClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	// Object should not have been created on the non-owning client
	UE_NET_ASSERT_EQ(Clients[NonOwningClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Change filter to not allow replication. As it's an inclusion filter this should not change things.
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Disallow);

	// Send and deliver packets
	Server->UpdateAndSend(ClientArray, DeliverPacket);

	// Object creation status should remain the same as before
	UE_NET_ASSERT_NE(Clients[OwningClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Clients[NonOwningClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, InclusionGroupDoesNotOverrideExclusionGroupFilter)
{
	// Add clients
	constexpr uint32 AllowedClientIndex = 0;
	constexpr uint32 DisallowedOwningClientIndex = 1;

	FReplicationSystemTestClient* ClientArray[2];
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		Client = CreateClient();
	}

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});

	// Setup exclusion group filter
	FNetObjectGroupHandle ExclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(ExclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(ExclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(ExclusionGroupHandle, ClientArray[AllowedClientIndex]->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Setup inclusion group filter
	FNetObjectGroupHandle InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->NetUpdate();
	for (FReplicationSystemTestClient* Client : ClientArray)
	{
		Server->SendAndDeliverTo(Client, DeliverPacket);
	}
	Server->PostSendUpdate();

	// Object should have been created on the allowed client
	UE_NET_ASSERT_NE(Clients[AllowedClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	// Object should not have been created on the disallowed client
	UE_NET_ASSERT_EQ(Clients[DisallowedOwningClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Change inclusion filter to not allow replication. As it's an inclusion filter this should not change things.
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ENetFilterStatus::Disallow);

	// Send and deliver packets
	Server->UpdateAndSend(ClientArray, DeliverPacket);

	// Object creation status should remain the same as before
	UE_NET_ASSERT_NE(Clients[AllowedClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Clients[DisallowedOwningClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, DisabledInclusionGroupDoesNotOverrideDynamicFilter)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add clients
	FReplicationSystemTestClient* ClientArray[2];
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		Client = CreateClient();
	}

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});

	// Set filter which will filter out all objects
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Setup inclusion group filter
	FNetObjectGroupHandle InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ENetFilterStatus::Disallow);

	// Send and deliver packet
	Server->UpdateAndSend(ClientArray, DeliverPacket);

	// Object should not have been created on any clients
	for (FReplicationSystemTestClient* Client : ClientArray)
	{
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	}
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, EnabledInclusionGroupDoesOverrideDynamicFilter)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add clients
	constexpr uint32 AllowedClientIndex = 1;
	constexpr uint32 DisallowedOwningClientIndex = 0;

	FReplicationSystemTestClient* ClientArray[2];
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		Client = CreateClient();
	}

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});

	// Set filter which will filter out all objects
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Setup inclusion group filter
	FNetObjectGroupHandle InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
	// Disallow by default
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ENetFilterStatus::Disallow);
	// Allow one client
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ClientArray[AllowedClientIndex]->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend(ClientArray, DeliverPacket);

	// Object should have been created on the allowed client
	UE_NET_ASSERT_NE(Clients[AllowedClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	// Object should not have been created on the disallowed client
	UE_NET_ASSERT_EQ(Clients[DisallowedOwningClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, LateAddingToEnabledInclusionGroupDoesOverrideDynamicFilter)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add clients
	constexpr uint32 AllowedClientIndex = 1;
	constexpr uint32 DisallowedOwningClientIndex = 0;

	FReplicationSystemTestClient* ClientArray[2];
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		Client = CreateClient();
	}

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});

	// Set filter which will filter out all objects
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Setup inclusion group filter
	FNetObjectGroupHandle InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
	// Disallow by default
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ENetFilterStatus::Disallow);
	// Allow one client
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ClientArray[AllowedClientIndex]->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend(ClientArray, DeliverPacket);

	// At this point the object should not have been created on any client
	for (FReplicationSystemTestClient* Client : ClientArray)
	{
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	}

	// Late adding to group
	Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerObject->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend(ClientArray, DeliverPacket);

	// Object should have been created on the allowed client
	UE_NET_ASSERT_NE(Clients[AllowedClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	// Object should not have been created on the disallowed client
	UE_NET_ASSERT_EQ(Clients[DisallowedOwningClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, LateEnablingInclusionGroupDoesOverrideDynamicFilter)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add clients
	constexpr uint32 AllowedClientIndex = 1;
	constexpr uint32 DisallowedOwningClientIndex = 0;

	FReplicationSystemTestClient* ClientArray[2];
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		Client = CreateClient();
	}

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});

	// Set filter which will filter out all objects
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Setup inclusion group filter
	FNetObjectGroupHandle InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
	// Disallow by default
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ENetFilterStatus::Disallow);

	// Send and deliver packet
	Server->UpdateAndSend(ClientArray, DeliverPacket);

	// At this point the object should not have been created on any client
	for (FReplicationSystemTestClient* Client : ClientArray)
	{
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	}

	// Late enabling client
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ClientArray[AllowedClientIndex]->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend(ClientArray, DeliverPacket);

	// Object should have been created on the allowed client
	UE_NET_ASSERT_NE(Clients[AllowedClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	// Object should not have been created on the disallowed client
	UE_NET_ASSERT_EQ(Clients[DisallowedOwningClientIndex]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, RemovingFromInclusionGroupRemovesDynamicFilterOverride)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});

	// Set filter which will filter out all objects
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Setup inclusion group filter
	FNetObjectGroupHandle InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Object should have been created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Remove object from inclusion group. This should cause the object to be filtered out again.
	Server->ReplicationSystem->RemoveFromGroup(InclusionGroupHandle, ServerObject->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Object should have been created
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, SubObjectAddedToAllowedInclusionGroupFollowsOwnerNotInInclusionGroup)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with subobject on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents{});

	// Set filter which will filter out all objects
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Setup inclusion group filter
	FNetObjectGroupHandle InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerSubObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Neither object nor subobject should have been created on the client.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, SubObjectLateAddedToAllowedInclusionGroupFollowsOwnerNotInInclusionGroup)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with subobject on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents{});

	// Set filter which will filter out all objects
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Setup inclusion group filter
	FNetObjectGroupHandle InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Late add subobject to group
	Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerSubObject->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Neither object nor subobject should have been created on the client.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, LateAddedSubObjectFollowsOwnerInAllowedInclusionGroup)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});

	// Set filter which will filter out all objects
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Setup inclusion group filter and add object
	FNetObjectGroupHandle InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
	Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Object should now have been created on the client.
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Late add subobject to owner
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents{});

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Subobject should now have been created on the client.
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, LateAddedSubObjectFollowsOwnerInDisallowedInclusionGroup)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});

	// Set filter which will filter out all objects
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Setup inclusion group filter and add object
	FNetObjectGroupHandle InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
	Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Object should not have been created on the client.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Late add subobject to owner
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents{});

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Subobject should not have been created on the client.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, SubObjectAddedToDisallowedInclusionGroupFollowsOwnerNotInInclusionGroup)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with subobject on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents{});

	// Set filter which will filter out all objects
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Setup inclusion group filter
	FNetObjectGroupHandle InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerSubObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Both object and subobject should have been created. Disallowing replication of members in an inclusion group does not filter out, not objects nor subobjects.
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, SubObjectRemovedFromAllowedInclusionGroupFollowsOwnerNotInInclusionGroup)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with subobject on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents{});

	// Set filter which will filter out all objects
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Setup inclusion group filter
	FNetObjectGroupHandle InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerSubObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Remove subobject from group
	Server->ReplicationSystem->RemoveFromGroup(InclusionGroupHandle, ServerSubObject->NetRefHandle);
 
	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Neither object nor subobject should have been created on the client.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, SubObjectRemovedFromDisallowedInclusionGroupFollowsOwnerNotInInclusionGroup)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with subobject on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents{});

	// Set filter which will filter out all objects
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Setup inclusion group filter
	FNetObjectGroupHandle InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerSubObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Remove subobject from group
	Server->ReplicationSystem->RemoveFromGroup(InclusionGroupHandle, ServerSubObject->NetRefHandle);
 
	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Both object and subobject should have been created.
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, SubObjectAddedToInclusionGroupFollowsOwnerInOtherInclusionGroupp)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with subobject on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents{});

	// Set filter which will filter out all objects
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Setup separate inclusion group filters for object and subobject
	FNetObjectGroupHandle ObjectInclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(ObjectInclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(ObjectInclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(ObjectInclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);
	
	FNetObjectGroupHandle SubObjectInclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(SubObjectInclusionGroupHandle, ServerSubObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(SubObjectInclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(SubObjectInclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Neither object nor subobject should have been created on the client.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);

	// Allow subobject group to be replicated. This should not change anything.
	Server->ReplicationSystem->SetGroupFilterStatus(SubObjectInclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Neither object nor subobject should have been created on the client.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);

	// Allow object group to be replicated and subobject group to not be replicated. This should result in both the object and subobject being created on the client.
	Server->ReplicationSystem->SetGroupFilterStatus(ObjectInclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);
	Server->ReplicationSystem->SetGroupFilterStatus(SubObjectInclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Both object and subobject should have been created on the client.
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, SubObjectLateAddedToInclusionGroupFollowsOwnerInOtherInclusionGroupp)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with subobject on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents{});

	// Set filter which will filter out all objects
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Setup separate inclusion group filters for object and subobject
	FNetObjectGroupHandle ObjectInclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(ObjectInclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(ObjectInclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(ObjectInclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);
	
	FNetObjectGroupHandle SubObjectInclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddInclusionFilterGroup(SubObjectInclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(SubObjectInclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Late add subobject to group
	Server->ReplicationSystem->AddToGroup(SubObjectInclusionGroupHandle, ServerSubObject->NetRefHandle);
 
	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Neither object nor subobject should have been created on the client.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, SubObjectRemovedFromInclusionGroupFollowsOwnerInOtherInclusionGroup)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with subobject on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents{});

	// Set filter which will filter out all objects
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Setup separate inclusion group filters for object and subobject
	FNetObjectGroupHandle ObjectInclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(ObjectInclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(ObjectInclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(ObjectInclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);
	
	FNetObjectGroupHandle SubObjectInclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(SubObjectInclusionGroupHandle, ServerSubObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(SubObjectInclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(SubObjectInclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Neither object nor subobject should have been created on the client.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);

	// Allow subobject group to be replicated. This should not change anything.
	Server->ReplicationSystem->SetGroupFilterStatus(SubObjectInclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Neither object nor subobject should have been created on the client.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);

	// Remove subobject from group. This should not affect anything.
	Server->ReplicationSystem->RemoveFromGroup(SubObjectInclusionGroupHandle, ServerSubObject->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Neither object nor subobject should have been created on the client.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);

	// Re-add subobject to its group for a second round of tests...
	Server->ReplicationSystem->AddToGroup(SubObjectInclusionGroupHandle, ServerSubObject->NetRefHandle);
	
	// Allow object group to be replicated and subobject group to not be replicated. This should result in both the object and subobject being created on the client.
	Server->ReplicationSystem->SetGroupFilterStatus(ObjectInclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);
	Server->ReplicationSystem->SetGroupFilterStatus(SubObjectInclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Both object and subobject should have been created on the client.
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);

	// Remove subobject from group. This should not affect anything.
	Server->ReplicationSystem->RemoveFromGroup(SubObjectInclusionGroupHandle, ServerSubObject->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Both object and subobject should have been created on the client.
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, InclusionGroupsWorksWithMultipleObjects)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add clients
	FReplicationSystemTestClient* ClientArray[3];
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		Client = CreateClient();
	}

	// Spawn objects on server
	UReplicatedTestObject* ServerObjects[3];
	for (UReplicatedTestObject*& ServerObject : ServerObjects)
	{
		ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
		// Filter out by default
		Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);
	}

	// Setup inclusion group filters
	FNetObjectGroupHandle InclusionGroupHandles[UE_ARRAY_COUNT(ServerObjects)];
	for (FNetObjectGroupHandle& InclusionGroupHandle : InclusionGroupHandles)
	{
		const SIZE_T Index = &InclusionGroupHandle - &InclusionGroupHandles[0];
		InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
		Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
		Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerObjects[Index]->NetRefHandle);
		// Disallow by default
		Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ENetFilterStatus::Disallow);
		// Allow one client
		Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ClientArray[Index]->ConnectionIdOnServer, ENetFilterStatus::Allow);
	}

	// Send and deliver packet
	Server->UpdateAndSend(ClientArray, DeliverPacket);

	// Exactly one object should have been replicated to each
	for (UReplicatedTestObject*& ServerObject : ServerObjects)
	{
		const SIZE_T Index = IntCastChecked<uint32>(&ServerObject - &ServerObjects[0]);
		UE_NET_ASSERT_NE(ClientArray[Index]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
		UE_NET_ASSERT_EQ(ClientArray[(Index + 1U) % UE_ARRAY_COUNT(ServerObjects)]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
		UE_NET_ASSERT_EQ(ClientArray[(Index + 2U) % UE_ARRAY_COUNT(ServerObjects)]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	}
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, InclusionGroupsAreCumulative)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add clients
	FReplicationSystemTestClient* ClientArray[3];
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		Client = CreateClient();
	}

	// Spawn objects on server
	UReplicatedTestObject* ServerObjects[3];
	for (UReplicatedTestObject*& ServerObject : ServerObjects)
	{
		ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
		// Filter out by default
		Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);
	}

	// Setup inclusion group filters
	FNetObjectGroupHandle InclusionGroupHandles[UE_ARRAY_COUNT(ServerObjects)];
	for (FNetObjectGroupHandle& InclusionGroupHandle : InclusionGroupHandles)
	{
		const SIZE_T Index = &InclusionGroupHandle - &InclusionGroupHandles[0];
		InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
		Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
		Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerObjects[Index]->NetRefHandle);
		// Disallow by default
		Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ENetFilterStatus::Disallow);
		// Allow one client
		Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ClientArray[Index]->ConnectionIdOnServer, ENetFilterStatus::Allow);
	}

	// Send and deliver packet
	Server->UpdateAndSend(ClientArray, DeliverPacket);

	// Exactly one object should have been replicated to each connection
	for (UReplicatedTestObject*& ServerObject : ServerObjects)
	{
		const int32 Index = static_cast<int32>(&ServerObject - &ServerObjects[0]);
		UE_NET_ASSERT_NE(Clients[Index]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
		UE_NET_ASSERT_EQ(Clients[(Index + 1U) % UE_ARRAY_COUNT(ServerObjects)]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
		UE_NET_ASSERT_EQ(Clients[(Index + 2U) % UE_ARRAY_COUNT(ServerObjects)]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	}
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, LateAddedConnectionWorksWithSimpleGroupInclusionFilter)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UReplicatedTestObject* ServerObjects[4];
	for (UReplicatedTestObject*& ServerObject : ServerObjects)
	{
		ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
		// Filter out by default
		Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);
	}

	// Setup inclusion group filters differently for each object
	FNetObjectGroupHandle InclusionGroupHandles[UE_ARRAY_COUNT(ServerObjects)];
	for (FNetObjectGroupHandle& InclusionGroupHandle : InclusionGroupHandles)
	{
		const SIZE_T Index = &InclusionGroupHandle - &InclusionGroupHandles[0];
		InclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
		Server->ReplicationSystem->AddInclusionFilterGroup(InclusionGroupHandle);
		Server->ReplicationSystem->AddToGroup(InclusionGroupHandle, ServerObjects[Index]->NetRefHandle);

		switch (Index)
		{
		case 0:
			Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ENetFilterStatus::Disallow);
			break;
		case 1:
			Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ENetFilterStatus::Allow);
			break;
		case 2:
			Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ENetFilterStatus::Disallow);
			Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);
			break;
		case 3:
			Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, ENetFilterStatus::Disallow);
			// For testing purposes we predict the next client ID and allow the object to be replicated to it.
			Server->ReplicationSystem->SetGroupFilterStatus(InclusionGroupHandle, Client->ConnectionIdOnServer + 1, ENetFilterStatus::Allow);
			break;
		};
	}

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Add client
	FReplicationSystemTestClient* LateAddedClient = CreateClient();

	// Send and deliver packet
	Server->UpdateAndSend({Client, LateAddedClient}, DeliverPacket);

	// Verify objects were created or not according to inclusion filters
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[0]->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(LateAddedClient->GetReplicationBridge()->GetReplicatedObject(ServerObjects[0]->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[1]->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(LateAddedClient->GetReplicationBridge()->GetReplicatedObject(ServerObjects[1]->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[2]->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(LateAddedClient->GetReplicationBridge()->GetReplicatedObject(ServerObjects[2]->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[3]->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(LateAddedClient->GetReplicationBridge()->GetReplicatedObject(ServerObjects[3]->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, LateAddedConnectionWorksWithComplexGroupInclusionFilter)
{
	// Setup dynamic filter for the test
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects with one subobject on server
	UReplicatedTestObject* ServerObjects[4];
	UReplicatedTestObject* ServerSubObjects[4];
	for (UReplicatedTestObject*& ServerObject : ServerObjects)
	{
		const SIZE_T Index = &ServerObject - &ServerObjects[0];

		ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
		ServerSubObjects[Index] = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents{});

		// Filter out by default
		Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);
	}

	// Setup inclusion group filters differently for each object, and subobject differently than the root object
	FNetObjectGroupHandle ObjectInclusionGroupHandles[UE_ARRAY_COUNT(ServerObjects)];
	FNetObjectGroupHandle SubObjectInclusionGroupHandles[UE_ARRAY_COUNT(ServerSubObjects)];
	for (FNetObjectGroupHandle& ObjectInclusionGroupHandle : ObjectInclusionGroupHandles)
	{
		const SIZE_T Index = &ObjectInclusionGroupHandle - &ObjectInclusionGroupHandles[0];

		ObjectInclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
		Server->ReplicationSystem->AddInclusionFilterGroup(ObjectInclusionGroupHandle);
		Server->ReplicationSystem->AddToGroup(ObjectInclusionGroupHandle, ServerObjects[Index]->NetRefHandle);

		FNetObjectGroupHandle SubObjectInclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
		Server->ReplicationSystem->AddInclusionFilterGroup(SubObjectInclusionGroupHandle);
		Server->ReplicationSystem->AddToGroup(SubObjectInclusionGroupHandle, ServerSubObjects[Index]->NetRefHandle);

		
		switch (Index)
		{
		case 0:
			Server->ReplicationSystem->SetGroupFilterStatus(ObjectInclusionGroupHandle, ENetFilterStatus::Disallow);
			Server->ReplicationSystem->SetGroupFilterStatus(SubObjectInclusionGroupHandle, ENetFilterStatus::Allow);
			break;
		case 1:
			Server->ReplicationSystem->SetGroupFilterStatus(ObjectInclusionGroupHandle, ENetFilterStatus::Allow);
			Server->ReplicationSystem->SetGroupFilterStatus(SubObjectInclusionGroupHandle, ENetFilterStatus::Disallow);
			break;
		case 2:
			Server->ReplicationSystem->SetGroupFilterStatus(ObjectInclusionGroupHandle, ENetFilterStatus::Disallow);
			Server->ReplicationSystem->SetGroupFilterStatus(ObjectInclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

			Server->ReplicationSystem->SetGroupFilterStatus(SubObjectInclusionGroupHandle, ENetFilterStatus::Allow);
			Server->ReplicationSystem->SetGroupFilterStatus(SubObjectInclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);
			break;
		case 3:
			Server->ReplicationSystem->SetGroupFilterStatus(ObjectInclusionGroupHandle, ENetFilterStatus::Disallow);
			// For testing purposes we predict the next client ID and allow the object to be replicated to it.
			Server->ReplicationSystem->SetGroupFilterStatus(ObjectInclusionGroupHandle, Client->ConnectionIdOnServer + 1, ENetFilterStatus::Allow);

			Server->ReplicationSystem->SetGroupFilterStatus(SubObjectInclusionGroupHandle, ENetFilterStatus::Allow);
			Server->ReplicationSystem->SetGroupFilterStatus(SubObjectInclusionGroupHandle, Client->ConnectionIdOnServer + 1, ENetFilterStatus::Disallow);
			break;
		};
	}

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Add client
	FReplicationSystemTestClient* LateAddedClient = CreateClient();

	// Send and deliver packet
	Server->UpdateAndSend({Client, LateAddedClient}, DeliverPacket);

	// Verify objects were created or not according to inclusion filters
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[0]->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(LateAddedClient->GetReplicationBridge()->GetReplicatedObject(ServerObjects[0]->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObjects[0]->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(LateAddedClient->GetReplicationBridge()->GetReplicatedObject(ServerSubObjects[0]->NetRefHandle), nullptr);

	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[1]->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(LateAddedClient->GetReplicationBridge()->GetReplicatedObject(ServerObjects[1]->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObjects[1]->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(LateAddedClient->GetReplicationBridge()->GetReplicatedObject(ServerSubObjects[1]->NetRefHandle), nullptr);
	
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[2]->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(LateAddedClient->GetReplicationBridge()->GetReplicatedObject(ServerObjects[2]->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObjects[2]->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(LateAddedClient->GetReplicationBridge()->GetReplicatedObject(ServerSubObjects[2]->NetRefHandle), nullptr);
	
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[3]->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(LateAddedClient->GetReplicationBridge()->GetReplicatedObject(ServerObjects[3]->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObjects[3]->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(LateAddedClient->GetReplicationBridge()->GetReplicatedObject(ServerSubObjects[3]->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisIsEnabled)
{
	UE_NET_ASSERT_TRUE_MSG(GetDefault<UReplicationFilteringConfig>()->IsObjectScopeHysteresisEnabled(), "Error: Hysteresis is disabled. All hysteresis tests will fail.");
}

// Dynamic filtering should cause hysteresis to kick in for a filtered out object with a filter profile.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisKicksInForDynamicallyFilteredOutObjectWithFilterProfile)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	// Spawn object on server and set filter and filter profile for hysteresis
	UReplicatedTestObject* ServerObject = Server->CreateObject({.IrisComponentCount = 0});
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle, /* .FilterProfile = */ "FiveFrames");

	Server->UpdateAndSend({ Client });

	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	const uint32 HysteresisFrameCount = GetHysteresisFrameCount("FiveFrames");
	UE_NET_ASSERT_EQ(HysteresisFrameCount, 5U);
	for (uint32 It = 0, EndIt = HysteresisFrameCount; It < EndIt; ++It)
	{
		Server->UpdateAndSend({ Client });
	}

	// At this point the object should still exist on the client due to hysteresis
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	Server->UpdateAndSend({ Client });

	// Hysteresis frame count has passed. The object shuld now be destroyed.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

// Dynamic filtering should cause hysteresis to kick in for a filtered out object without a filter profile, thus using default hysteresis frame count.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisKicksInForDynamicallyFilteredOutObjectWithoutFilterProfile)
{
	constexpr uint32 DefaultHysteresisFrameCount = 3;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(DefaultHysteresisFrameCount);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	// Spawn object on server and set filter and filter profile for hysteresis
	UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	Server->UpdateAndSend({ Client });

	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	for (uint32 It = 0, EndIt = DefaultHysteresisFrameCount; It < EndIt; ++It)
	{
		Server->UpdateAndSend({ Client });
	}

	// At this point the object should still exist on the client due to hysteresis
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	Server->UpdateAndSend({ Client });

	// Hysteresis frame count has passed. The object shuld now be destroyed.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

// Owner filtering changes should not cause hysteresis to kick in.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisDoesNotKickInForOwnerFilteredObject)
{
	constexpr uint32 DefaultHysteresisFrameCount = 3;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(DefaultHysteresisFrameCount);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, ToOwnerFilterHandle);
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Object should have been created on the client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Switch owner and make sure objects gets immediately destroyed on the client
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, InvalidConnectionId);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Owner filtering should not cause hysteresis to kick in so the owner change should cause the client object to be destroyed immediately.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

// Exclusion group filtering changes should not cause hysteresis to kick in.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisDoesNotKickInForExclusionGroupFilteredObject)
{
	constexpr uint32 DefaultHysteresisFrameCount = 3;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(DefaultHysteresisFrameCount);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });

	// Add to exclusion group that allows replication to all connections.
	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Object should have been created on the client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Disallow the group to be replicated.
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Disallow);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Exclusion group filtering changes should not cause hysteresis to kick in so the client object to be destroyed immediately when the group disallows replication.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

// If an object is filtered out from the start we should not start replicating it at all
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisDoesNotKickInForNewlyCreatedObject)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Spawn object on server and set filter and filter profile for hysteresis
	UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle, /* .FilterProfile = */ "FiveFrames");

	Server->UpdateAndSend({ Client });

	// The object was filtered out from the start so hysteresis should not cause it to start replicating.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

// Destroyed objects are expected to be destroyed as quickly as possible.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisDoesNotKickInForDestroyedObject)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	// Spawn object on server and set filter and filter profile for hysteresis
	UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle, /* .FilterProfile = */ "FiveFrames");

	Server->UpdateAndSend({ Client });

	// Destroy object and make sure it immediately gets destroyed on the client as well.
	const FNetRefHandle ServerNetRefHandle = ServerObject->NetRefHandle;
	Server->DestroyObject(ServerObject);
	ServerObject = nullptr;

	Server->UpdateAndSend({ Client });

	// The object was destroyed on the server and should be destroyed as soon as possible on the client as well.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerNetRefHandle), nullptr);
}

// Test case where dependent objects are filtered out yet should be replicated due to their parent being in scope.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisDoesNotKickInForDependentObjectWithReplicatedParent)
{
	constexpr uint32 DefaultHysteresisFrameCount = 2;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(DefaultHysteresisFrameCount);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Only the dependent object will have a filter set in this test. Filter out by default.
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
	UReplicatedTestObject* ServerDependentObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, MockFilterHandle);

	Server->UpdateAndSend({ Client });

	// Add the dependency
	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	Server->UpdateAndSend({ Client });

	// Make sure the dependent object has been created 
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);

	// Perform send update for a few frames and make sure the dependent object stays relevant.
	for (uint32 It = 0; It != DefaultHysteresisFrameCount; ++It)
	{
		Server->UpdateAndSend({ Client });
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);
	}
}

// Test case where dependent objects are filtered out yet should be replicated due to their parent being in scope.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisKicksInForDependentObjectWhenParentIsFilteredOut)
{
	constexpr uint32 DefaultHysteresisFrameCount = 3;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(DefaultHysteresisFrameCount);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Both objects have a filter set in this test
	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle, /* .FilterProfile = */ "ZeroFrames");

	UReplicatedTestObject* ServerDependentObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, MockFilterHandle);

	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	Server->UpdateAndSend({ Client });

	// Make sure tall objects have been created 
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);

	// Filter out both objects.
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Perform send update for a few frames and make sure the dependent object stays relevant.
	for (uint32 It = 0; It != DefaultHysteresisFrameCount; ++It)
	{
		Server->UpdateAndSend({ Client });
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);
	}

	// Eventually the dependent object should be filtered out
	Server->UpdateAndSend({ Client });
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);

	// Make sure it stays filtered out
	for (uint32 It = 0; It != DefaultHysteresisFrameCount + 1; ++It)
	{
		Server->UpdateAndSend({ Client });
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);
	}
}

// Verify dependent object is replicated after parent is filtered out and ends up in hysteresis
UE_NET_TEST_FIXTURE(FTestFilteringFixture, DependentObjectIsReplicatedWhenParentIsInHysteresis)
{
	constexpr uint32 DefaultHysteresisFrameCount = 3;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(DefaultHysteresisFrameCount);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Both objects have a filter set in this test
	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	UReplicatedTestObject* ServerDependentObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, MockFilterHandle, /* .FilterProfile = */ "ZeroFrames");

	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	Server->UpdateAndSend({ Client });

	// Make sure tall objects have been created 
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);

	// Filter out both objects.
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Perform send update for a few frames and make sure both objects stays relevant.
	for (uint32 It = 0; It != DefaultHysteresisFrameCount; ++It)
	{
		Server->UpdateAndSend({ Client });
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);
	}

	// Eventually the both objects should be filtered out
	Server->UpdateAndSend({ Client });
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);

	// Make sure they stay filtered out
	for (uint32 It = 0; It != DefaultHysteresisFrameCount + 1; ++It)
	{
		Server->UpdateAndSend({ Client });
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);
	}
}

// Verify dependent object with hysteresis is replicated as long as the parent object with hysteresis is.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, DependentObjectWithHysteresisIsReplicatedWhenParentIsInHysteresis)
{
	constexpr uint32 TwoFrameHysteresis = 2;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(TwoFrameHysteresis);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Both objects have a filter set in this test
	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	UReplicatedTestObject* ServerDependentObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, MockFilterHandle, /* .FilterProfile = */ "FiveFrames");

	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	Server->UpdateAndSend({ Client });

	// Make sure tall objects have been created 
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);

	// Filter out both objects.
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Perform send update for a few frames and make sure both objects stays relevant.
	for (uint32 It = 0; It != TwoFrameHysteresis; ++It)
	{
		Server->UpdateAndSend({ Client });
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);
	}

	// Eventually the parent object should be filtered out
	Server->UpdateAndSend({ Client });
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// At this point the dependent object should still exist as 5 > 2
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);

	// Make sure dependent object sticks around for a bit
	for (uint32 It = 0; It != 5 - TwoFrameHysteresis - 1; ++It)
	{
		Server->UpdateAndSend({ Client });
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);
	}

	// We don't know exactly when hysteresis kicks in, if it starts when parent hysteresis starts or when parent is finally filtered out.
	// BUT at the very least it should be filtered out after an additional five frames have passed.
	for (uint32 It = 0; It != 5; ++It)
	{
		Server->UpdateAndSend({ Client });
	}
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);
}

// Test case where dependent objects are filtered out yet should be replicated due to their parent being in scope. Dependency is then removed and hysteresis should kick in.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisKicksInForFilteredOutFormerlyDependentObject)
{
	constexpr uint32 DefaultHysteresisFrameCount = 2;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(DefaultHysteresisFrameCount);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Only the dependent object will have a filter set in this test. Filter out by default.
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
	UReplicatedTestObject* ServerDependentObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, MockFilterHandle);

	Server->UpdateAndSend({ Client });

	// Add the dependency
	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	Server->UpdateAndSend({ Client });

	// Make sure the dependent object has been created 
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);

	// Remove dependency
	Server->ReplicationBridge->RemoveDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	// Perform send update for a few frames and make sure the dependent object stays relevant.
	for (uint32 It = 0; It != DefaultHysteresisFrameCount; ++It)
	{
		Server->UpdateAndSend({ Client });
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);
	}

	// The formerly dependent object should now be filtered out
	Server->UpdateAndSend({ Client });
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);
}

// Test case where an object is replicated, then filtered out and filtered in prior to hysteresis frame timeout.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, FilteringChangesDoesNotCauseHysteresisToFilterOutObject)
{
	constexpr uint32 DefaultHysteresisFrameCount = 3;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(DefaultHysteresisFrameCount);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	Server->UpdateAndSend({ Client });

	// Make sure the object has been created 
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Filter out object and perform update. Due to hysteresis the object should remain replicated.
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	Server->UpdateAndSend({ Client });

	// Hysteresis should cause the object to remain replicated.
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Filter in the object again and make sure it stays replicated.
	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	// Perform send update for a few frames and make sure the dependent object stays relevant.
	for (uint32 It = 0; It != DefaultHysteresisFrameCount; ++It)
	{
		Server->UpdateAndSend({ Client });
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	}
}

// Test case where inclusion group added objects are filtered out yet should be replicated due to the inclusion group allowing replication. Inclusion group then disallows replication causing hysteresis to kick in.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisKicksInForFormerlyInclusionGroupAllowedObjectWhenFilterDisallowsReplication)
{
	constexpr uint32 DefaultHysteresisFrameCount = 2;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(DefaultHysteresisFrameCount);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Add object to filter which disallows replication but inclusion group that allows it.
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(GroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Allow);

	Server->UpdateAndSend({ Client });

	// Make sure the object has been created 
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Disallow replication of inclusion group
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Disallow);

	// Perform send update for the hysteresis duration and make sure the object stays replicated.
	for (uint32 It = 0; It != DefaultHysteresisFrameCount; ++It)
	{
		Server->UpdateAndSend({ Client });
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	}

	// Now the object should be filtered out
	Server->UpdateAndSend({ Client });
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

// Variant of above test. Test case where inclusion group added objects are filtered out yet should be replicated due to the inclusion group allowing replication. Object is then removed from inclusion group causing hysteresis to kick in.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisKicksInForFormerInclusionGroupMemberWhenFilterDisallowsReplication)
{
	constexpr uint32 DefaultHysteresisFrameCount = 2;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(DefaultHysteresisFrameCount);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Add object to filter which disallows replication but inclusion group that allows it.
	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(GroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Allow);

	Server->UpdateAndSend({ Client });

	// Make sure the object has been created 
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Remove object from inclusion group
	Server->ReplicationSystem->RemoveFromGroup(GroupHandle, ServerObject->NetRefHandle);

	// Perform send update for the hysteresis duration and make sure the object stays replicated.
	for (uint32 It = 0; It != DefaultHysteresisFrameCount; ++It)
	{
		Server->UpdateAndSend({ Client });
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	}

	// Now the object should be filtered out
	Server->UpdateAndSend({ Client });
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

// Test case where inclusion group added objects are allowed to be replicated by dyanmic filter too. Inclusion group then disallows replication which should not cause hysteresis to kick in.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisDoesNotKickInForFormerlyInclusionGroupAllowedObjectWhenFilterAllowsReplication)
{
	constexpr uint32 DefaultHysteresisFrameCount = 2;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(DefaultHysteresisFrameCount);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Add object to filter which allows replication.
	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(GroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Allow);

	Server->UpdateAndSend({ Client });

	// Make sure the object has been created 
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Disallow replication of inclusion group
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Disallow);

	// Perform send update for the hysteresis duration and make sure the object stays replicated.
	for (uint32 It = 0; It != DefaultHysteresisFrameCount; ++It)
	{
		Server->UpdateAndSend({ Client });
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	}

	// The object should stay replicated
	Server->UpdateAndSend({ Client });
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

// Variant of above test. Test case where inclusion group added objects are allowed to be replicated by dynamic filter too. Object is then removed from inclusion group which should not cause hysteresis to kick in.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisDoesNotKickInForFormerInclusionGroupMemberWhenFilterAllowsReplication)
{
	constexpr uint32 DefaultHysteresisFrameCount = 2;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(DefaultHysteresisFrameCount);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Add object to filter which allows replication.
	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddInclusionFilterGroup(GroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Allow);

	Server->UpdateAndSend({ Client });

	// Make sure the object has been created 
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Remove object from inclusion group
	Server->ReplicationSystem->RemoveFromGroup(GroupHandle, ServerObject->NetRefHandle);

	// Perform send update for the hysteresis duration and make sure the object stays replicated.
	for (uint32 It = 0; It != DefaultHysteresisFrameCount; ++It)
	{
		Server->UpdateAndSend({ Client });
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	}

	// The object should stay replicated
	Server->UpdateAndSend({ Client });
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

// Make sure that connection throttling does not cause objects to be filtered out too soon. Also verify throttling occurs.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisConnectionThrottlingWorksAsExpected)
{
	constexpr uint8 ConnectionThrottlingFrameCount = 5;
	FScopedHysteresisUpdateConnectionThrottling ConnectionThrottling(ConnectionThrottlingFrameCount);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Figure out hysteresis update frame.
	{
		SetDynamicFilterStatus(ENetFilterStatus::Allow);

		// Spawn object on server and set filter and filter profile for hysteresis
		UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
		Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle, /* .FilterProfile = */ "OneFrame");

		Server->UpdateAndSend({ Client });

		SetDynamicFilterStatus(ENetFilterStatus::Disallow);

		// As we have an hysteresis of one frame we will detect immediately when the trottling is updated
		for (uint32 It = 0, EndIt = ConnectionThrottlingFrameCount; It < EndIt; ++It)
		{
			Server->UpdateAndSend({ Client });
			if (Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle) == nullptr)
			{
				break;
			}
		}

		// Object must have been destroyed on the client by now.
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	}

	// Make sure object is kept alive for at least the expected frame count
	{
		SetDynamicFilterStatus(ENetFilterStatus::Allow);

		UReplicatedTestObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
		Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle, /* .FilterProfile = */ "FiveFrames");

		// Advance up to one frame before we expect hysteresis update
		for (uint32 It = 0, EndIt = ConnectionThrottlingFrameCount - 1; It < EndIt; ++It)
		{
			Server->UpdateAndSend({ Client });
		}

		// Filter out object. Hysteresis update should be performed but the object should not be filtered out immediately as not enough frames have passed.
		SetDynamicFilterStatus(ENetFilterStatus::Disallow);

		uint32 WaitFrameCount = 0;
		for (uint32 It = 0, EndIt = 2U * ConnectionThrottlingFrameCount; It < EndIt; ++It)
		{
			++WaitFrameCount;
			Server->UpdateAndSend({ Client });

			if (Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle) == nullptr)
			{
				break;
			}
		}

		// Object must have been destroyed on the client by now.
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

		// FiveFames profile means at least five frames of waiting
		UE_NET_ASSERT_GT(WaitFrameCount, 5U);

		// This assert assumes ConnectionThrottlingFrameCount is five as well. If it's four we'd expect 4+4+1 frames.
		UE_NET_ASSERT_EQ(WaitFrameCount, ConnectionThrottlingFrameCount + 1U);
	}
}

// Test that a lot of objects can be filtered out on the same frame.
UE_NET_TEST_FIXTURE(FTestFilteringFixture, LotsOfObjectsCanBeFilteredOutViaHysteresisInOneFrame)
{
	constexpr uint32 HighObjectCount = 65;

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	SetDynamicFilterStatus(ENetFilterStatus::Allow);

	// Spawn lots of objects on server
	UReplicatedTestObject* ServerObjects[HighObjectCount];
	for (UReplicatedTestObject*& ServerObject : ServerObjects)
	{
		ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
		Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle, /* .FilterProfile = */ "OneFrame");
	}

	// Send and deliver packets until we believe all objects have been created on the client
	bool bAllObjectsCreated = false;
	for (uint32 It = 0; It < HighObjectCount; ++It)
	{
		Server->UpdateAndSend({ Client });
		if (Client->IsResolvableNetRefHandle(ServerObjects[HighObjectCount - 1]->NetRefHandle))
		{
			bAllObjectsCreated = true;

			for (const UReplicatedTestObject* ServerObject : ServerObjects)
			{
				if (!Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle))
				{
					bAllObjectsCreated = false;
					break;
				}
			}
			
			if (bAllObjectsCreated)
			{
				break;
			}
		}
	}

	UE_NET_ASSERT_TRUE(bAllObjectsCreated);

	SetDynamicFilterStatus(ENetFilterStatus::Disallow);

	// Send and deliver packet. Need to update a couple of times to have the objects filtered out.
	Server->UpdateAndSend({ Client });
	for (uint32 It = 0; It < HighObjectCount; ++It)
	{
		if (!Server->UpdateAndSend({ Client }))
		{
			break;
		}
	}

	// All client objects should be destroyed
	for (const UReplicatedTestObject* ServerObject : ServerObjects)
	{
		UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerObject->NetRefHandle));
	}
}

// Test that a dynamically filtered out object doesn't enter hysteresis for late joining client 
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisDoesNotAffectDynamicallyFilteredOutObjectForLateJoiningClient)
{
	constexpr uint32 DefaultHysteresisFrameCount = 3;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(DefaultHysteresisFrameCount);

	// Create object with filter which disallows replication.
	UReplicatedTestObject* ServerObject = Server->CreateObject();
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, NotRoutedFilterHandle);

	Server->UpdateAndSend({});

	// Add late joining client
	FReplicationSystemTestClient* Client = CreateClient();

	// Perform send update for more than the hysteresis duration and make sure the object never gets created.
	for (uint32 It = 0; It < DefaultHysteresisFrameCount + 1U; ++It)
	{
		Server->UpdateAndSend({ Client });
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	}
}

// Test that a dynamically filtered out dependent objects whose parent is not in scope doesn't enter hysteresis for late joining client 
UE_NET_TEST_FIXTURE(FTestFilteringFixture, HysteresisDoesNotAffectDynamicallyFilteredOutDependentObjectForLateJoiningClient)
{
	constexpr uint32 DefaultHysteresisFrameCount = 3;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(DefaultHysteresisFrameCount);

	// Create object with filter which disallows replication.
	UReplicatedTestObject* ServerObject = Server->CreateObject();
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, NotRoutedFilterHandle);

	// Create dependent object with filter which disallows replication.
	UReplicatedTestObject* ServerDependentObject = Server->CreateObject();
	Server->ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, NotRoutedFilterHandle);

	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	Server->UpdateAndSend({});

	// Add late joining client
	FReplicationSystemTestClient* Client = CreateClient();

	// Perform send update for more than the hysteresis duration and make sure the objects never get created.
	for (uint32 It = 0; It < DefaultHysteresisFrameCount + 1U; ++It)
	{
		Server->UpdateAndSend({ Client });
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle), nullptr);
	}
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, TestDependentObjectIsFilteredOutTogetherWithExclusionFilteredOutParent)
{
	constexpr uint32 DefaultHysteresisFrameCount = 2;
	FScopedDefaultHysteresisFrameCount ScopedDefaultHysteresisFrameCount(DefaultHysteresisFrameCount);

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	UTestReplicatedIrisObject* ServerDependentObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});

	// Add parent object to MockFilter so we can filter in/out as desired.
	SetDynamicFilterStatus(ENetFilterStatus::Allow);
	ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Dynamically filter out dependent object
	ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, NotRoutedFilterHandle);

	Bridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	// Send data
	Server->UpdateAndSend({Client});

	// Verify that objects have been created on the client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerDependentObject->NetRefHandle));

	// Throw in some filtering groups into the mix
	FNetObjectGroupHandle ExclusionFilterGroup = ReplicationSystem->CreateGroup(NAME_None);
	ReplicationSystem->AddExclusionFilterGroup(ExclusionFilterGroup);
	FNetObjectGroupHandle InclusionFilterGroup = ReplicationSystem->CreateGroup(NAME_None);
	ReplicationSystem->AddInclusionFilterGroup(InclusionFilterGroup);

	// Allow replication of the InclusionGroup to the client
	ReplicationSystem->SetGroupFilterStatus(InclusionFilterGroup, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Add parent to groups
	ReplicationSystem->AddToGroup(ExclusionFilterGroup, ServerObject->NetRefHandle);
	ReplicationSystem->AddToGroup(InclusionFilterGroup, ServerObject->NetRefHandle);

	// Send data
	Server->UpdateAndSend({Client});

	// Allow replication of the ExclusionGroup to the client
	ReplicationSystem->SetGroupFilterStatus(ExclusionFilterGroup, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Send data
	Server->UpdateAndSend({Client});

	// Verify that the objects exist on the client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerDependentObject->NetRefHandle));

	// Disallow replication of the ExclusionGroup to the client
	ReplicationSystem->SetGroupFilterStatus(ExclusionFilterGroup, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	// Perform send update for the hysteresis duration
	for (uint32 It = 0; It != DefaultHysteresisFrameCount + 1; ++It)
	{
		Server->UpdateAndSend({ Client });
	}

	// Verify that the objects are no longer valid on the client
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerDependentObject->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, DependentObjectAddedToExclusionFilteredOutObjectIsNotInScope)
{
	FReplicationSystemTestClient* ClientArray[] = { CreateClient(), CreateClient() };

	UReplicatedTestObject* ServerObject = Server->CreateObject();

	UReplicatedTestObject* ServerDependentObject1 = Server->CreateObject();
	Server->ReplicationSystem->SetFilter(ServerDependentObject1->NetRefHandle, NotRoutedFilterHandle, /* .FilterConfigProfile */ "VeryManyFrames");

	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject1->NetRefHandle);

	Server->UpdateAndSend(ClientArray);

	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerDependentObject1->NetRefHandle));

	UE_NET_ASSERT_TRUE(ClientArray[1]->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientArray[1]->IsResolvableNetRefHandle(ServerDependentObject1->NetRefHandle));

	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle);

	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Allow);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ClientArray[1]->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	Server->UpdateAndSend(ClientArray);

	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerDependentObject1->NetRefHandle));

	UE_NET_ASSERT_FALSE(ClientArray[1]->IsValidNetRefHandle(ServerObject->NetRefHandle));
	// The dependent object was in scope for a while and should be subject to hysteresis, thus would be resolvable even for the client for which the dependant was just filtered out
	UE_NET_ASSERT_TRUE(ClientArray[1]->IsValidNetRefHandle(ServerDependentObject1->NetRefHandle));

	UReplicatedTestObject* ServerDependentObject2 = Server->CreateObject();
	Server->ReplicationSystem->SetFilter(ServerDependentObject2->NetRefHandle, NotRoutedFilterHandle, /* .FilterConfigProfile */ "VeryManyFrames");

	Server->ReplicationBridge->RemoveDependentObject(ServerObject->NetRefHandle, ServerDependentObject1->NetRefHandle);
	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject2->NetRefHandle);

	Server->UpdateAndSend(ClientArray);

	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerDependentObject1->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerDependentObject2->NetRefHandle));

	UE_NET_ASSERT_FALSE(ClientArray[1]->IsValidNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientArray[1]->IsResolvableNetRefHandle(ServerDependentObject1->NetRefHandle));
	// The ServerDependentObject2 was never in scope for ClientArray[1] and isn't expected to be now that is was just added.
	UE_NET_ASSERT_FALSE(ClientArray[1]->IsValidNetRefHandle(ServerDependentObject2->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, AlreadyCreatedObjectAddedAsDependencyToExclusionFilteredOutObjectIsNotInScope)
{
	FReplicationSystemTestClient* ClientArray[] = { CreateClient(), CreateClient() };

	UReplicatedTestObject* ServerObject = Server->CreateObject();

	UReplicatedTestObject* ServerDependentObject1 = Server->CreateObject();
	UReplicatedTestObject* ServerDependentObject2 = Server->CreateObject();
	Server->ReplicationSystem->SetFilter(ServerDependentObject1->NetRefHandle, NotRoutedFilterHandle, /* .FilterConfigProfile */ "VeryManyFrames");
	Server->ReplicationSystem->SetFilter(ServerDependentObject2->NetRefHandle, NotRoutedFilterHandle, /* .FilterConfigProfile */ "VeryManyFrames");

	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject1->NetRefHandle);

	Server->UpdateAndSend(ClientArray);

	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerDependentObject1->NetRefHandle));

	UE_NET_ASSERT_TRUE(ClientArray[1]->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientArray[1]->IsResolvableNetRefHandle(ServerDependentObject1->NetRefHandle));

	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle);

	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Allow);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ClientArray[1]->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	Server->UpdateAndSend(ClientArray);

	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerDependentObject1->NetRefHandle));

	UE_NET_ASSERT_FALSE(ClientArray[1]->IsValidNetRefHandle(ServerObject->NetRefHandle));
	// The dependent object was in scope for a while and should be subject to hysteresis, thus would be resolvable even for the client for which the dependant was just filtered out
	UE_NET_ASSERT_TRUE(ClientArray[1]->IsValidNetRefHandle(ServerDependentObject1->NetRefHandle));

	Server->ReplicationBridge->RemoveDependentObject(ServerObject->NetRefHandle, ServerDependentObject1->NetRefHandle);
	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject2->NetRefHandle);

	Server->UpdateAndSend(ClientArray);

	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerDependentObject1->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerDependentObject2->NetRefHandle));

	UE_NET_ASSERT_FALSE(ClientArray[1]->IsValidNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientArray[1]->IsResolvableNetRefHandle(ServerDependentObject1->NetRefHandle));
	// The ServerDependentObject2 was never in scope for ClientArray[1] and isn't expected to be now that is was just added.
	UE_NET_ASSERT_FALSE(ClientArray[1]->IsValidNetRefHandle(ServerDependentObject2->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, DependentObjectWithHysteresisAddedToExclusionGroupObjectIsNotInScope)
{
	FReplicationSystemTestClient* ClientArray[] = { CreateClient(), CreateClient() };

	UReplicatedTestObject* ServerObject = Server->CreateObject();

	Server->UpdateAndSend(ClientArray);

	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientArray[1]->IsResolvableNetRefHandle(ServerObject->NetRefHandle));

	UReplicatedTestObject* ServerDependentObject = Server->CreateObject();
	Server->ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, NotRoutedFilterHandle, /* .FilterConfigProfile */ "VeryManyFrames");

	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	Server->UpdateAndSend(ClientArray);

	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerDependentObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle);

	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Allow);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ClientArray[1]->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	Server->UpdateAndSend(ClientArray);

	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerDependentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(ClientArray[1]->IsValidNetRefHandle(ServerDependentObject->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, DependentObjectWithHysteresisAddedToExclusionGroupThenAllowedIsNotFilteredOutByHysteresis)
{
	FReplicationSystemTestClient* ClientArray[] = { CreateClient(), CreateClient() };

	UReplicatedTestObject* ServerObject = Server->CreateObject();

	Server->UpdateAndSend(ClientArray);

	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientArray[1]->IsResolvableNetRefHandle(ServerObject->NetRefHandle));

	UReplicatedTestObject* ServerDependentObject = Server->CreateObject();
	Server->ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, NotRoutedFilterHandle, /* .FilterConfigProfile */ "FiveFrames");

	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	Server->UpdateAndSend(ClientArray);

	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerDependentObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle);

	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Allow);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ClientArray[1]->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	Server->UpdateAndSend(ClientArray);

	UE_NET_ASSERT_TRUE(ClientArray[0]->IsResolvableNetRefHandle(ServerDependentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(ClientArray[1]->IsValidNetRefHandle(ServerDependentObject->NetRefHandle));

	// Allow dependent object again
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ClientArray[1]->ConnectionIdOnServer, ENetFilterStatus::Allow);

	Server->UpdateAndSend(ClientArray);

	// Dependent object should now have been replicated again
	UE_NET_ASSERT_TRUE(ClientArray[1]->IsResolvableNetRefHandle(ServerDependentObject->NetRefHandle));

	// Update for the expected hysteresis duration and make sure the dependent object never goes out of scope.
	for (unsigned FrameIt = 0; FrameIt < 6U; ++FrameIt)
	{
		Server->UpdateAndSend(ClientArray);
		UE_NET_ASSERT_TRUE(ClientArray[1]->IsResolvableNetRefHandle(ServerDependentObject->NetRefHandle));
	}
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, CreationDependentObjectRespectsParentHysteresis)
{
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	FReplicationSystemTestClient* Client = CreateClient();

	UReplicatedTestObject* Parent = Server->CreateObject();
	UReplicatedTestObject* Child = Server->CreateObject();
	Bridge->AddCreationDependencyLink(Parent->NetRefHandle, Child->NetRefHandle);

	// Create objects on client.
	Server->UpdateAndSend({Client});
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Parent->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Child->NetRefHandle));

	// Filter out the parent, but set hysteresis to five frames.
	Server->ReplicationSystem->SetFilter(Parent->NetRefHandle, NotRoutedFilterHandle, "FiveFrames");

	// Parent and child should both stay in scope for 5 frames.
	for (unsigned FrameIt = 0; FrameIt < 5U; ++FrameIt)
	{
		Server->UpdateAndSend({Client});
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Parent->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Child->NetRefHandle));
	}

	// Parent and child should both go out of scope after 5 frames.
	Server->UpdateAndSend({Client});
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Parent->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Child->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, CreationDependentObjectHysteresisIgnoredWhenParentIsFilteredOut)
{
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	FReplicationSystemTestClient* Client = CreateClient();

	UReplicatedTestObject* Parent = Server->CreateObject();
	UReplicatedTestObject* Child = Server->CreateObject();
	Bridge->AddCreationDependencyLink(Parent->NetRefHandle, Child->NetRefHandle);

	// Create objects on client.
	Server->UpdateAndSend({Client});
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Parent->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Child->NetRefHandle));

	// Filter out parent, filter out child with hysteresis.
	Server->ReplicationSystem->SetFilter(Parent->NetRefHandle, NotRoutedFilterHandle);
	Server->ReplicationSystem->SetFilter(Child->NetRefHandle, NotRoutedFilterHandle, "FiveFrames");

	// Creation dependency should override child hysteresis, so both objects shouldn't exist on client.
	Server->UpdateAndSend({Client});
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Parent->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Child->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, CreationDependentObjectInHysteresisFilteredOutWhenParentHysteresisEnds)
{
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	FReplicationSystemTestClient* Client = CreateClient();

	UReplicatedTestObject* Parent = Server->CreateObject();
	UReplicatedTestObject* Child = Server->CreateObject();
	Bridge->AddCreationDependencyLink(Parent->NetRefHandle, Child->NetRefHandle);

	// Create objects on client.
	Server->UpdateAndSend({Client});
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Parent->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Child->NetRefHandle));

	// Filter out parent after one frame, filter out child after five frames.
	Server->ReplicationSystem->SetFilter(Parent->NetRefHandle, NotRoutedFilterHandle, "OneFrame");
	Server->ReplicationSystem->SetFilter(Child->NetRefHandle, NotRoutedFilterHandle, "FiveFrames");

	// Both objects should still exist after one frame.
	Server->UpdateAndSend({Client});
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Parent->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Child->NetRefHandle));

	// Parent hysteresis ends after one frame, so child should be pulled out of hysteresis.
	// Parent and child should remain out of scope for the duration of what would have been the child's hysteresis.
	for (unsigned FrameIt = 0; FrameIt < 5U; ++FrameIt)
	{
		Server->UpdateAndSend( { Client });
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Parent->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Child->NetRefHandle));
	}
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, OwningConnectionIsSetProperly)
{
	FReplicationSystemTestClient* Client = CreateClient();

	// Create object with subobject
	UReplicatedTestObject* ServerObject1 = Server->CreateObject();
	UReplicatedTestObject* ServerObject1SubObject = Server->CreateSubObject(ServerObject1->NetRefHandle, 0, 0);
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject1->NetRefHandle, Client->ConnectionIdOnServer);

	Server->UpdateAndSend({Client});

	// Delete subobject 
	Server->DestroyObject(ServerObject1SubObject);
	
	// Create new subobject
	ServerObject1SubObject = Server->CreateSubObject(ServerObject1->NetRefHandle, 0, 0);

	// Update twice to have the subobject internal index available for reuse
	Server->UpdateAndSend({Client});
	Server->UpdateAndSend({Client});

	// Create another object with subobject
	UReplicatedTestObject* ServerObject2 = Server->CreateObject();
	UReplicatedTestObject* ServerObject2SubObject = Server->CreateSubObject(ServerObject2->NetRefHandle, 0, 0);
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject2->NetRefHandle, Client->ConnectionIdOnServer);

	Server->UpdateAndSend({Client});

	// Verify all objects have the correct owning connection
	UE_NET_ASSERT_EQ(Server->ReplicationSystem->GetOwningNetConnection(ServerObject1->NetRefHandle), Client->ConnectionIdOnServer);
	UE_NET_ASSERT_EQ(Server->ReplicationSystem->GetOwningNetConnection(ServerObject1SubObject->NetRefHandle), Client->ConnectionIdOnServer);

	UE_NET_ASSERT_EQ(Server->ReplicationSystem->GetOwningNetConnection(ServerObject2->NetRefHandle), Client->ConnectionIdOnServer);
	UE_NET_ASSERT_EQ(Server->ReplicationSystem->GetOwningNetConnection(ServerObject2SubObject->NetRefHandle), Client->ConnectionIdOnServer);
}

UE_NET_TEST_FIXTURE(FTestFilteringFixture, TestIsRelevantForConnection)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_TRUE(Server->ReplicationSystem->IsRelevantForConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer));

	// Object should now exist on client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Turn on owner filter
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, ToOwnerFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_FALSE(Server->ReplicationSystem->IsRelevantForConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer));

	// As object is now filtered it should be deleted on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Set the client as owner
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_TRUE(Server->ReplicationSystem->IsRelevantForConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer));

	// The object should have been created again
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
	

	// Test Subobjects
	{
		// Test a normal subobject
		UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle);

		Server->UpdateAndSend({ Client });

		UE_NET_ASSERT_TRUE(Server->ReplicationSystem->IsRelevantForConnection(ServerSubObject->NetRefHandle, Client->ConnectionIdOnServer));

		// Object should now exist on client
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
	}

	// Test a normal subobject
	{
		UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle);

		Server->UpdateAndSend({ Client });

		UE_NET_ASSERT_TRUE(Server->ReplicationSystem->IsRelevantForConnection(ServerSubObject->NetRefHandle, Client->ConnectionIdOnServer));

		// Object should now exist on client
		UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
	}

	// Test a subobject set to COND_Never
	{
		UReplicatedTestObject* ServerSubObjectNever = Server->CreateSubObject(ServerObject->NetRefHandle);

		Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObjectNever->NetRefHandle, COND_Never);

		Server->UpdateAndSend({ Client });

		UE_NET_ASSERT_FALSE(Server->ReplicationSystem->IsRelevantForConnection(ServerSubObjectNever->NetRefHandle, Client->ConnectionIdOnServer));

		// Object should never exist on client
		UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObjectNever->NetRefHandle), nullptr);
	}

	// Finally, remove the owning connection
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, 0);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_FALSE(Server->ReplicationSystem->IsRelevantForConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer));

	// The client is no longer owning the object so it should be deleted
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	Server->DestroyObject(ServerObject);
}


} // end namespace UE::Net::Private
