// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestFilteringObject.h"
#include "Net/UnrealNetwork.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"

void UTestFilteringObject::SetFilterOut(bool bFilterOut)
{
	NetTest_FilterOut = bFilterOut;
}

void UTestFilteringObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	// Build descriptors and allocate PropertyReplicationFragments for this object
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

void UTestFilteringObject::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
	DOREPLIFETIME(ThisClass, NetTest_FilterOut);
	DOREPLIFETIME(ThisClass, ReplicatedCounter);
}
