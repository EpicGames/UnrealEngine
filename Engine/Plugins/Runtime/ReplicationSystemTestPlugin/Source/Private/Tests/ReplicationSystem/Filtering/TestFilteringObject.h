// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "TestFilteringObject.generated.h"

UCLASS()
class UTestFilteringObject : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	void SetFilterOut(bool bFilterOut);
	bool GetFilterOut() const { return NetTest_FilterOut; }

	UPROPERTY(Replicated)
	uint32 ReplicatedCounter = 0;

protected:
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

private:
	// The property name below violates the UE coding standard due to hardcoded prefix in the descriptor builder for testing purposes.
	UPROPERTY(Replicated)
	bool NetTest_FilterOut = false;
};
