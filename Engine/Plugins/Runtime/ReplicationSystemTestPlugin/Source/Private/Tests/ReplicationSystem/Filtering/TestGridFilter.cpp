// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilterDefinitions.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectGridFilter.h"
#include "Iris/ReplicationSystem/WorldLocations.h"

#include "Tests/ReplicationSystem/Filtering/MockNetObjectFilter.h"
#include "Tests/ReplicationSystem/Filtering/TestFilteringObject.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"

#include "GenericPlatform/GenericPlatformMath.h"

namespace UE::Net::Private
{

class FTestGridFilterFixture : public FReplicationSystemServerClientTestFixture
{
protected:
	virtual void SetUp() override
	{
		InitFilterDefinitions();
		FReplicationSystemServerClientTestFixture::SetUp();
		InitFilterHandles();
	}

	virtual void TearDown() override
	{
		FReplicationSystemServerClientTestFixture::TearDown();
		RestoreFilterDefinitions();
	}

private:
	void InitFilterDefinitions()
	{
		const UClass* NetObjectFilterDefinitionsClass = UNetObjectFilterDefinitions::StaticClass();
		const FProperty* DefinitionsProperty = NetObjectFilterDefinitionsClass->FindPropertyByName("NetObjectFilterDefinitions");
		check(DefinitionsProperty != nullptr);

		// Save CDO state.
		UNetObjectFilterDefinitions* FilterDefinitions = GetMutableDefault<UNetObjectFilterDefinitions>();
		DefinitionsProperty->CopyCompleteValue(&OriginalFilterDefinitions, (void*)(UPTRINT(FilterDefinitions) + DefinitionsProperty->GetOffset_ForInternal()));

		// Modify definitions to only include our filters. Ugly... 
		TArray<FNetObjectFilterDefinition> NewFilterDefinitions;
		{
			FNetObjectFilterDefinition& GridWorldLocDef = NewFilterDefinitions.Emplace_GetRef();
			GridWorldLocDef.FilterName = "NetObjectGridWorldLocFilter";
			GridWorldLocDef.ClassName = "/Script/IrisCore.NetObjectGridWorldLocFilter";
			GridWorldLocDef.ConfigClassName = "/Script/IrisCore.NetObjectGridFilterConfig";
		}

		DefinitionsProperty->CopyCompleteValue((void*)(UPTRINT(FilterDefinitions) + DefinitionsProperty->GetOffset_ForInternal()), &NewFilterDefinitions);

		// Setup custom filter configs via the CDO. Ugly! We need to support non-CDO engine configs in Iris
		UNetObjectGridFilterConfig* GridFilterConfig = GetMutableDefault<UNetObjectGridFilterConfig>();
		
		// Save original CDO state
		OriginalFilterProfiles = GridFilterConfig->FilterProfiles;

		// Add unique profiles
		GridFilterConfig->FilterProfiles.Add(
			// This profile disables the culling histerisis so that filtering out an object has impact on the current tick.
			FNetObjectGridFilterProfile
			{ 
				.FilterProfileName = TEXT("CullDistanceTest"),
				.FrameCountBeforeCulling = 1,
			});
	}

	void RestoreFilterDefinitions()
	{
		// Restore CDO state from the saved state.
		const UClass* NetObjectFilterDefinitionsClass = UNetObjectFilterDefinitions::StaticClass();
		const FProperty* DefinitionsProperty = NetObjectFilterDefinitionsClass->FindPropertyByName("NetObjectFilterDefinitions");
		UNetObjectFilterDefinitions* FilterDefinitions = GetMutableDefault<UNetObjectFilterDefinitions>();
		DefinitionsProperty->CopyCompleteValue((void*)(UPTRINT(FilterDefinitions) + DefinitionsProperty->GetOffset_ForInternal()), &OriginalFilterDefinitions);
		OriginalFilterDefinitions.Empty();

		WorldLocFilterHandle = InvalidNetObjectFilterHandle;
		WorldLocFilter = nullptr;

		UNetObjectGridFilterConfig* GridFilterConfig = GetMutableDefault<UNetObjectGridFilterConfig>();
		GridFilterConfig->FilterProfiles = OriginalFilterProfiles;
	}

	void InitFilterHandles()
	{
		WorldLocFilter = ExactCast<UNetObjectGridWorldLocFilter>(Server->GetReplicationSystem()->GetFilter("NetObjectGridWorldLocFilter"));
		WorldLocFilterHandle = Server->GetReplicationSystem()->GetFilterHandle("NetObjectGridWorldLocFilter");
	}


public:

	// Array that preemptively registers the world info so it is assigned to an object during his creation.
	struct FObjectWorldInfo
	{
		FVector Loc = FVector::ZeroVector;
		float CullDistance = 1500.f;
	};
	TArray<FObjectWorldInfo> WorldInfoToBeAssigned;
	TMap<const UObject*, FObjectWorldInfo> ObjectWorldInfoMap;

	FObjectWorldInfo GetWorldInfo(const UObject* ReplicatedObject)
	{
		if (ObjectWorldInfoMap.Find(ReplicatedObject) == nullptr)
		{
			checkf(!WorldInfoToBeAssigned.IsEmpty(), TEXT("No info was pushed for assignation"));
			FObjectWorldInfo NewInfo = WorldInfoToBeAssigned[0];
			WorldInfoToBeAssigned.RemoveAt(0);

			ObjectWorldInfoMap.Add(ReplicatedObject, NewInfo);
		}

		return ObjectWorldInfoMap[ReplicatedObject];
	}

	void SetWorldInfo(const UObject* ReplicatedObject, const FObjectWorldInfo& WorldInfo)
	{
		ObjectWorldInfoMap[ReplicatedObject] = WorldInfo;
	}

protected:
	UNetObjectGridWorldLocFilter* WorldLocFilter;
	FNetObjectFilterHandle WorldLocFilterHandle;
private:
	TArray<FNetObjectFilterDefinition> OriginalFilterDefinitions;
	TArray<FNetObjectGridFilterProfile> OriginalFilterProfiles;
};

UE_NET_TEST_FIXTURE(FTestGridFilterFixture, TestWorldLocGridFilter)
{
	Server->GetReplicationBridge()->SetExternalWorldLocationUpdateFunctor([&](FNetRefHandle NetHandle, const UObject* ReplicatedObject, FVector& OutLocation, float& OutCullDistance)
	{
		const FObjectWorldInfo& Info = GetWorldInfo(ReplicatedObject);
		OutLocation = Info.Loc;
		OutCullDistance = Info.CullDistance;
	});

	const UNetObjectGridFilterConfig* DefaultGridConfig = GetDefault<UNetObjectGridFilterConfig>();

	// Spawn object with WorldLocation's on server
	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	Params.bNeedsWorldLocationUpdate = true;
	Params.bUseClassConfigDynamicFilter = true;
	
	// Relevant objects
	WorldInfoToBeAssigned.Add(FObjectWorldInfo{ .Loc = FVector::ZeroVector, .CullDistance = 1500.f });
	UReplicatedTestObject* ServerObjectZero = Server->CreateObject(Params);

	WorldInfoToBeAssigned.Add(FObjectWorldInfo{ .Loc = FVector(100.f, 100.f, 100.f), .CullDistance = 1500.f });
	UReplicatedTestObject* ServerObjectNear = Server->CreateObject(Params);

	// Make a location for an object that sits right in the limit of a grid cell
	const FVector CellLimitPos(DefaultGridConfig->CellSizeX, DefaultGridConfig->CellSizeY, 0.0f);
	WorldInfoToBeAssigned.Add(FObjectWorldInfo{ .Loc = CellLimitPos, .CullDistance = (float)CellLimitPos.Size() });
	UReplicatedTestObject* ServerObjectLimit = Server->CreateObject(Params);

	// Culled objects
	WorldInfoToBeAssigned.Add(FObjectWorldInfo{ .Loc = FVector(DefaultGridConfig->CellSizeX + 100.f, DefaultGridConfig->CellSizeY + 100.f, 100.f), .CullDistance = 1500.f });
	UReplicatedTestObject* ServerObjectCulled = Server->CreateObject(Params);

	WorldInfoToBeAssigned.Add(FObjectWorldInfo{ .Loc = FVector(DefaultGridConfig->CellSizeX + 99999.f, DefaultGridConfig->CellSizeY + 99999.f, 99999.f), .CullDistance = 1500.f });
	UReplicatedTestObject* ServerObjectVeryFar = Server->CreateObject(Params);

	TArray<UReplicatedTestObject*> ServerReplicatedObjects;
	{
		ServerReplicatedObjects.Add(ServerObjectZero);
		ServerReplicatedObjects.Add(ServerObjectNear);
		ServerReplicatedObjects.Add(ServerObjectLimit);
		ServerReplicatedObjects.Add(ServerObjectCulled);
		ServerReplicatedObjects.Add(ServerObjectVeryFar);
	}

	// Apply grid filter
	for (auto It : ServerReplicatedObjects)
	{
		Server->ReplicationSystem->SetFilter(It->NetRefHandle, WorldLocFilterHandle);
	}

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Set the view location of the client to (0,0,0)
	FReplicationView ReplicationView;
	FReplicationView::FView View = ReplicationView.Views.Emplace_GetRef();
	View.Pos = FVector(0.0f, 0.0f, 0.0f);
	Server->ReplicationSystem->SetReplicationView(Client->ConnectionIdOnServer, ReplicationView);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Test visible objects
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectZero->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectNear->NetRefHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectLimit->NetRefHandle), nullptr);

	// Test culled objects
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectCulled->NetRefHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectVeryFar->NetRefHandle), nullptr);

	for (auto It : ServerReplicatedObjects)
	{
		Server->DestroyObject(It);
	}
}

UE_NET_TEST_FIXTURE(FTestGridFilterFixture, TestWorldLocGridFilterWithHugeCullDistance)
{
	Server->GetReplicationBridge()->SetExternalWorldLocationUpdateFunctor([&](FNetRefHandle NetHandle, const UObject* ReplicatedObject, FVector& OutLocation, float& OutCullDistance)
	{
		const FObjectWorldInfo& Info = GetWorldInfo(ReplicatedObject);
		OutLocation = Info.Loc;
		OutCullDistance = Info.CullDistance;
	});

	const UNetObjectGridFilterConfig* DefaultGridConfig = GetDefault<UNetObjectGridFilterConfig>();
	const UWorldLocationsConfig* DefaultWorldLocFonfig = GetDefault<UWorldLocationsConfig>();

	// Spawn object with WorldLocation's on server
	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	Params.bNeedsWorldLocationUpdate = true;
	Params.bUseClassConfigDynamicFilter = true;

	// We do expect and ensure
	FTestEnsureScope SuppressEnsureScope;
	
	// Relevant objects
	WorldInfoToBeAssigned.Add(FObjectWorldInfo{ .Loc = FVector::ZeroVector, .CullDistance = DefaultWorldLocFonfig->MaxNetCullDistance-SMALL_NUMBER });
	UReplicatedTestObject* ServerObjectHugeCullDistance = Server->CreateObject(Params);

	// Apply grid filter
	Server->ReplicationSystem->SetFilter(ServerObjectHugeCullDistance->NetRefHandle, WorldLocFilterHandle);

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Set the view location of the client to (0,0,0)
	FReplicationView ReplicationView;
	FReplicationView::FView View = ReplicationView.Views.Emplace_GetRef();
	View.Pos = FVector(0.0f, 0.0f, 0.0f);
	Server->ReplicationSystem->SetReplicationView(Client->ConnectionIdOnServer, ReplicationView);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Test that the object is relevant
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectHugeCullDistance->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestGridFilterFixture, TestWorldLocationIsFrequentlyUpdatedForNonDormantObject)
{
	Server->GetReplicationBridge()->SetExternalWorldLocationUpdateFunctor([&](FNetRefHandle NetHandle, const UObject* ReplicatedObject, FVector& OutLocation, float& OutCullDistance)
	{
		const FObjectWorldInfo& Info = GetWorldInfo(ReplicatedObject);
		OutLocation = Info.Loc;
		OutCullDistance = Info.CullDistance;
	});

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Set the view location of the client to (0,0,0)
	FReplicationView ReplicationView;
	FReplicationView::FView View = ReplicationView.Views.Emplace_GetRef();
	View.Pos = FVector(0.0f, 0.0f, 0.0f);
	Server->ReplicationSystem->SetReplicationView(Client->ConnectionIdOnServer, ReplicationView);

	// Spawn object with WorldLocation's on server
	const UNetObjectGridFilterConfig* DefaultGridConfig = GetDefault<UNetObjectGridFilterConfig>();
	WorldInfoToBeAssigned.Add(FObjectWorldInfo{ .Loc = FVector::ZeroVector, .CullDistance = 1500.f });

	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	Params.bNeedsWorldLocationUpdate = true;
	Params.bIsDormant = false;
	Params.bUseClassConfigDynamicFilter = false;
	Params.bUseExplicitDynamicFilter = true;
	Params.ExplicitDynamicFilterName = FName("NetObjectGridWorldLocFilter");

	UReplicatedTestObject* ServerObject = Server->CreateObject(Params);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Verify objects has been created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Update the world location to a cell the object shouldn't previously have been touching, without marking the object dirty
	SetWorldInfo(ServerObject, FObjectWorldInfo{ .Loc = FVector(DefaultGridConfig->CellSizeX + 1600.0f,DefaultGridConfig->CellSizeY + 1600.0f, 0), .CullDistance = 1500.f});

	// Send and deliver packet
	for (uint32 LoopIt = 0, LoopEndIt = DefaultGridConfig->ViewPosRelevancyFrameCount + DefaultGridConfig->DefaultFrameCountBeforeCulling; LoopIt <= LoopEndIt; ++LoopIt)
	{
		Server->UpdateAndSend({ Client });
	}

	// Object should now have been destroyed.
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Return object to origin.
	SetWorldInfo(ServerObject, FObjectWorldInfo{ .Loc = FVector::ZeroVector, .CullDistance = 1500.f });

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Object should have been re-created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestGridFilterFixture, TestWorldLocCullDistance)
{
	Server->GetReplicationBridge()->SetExternalWorldLocationUpdateFunctor([&](FNetRefHandle NetHandle, const UObject* ReplicatedObject, FVector& OutLocation, float& OutCullDistance)
	{
		const FObjectWorldInfo& Info = GetWorldInfo(ReplicatedObject);
		OutLocation = Info.Loc;
		OutCullDistance = Info.CullDistance;
	});

	// Spawn an object that resides at the exact limit of it's culldistance and at the boundary of a grid cell.
	const UNetObjectGridFilterConfig* DefaultGridConfig = GetDefault<UNetObjectGridFilterConfig>();
	const float DefaultCullDistance = DefaultGridConfig->CellSizeX;
	FObjectWorldInfo ObjectWorldInfo{ .Loc = FVector(DefaultCullDistance, 0.0f, 0.0f), .CullDistance = DefaultCullDistance };
	WorldInfoToBeAssigned.Add(ObjectWorldInfo);

	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	Params.bNeedsWorldLocationUpdate = true;
	Params.bUseClassConfigDynamicFilter = true;
	UReplicatedTestObject* ServerObjectTest = Server->CreateObject(Params);

	// Apply grid filter
	// Use the CullDistanceTest profile that disables the FrameCountBeforeCulling feature
	Server->ReplicationSystem->SetFilter(ServerObjectTest->NetRefHandle, WorldLocFilterHandle, TEXT("CullDistanceTest"));

	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Set the view location of the client to (0,0,0)
	FReplicationView ReplicationView;
	FReplicationView::FView View = ReplicationView.Views.Emplace_GetRef();
	View.Pos = FVector(0.0f, 0.0f, 0.0f);
	Server->ReplicationSystem->SetReplicationView(Client->ConnectionIdOnServer, ReplicationView);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });
	
	// Object should be relevant
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectTest->NetRefHandle), nullptr);

	// Reduce his culldistance via override
	Server->ReplicationSystem->SetCullDistanceOverride(ServerObjectTest->NetRefHandle, DefaultCullDistance-1.0f);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Object should be filtered out
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectTest->NetRefHandle), nullptr);

	// Go back to the initial cull distance
	Server->ReplicationSystem->ClearCullDistanceOverride(ServerObjectTest->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Object should be relevant again
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectTest->NetRefHandle), nullptr);

	// Change the object's internal cull distance
	ObjectWorldInfo.CullDistance = DefaultCullDistance - 100.f;
	SetWorldInfo(ServerObjectTest, ObjectWorldInfo);

	// The object needs to be dirty for the cull distance update to be picked up
	Server->ReplicationSystem->MarkDirty(ServerObjectTest->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Object should be filtered out
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectTest->NetRefHandle), nullptr);

	Server->DestroyObject(ServerObjectTest);
}

UE_NET_TEST_FIXTURE(FTestGridFilterFixture, TestStartQueueRpcFlushNetDormancyStop)
{
	// Need to set the world location update function to be able to set world locations.
	Server->GetReplicationBridge()->SetExternalWorldLocationUpdateFunctor([&](FNetRefHandle NetHandle, const UObject* Object, FVector& OutLocation, float& OutCullDistance)
		{
		});

	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with WorldLocation's on server
	const UNetObjectGridFilterConfig* DefaultGridConfig = GetDefault<UNetObjectGridFilterConfig>();
	WorldInfoToBeAssigned.Add(FObjectWorldInfo{ .Loc = FVector::ZeroVector, .CullDistance = 1500.f });

	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	Params.bNeedsWorldLocationUpdate = true;
	Params.bIsDormant = false;
	Params.bUseClassConfigDynamicFilter = false;
	Params.bUseExplicitDynamicFilter = true;
	Params.ExplicitDynamicFilterName = FName("NetObjectGridWorldLocFilter");

	UReplicatedTestObject* ServerObject = Server->CreateObject(Params);

	// Filter out to ensure ServrObject is not referenced by any connection
	FNetObjectGroupHandle NotReplicatedGroupHandle = Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup();
	Server->ReplicationSystem->AddToGroup(NotReplicatedGroupHandle, ServerObject->NetRefHandle);

	Server->ReplicationBridge->NetFlushDormantObject(ServerObject->NetRefHandle);
	Server->DestroyObject(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::DestroyNetHandle | EEndReplicationFlags::Flush);

	// Send update
	Server->UpdateAndSend({Client});

	// Send update. Object should be destroyed on server.
	Server->UpdateAndSend({Client});
}


#if 0
UE_NET_TEST_FIXTURE(FTestGridFilterFixture, TestFilterPerformance)
{
	static const int32 NUM_CLIENTS = 14;
	static const int32 NUM_OBJECTS = 2000;
	static const int32 TEST_ITERATIONS = 100;

	struct FObjectLoc
	{
		FVector Loc;
		float CullDistance;
	};

	TMap<const UObject*, FObjectLoc> ObjectLocs;
	Server->GetReplicationBridge()->SetExternalWorldLocationUpdateFunctor([&](FNetRefHandle NetHandle, const UObject* ReplicatedObject, FVector& OutLocation, float& OutCullDistance)
		{
			const FObjectLoc& ObjectLoc = ObjectLocs[ReplicatedObject];
			OutLocation = ObjectLoc.Loc;
			OutCullDistance = ObjectLoc.CullDistance;
		});

	// Create client connections.
	TArray<FReplicationSystemTestClient*> TestClients;
	for (int32 i = 0; i < NUM_CLIENTS; i++)
	{
		FReplicationSystemTestClient* TestClient = CreateClient();

		TestClients.Add(TestClient);

		FReplicationView ReplicationView;
		FReplicationView::FView View = ReplicationView.Views.Emplace_GetRef();
		View.Pos = FVector(0.0f, 0.0f, 0.0f);
		Server->ReplicationSystem->SetReplicationView(TestClient->ConnectionIdOnServer, ReplicationView);
	}

	// Create objects at random positions inside a cell.
	const UNetObjectGridFilterConfig* DefaultGridConfig = GetDefault<UNetObjectGridFilterConfig>();
	FGenericPlatformMath::SRandInit(0);
	for (int32 i = 0; i < NUM_OBJECTS; i++)
	{
		UObjectReplicationBridge::FRootObjectReplicationParams Params;
		Params.bNeedsWorldLocationUpdate = true;
		Params.bUseClassConfigDynamicFilter = true;

		FVector Pos;
		Pos.X = FGenericPlatformMath::SRand() * DefaultGridConfig->CellSizeX;
		Pos.Y = FGenericPlatformMath::SRand() * DefaultGridConfig->CellSizeY;
		Pos.Z = 0.0f;

		UReplicatedTestObject* Object = Server->CreateObject(Params);
		ObjectLocs.Add(Object, FObjectLoc{ Pos, 1500.f });

		Server->ReplicationSystem->SetFilter(Object->NetRefHandle, WorldLocFilterHandle);
	}

	// Run server replication multiple iterations.
	for (int32 i = 0; i < TEST_ITERATIONS; i++)
	{
		const double StartTime = FPlatformTime::Seconds();

		Server->PreSendUpdate();
		for (FReplicationSystemTestClient* TestClient : TestClients)
		{
			Server->SendAndDeliverTo(TestClient, DeliverPacket);
		}
		Server->PostSendUpdate();
	}
}
#endif

} // end namespace UE::Net::Private