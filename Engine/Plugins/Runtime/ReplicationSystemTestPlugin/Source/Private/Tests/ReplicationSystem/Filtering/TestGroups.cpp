// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectGroups.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Iris/ReplicationSystem/Filtering/ReplicationFiltering.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"

namespace UE::Net::Private
{

class FTestGroupsFixture : public FReplicationSystemServerClientTestFixture
{
public:

	FReplicationFiltering* ServerFiltering = nullptr;
	FNetObjectGroups* ServerGroups = nullptr;
	FDirtyNetObjectTracker* ServerDirtyNetObjectTracker = nullptr;
	FReplicationSystemTestClient* Client = nullptr;
	
protected:
	virtual void SetUp() override
	{
		FReplicationSystemServerClientTestFixture::SetUp();
		ServerFiltering = &Server->ReplicationSystem->GetReplicationSystemInternal()->GetFiltering();
		ServerGroups = &Server->ReplicationSystem->GetReplicationSystemInternal()->GetGroups();
		ServerDirtyNetObjectTracker = &Server->ReplicationSystem->GetReplicationSystemInternal()->GetDirtyNetObjectTracker();

		// Add client
		Client = CreateClient();
	}

	void Filter()
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
	}

	void VerifyGroupStatus(FNetObjectGroupHandle GroupHandle, uint32 ConnectionId)
	{
		FNetObjectGroup* Group = ServerGroups->GetGroup(GroupHandle);

		UE_NET_ASSERT_TRUE(Group != nullptr);

		ENetFilterStatus ReplicationStatus = ENetFilterStatus::Disallow;
		UE_NET_ASSERT_TRUE(ServerFiltering->GetGroupFilterStatus(GroupHandle, ConnectionId, ReplicationStatus));

		const FNetBitArrayView ObjectGroupFilter = ServerFiltering->GetGroupFilteredOutObjects(ConnectionId);
		for (uint32 InternalIndex : MakeArrayView(Group->Members.GetData(), Group->Members.Num()))
		{
			UE_NET_ASSERT_TRUE(ObjectGroupFilter.GetBit(InternalIndex) == (ReplicationStatus == ENetFilterStatus::Disallow));
		}
	}

	void VerifyObjectFilterStatus(FNetRefHandle Handle, uint32 ConnectionId, bool bExpectedReplicationAllowed)
	{
		const uint32 InternalIndex = Server->GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager().GetInternalIndex(Handle);
		UE_NET_ASSERT_NE((uint32)InvalidInternalNetRefIndex, InternalIndex);

		// Verify expected filter status
		const FNetBitArrayView ObjectGroupFilter = ServerFiltering->GetGroupFilteredOutObjects(ConnectionId);

		// Filter is subtractive, so if the bit is set we the object is filtered out
		UE_NET_ASSERT_TRUE(ObjectGroupFilter.GetBit(InternalIndex) == !bExpectedReplicationAllowed);

		// Look if the client received the object as expected
		UObject* ClientObject = Client->GetReplicationBridge()->GetReplicatedObject(Handle);
		UE_NET_ASSERT_EQ((ClientObject!=nullptr), bExpectedReplicationAllowed);
	}
};

UE_NET_TEST_FIXTURE(FTestGroupsFixture, PublicGroupAPI)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Create
	{
		FNetObjectGroupHandle GroupHandle = ReplicationSystem->CreateGroup(NAME_None);
		
		UE_NET_ASSERT_TRUE(GroupHandle.IsValid());
		UE_NET_ASSERT_TRUE(ReplicationSystem->IsValidGroup(GroupHandle));

		UE_NET_ASSERT_FALSE(ReplicationSystem->IsValidGroup(FNetObjectGroupHandle()));

		// Destroy
		ReplicationSystem->DestroyGroup(GroupHandle);

		// Group handle is still initialized
		UE_NET_ASSERT_TRUE(GroupHandle.IsValid());

		// But it's not considered a valid group.
		UE_NET_ASSERT_FALSE(ReplicationSystem->IsValidGroup(GroupHandle));
	}

	// Find
	{
		const FName NameGroupFind = FName(TEXT("TestGroupFind"));
		
		// Create a group
		FNetObjectGroupHandle OriginalGroupHandle = ReplicationSystem->CreateGroup(NameGroupFind);
		UE_NET_ASSERT_TRUE(OriginalGroupHandle.IsValid());
		UE_NET_ASSERT_TRUE(ReplicationSystem->IsValidGroup(OriginalGroupHandle));

		// Find the group handle via it's name
		FNetObjectGroupHandle FoundGroupHandle = ReplicationSystem->FindGroup(NameGroupFind);
		UE_NET_ASSERT_TRUE(FoundGroupHandle.IsValid());
		UE_NET_ASSERT_TRUE(ReplicationSystem->IsValidGroup(FoundGroupHandle));

		// Should be the same handle
		UE_NET_ASSERT_EQ(FoundGroupHandle.GetRawValue(), OriginalGroupHandle.GetRawValue());

		// Destroy the group
		ReplicationSystem->DestroyGroup(OriginalGroupHandle);

		// They both should be invalid
		UE_NET_ASSERT_FALSE(ReplicationSystem->IsValidGroup(OriginalGroupHandle));
		UE_NET_ASSERT_FALSE(ReplicationSystem->IsValidGroup(FoundGroupHandle));
	}

	// Destroy/Create
	{
		// Create a group
		FNetObjectGroupHandle DestroyedGroupHandle = ReplicationSystem->CreateGroup(NAME_None);
		UE_NET_ASSERT_TRUE(DestroyedGroupHandle.IsValid());
		UE_NET_ASSERT_TRUE(ReplicationSystem->IsValidGroup(DestroyedGroupHandle));

		// Destroy the group
		ReplicationSystem->DestroyGroup(DestroyedGroupHandle);

		// Should be invalid
		UE_NET_ASSERT_FALSE(ReplicationSystem->IsValidGroup(DestroyedGroupHandle));

		{
			// Create a new random group
			FNetObjectGroupHandle RandomGroupHandle = ReplicationSystem->CreateGroup(NAME_None);
			UE_NET_ASSERT_TRUE(ReplicationSystem->IsValidGroup(RandomGroupHandle));
		}

		// The destroyed group should still be invalid
		UE_NET_ASSERT_FALSE(ReplicationSystem->IsValidGroup(DestroyedGroupHandle));
	}

	// Add/Remove
	{
		// Spawn object on server
		UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

		FNetObjectGroupHandle GroupHandle = ReplicationSystem->CreateGroup(NAME_None);

		UE_NET_ASSERT_FALSE(ReplicationSystem->IsInGroup(GroupHandle, ServerObject->NetRefHandle));

		ReplicationSystem->AddToGroup(GroupHandle, ServerObject->NetRefHandle);

		UE_NET_ASSERT_TRUE(ReplicationSystem->IsInGroup(GroupHandle, ServerObject->NetRefHandle));

		ReplicationSystem->RemoveFromGroup(GroupHandle, ServerObject->NetRefHandle);

		UE_NET_ASSERT_FALSE(ReplicationSystem->IsInGroup(GroupHandle, ServerObject->NetRefHandle));

		ReplicationSystem->DestroyGroup(GroupHandle);

		UE_NET_ASSERT_FALSE(ReplicationSystem->IsInGroup(GroupHandle, ServerObject->NetRefHandle));
	}
}

UE_NET_TEST_FIXTURE(FTestGroupsFixture, PublicGroupAPIMemberOfMultipleGroups)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Create
	FNetObjectGroupHandle GroupHandle0 = ReplicationSystem->CreateGroup(NAME_None);
	FNetObjectGroupHandle GroupHandle1 = ReplicationSystem->CreateGroup(NAME_None);

	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

	UE_NET_ASSERT_FALSE(ReplicationSystem->IsInGroup(GroupHandle0, ServerObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(ReplicationSystem->IsInGroup(GroupHandle1, ServerObject->NetRefHandle));

	ReplicationSystem->AddToGroup(GroupHandle0, ServerObject->NetRefHandle);

	UE_NET_ASSERT_TRUE(ReplicationSystem->IsInGroup(GroupHandle0, ServerObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(ReplicationSystem->IsInGroup(GroupHandle1, ServerObject->NetRefHandle));

	ReplicationSystem->AddToGroup(GroupHandle1, ServerObject->NetRefHandle);

	UE_NET_ASSERT_TRUE(ReplicationSystem->IsInGroup(GroupHandle0, ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ReplicationSystem->IsInGroup(GroupHandle1, ServerObject->NetRefHandle));

	ReplicationSystem->RemoveFromGroup(GroupHandle0, ServerObject->NetRefHandle);

	UE_NET_ASSERT_FALSE(ReplicationSystem->IsInGroup(GroupHandle0, ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ReplicationSystem->IsInGroup(GroupHandle1, ServerObject->NetRefHandle));

	ReplicationSystem->RemoveFromGroup(GroupHandle1, ServerObject->NetRefHandle);

	UE_NET_ASSERT_FALSE(ReplicationSystem->IsInGroup(GroupHandle0, ServerObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(ReplicationSystem->IsInGroup(GroupHandle1, ServerObject->NetRefHandle));
}

// Group filtering tests
UE_NET_TEST_FIXTURE(FTestGroupsFixture, GroupFilterAPI)
{
	// Spawn object on server
	UReplicatedTestObject* ServerObject0 = Server->CreateObject(0, 0);
	UReplicatedTestObject* ServerObject1 = Server->CreateObject(0, 0);
	UReplicatedTestObject* ServerObject2 = Server->CreateObject(0, 0);

	// Create group
	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	UE_NET_ASSERT_TRUE(GroupHandle.IsValid());

	// Add Objects to group
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerObject0->NetRefHandle);
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerObject1->NetRefHandle);
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerObject2->NetRefHandle);
	
	UReplicatedTestObject* ServerObject3 = Server->CreateObject(0, 0);
	UReplicatedTestObject* ServerObject4 = Server->CreateObject(0, 0);
	UReplicatedTestObject* ServerObject5 = Server->CreateObject(0, 0);

	// Create another group
	FNetObjectGroupHandle GroupHandle2 = Server->ReplicationSystem->CreateGroup(NAME_None);
	UE_NET_ASSERT_TRUE(GroupHandle2.IsValid());

	// Add Objects to group
	Server->ReplicationSystem->AddToGroup(GroupHandle2, ServerObject3->NetRefHandle);
	Server->ReplicationSystem->AddToGroup(GroupHandle2, ServerObject4->NetRefHandle);
	Server->ReplicationSystem->AddToGroup(GroupHandle2, ServerObject5->NetRefHandle);
													 
	UReplicatedTestObject* ServerObject6 = Server->CreateObject(0, 0);

	// Create another group
	FNetObjectGroupHandle GroupHandle3 = Server->ReplicationSystem->CreateGroup(NAME_None);
	UE_NET_ASSERT_TRUE(GroupHandle3.IsValid());

	// Update filters
	Filter();

	const FNetBitArrayView ObjectGroupFilter = ServerFiltering->GetGroupFilteredOutObjects(Client->ConnectionIdOnServer);

	// Add group to group filter
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle2);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle3);

	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle2, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle3, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	// Update filters
	Filter();

	VerifyGroupStatus(GroupHandle, Client->ConnectionIdOnServer);
	VerifyGroupStatus(GroupHandle2, Client->ConnectionIdOnServer);
	VerifyGroupStatus(GroupHandle3, Client->ConnectionIdOnServer);

	UE_NET_ASSERT_TRUE(ObjectGroupFilter.IsAnyBitSet());

	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Update filters
	Filter();

	VerifyGroupStatus(GroupHandle, Client->ConnectionIdOnServer);
	VerifyGroupStatus(GroupHandle2, Client->ConnectionIdOnServer);
	VerifyGroupStatus(GroupHandle3, Client->ConnectionIdOnServer);

	UE_NET_ASSERT_TRUE(ObjectGroupFilter.IsAnyBitSet());

	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle2, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Update filters
	Filter();

	VerifyGroupStatus(GroupHandle, Client->ConnectionIdOnServer);
	VerifyGroupStatus(GroupHandle2, Client->ConnectionIdOnServer);
	VerifyGroupStatus(GroupHandle3, Client->ConnectionIdOnServer);

	UE_NET_ASSERT_TRUE(ObjectGroupFilter.IsNoBitSet());

	// Add Object to group 3 which is set, which should filter out the object
	Server->ReplicationSystem->AddToGroup(GroupHandle3, ServerObject6->NetRefHandle);

	// Update filters
	Filter();

	VerifyGroupStatus(GroupHandle, Client->ConnectionIdOnServer);
	VerifyGroupStatus(GroupHandle2, Client->ConnectionIdOnServer);
	VerifyGroupStatus(GroupHandle3, Client->ConnectionIdOnServer);

	UE_NET_ASSERT_TRUE(ObjectGroupFilter.IsAnyBitSet());

	// Add Object to group 3 which is set, which should filter out the object
	Server->ReplicationSystem->RemoveFromGroup(GroupHandle3, ServerObject6->NetRefHandle);

	// Update filters
	Filter();

	VerifyGroupStatus(GroupHandle, Client->ConnectionIdOnServer);
	VerifyGroupStatus(GroupHandle2, Client->ConnectionIdOnServer);
	VerifyGroupStatus(GroupHandle3, Client->ConnectionIdOnServer);

	UE_NET_ASSERT_TRUE(ObjectGroupFilter.IsNoBitSet());

	// Delete group
	Server->ReplicationSystem->DestroyGroup(GroupHandle);
	Server->ReplicationSystem->DestroyGroup(GroupHandle2);
	Server->ReplicationSystem->DestroyGroup(GroupHandle3);

	Filter();

	UE_NET_ASSERT_TRUE(ObjectGroupFilter.IsNoBitSet());
}

UE_NET_TEST_FIXTURE(FTestGroupsFixture, GroupFilterAPIManyFilters)
{
	// Spawn object on server
	UReplicatedTestObject* ServerObject0 = Server->CreateObject(0, 0);

	// Create groups and add them as filters
	const uint32 GroupCount = 256;

	FNetObjectGroupHandle GroupHandles[GroupCount];

	for (uint32 It = 0; It < GroupCount; ++It)
	{
		FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
		Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle);
	
		GroupHandles[It] = GroupHandle;	
	}
}

UE_NET_TEST_FIXTURE(FTestGroupsFixture, GroupFilterAPIObjectMemberOfMoreThanOneGroup)
{
	// Spawn object on server
	UReplicatedTestObject* ServerObject0 = Server->CreateObject(0, 0);
	const FNetRefHandle ServerHandle = ServerObject0->NetRefHandle;

	// Create groups
	FNetObjectGroupHandle GroupHandle0 = Server->ReplicationSystem->CreateGroup(NAME_None);
	FNetObjectGroupHandle GroupHandle1 = Server->ReplicationSystem->CreateGroup(NAME_None);
	UE_NET_ASSERT_TRUE(GroupHandle0.IsValid());
	UE_NET_ASSERT_TRUE(GroupHandle1.IsValid());

	// Add Objects to groups
	Server->ReplicationSystem->AddToGroup(GroupHandle0, ServerHandle);
	Server->ReplicationSystem->AddToGroup(GroupHandle1, ServerHandle);

	// Update filters
	Filter();

	const FNetBitArrayView ObjectGroupFilter = ServerFiltering->GetGroupFilteredOutObjects(Client->ConnectionIdOnServer);

	// Verify that object is not filtered out
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, true);

	// Add groups to group filter
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle0);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle1);

	// Filter out objects in the group
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle0, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle1, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	// Update filters
	Filter();

	// We now expect the object to be filtered out
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, false);

	// Allow objects from group 0
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle0, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Update filters
	Filter();

	// We still  expect the object to be filtered out since the object is a member of two groups
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, false);

	// Allow objects from group 1
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle1, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Update filters
	Filter();

	// We now expect the object to no longer be filtered out
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, true);
}


UE_NET_TEST_FIXTURE(FTestGroupsFixture, GroupFilterAPIAddThenRemoveFilterTrait)
{
	// Spawn object on server
	UReplicatedTestObject* ServerObject0 = Server->CreateObject(0, 0);
	const FNetRefHandle ServerHandle = ServerObject0->NetRefHandle;

	// Create groups
	FNetObjectGroupHandle GroupHandle0 = Server->ReplicationSystem->CreateGroup(NAME_None);
	FNetObjectGroupHandle GroupHandle1 = Server->ReplicationSystem->CreateGroup(NAME_None);
	UE_NET_ASSERT_TRUE(GroupHandle0.IsValid());
	UE_NET_ASSERT_TRUE(GroupHandle1.IsValid());

	// Add Objects to groups
	Server->ReplicationSystem->AddToGroup(GroupHandle0, ServerHandle);
	Server->ReplicationSystem->AddToGroup(GroupHandle1, ServerHandle);

	// Update filters
	Filter();

	const FNetBitArrayView ObjectGroupFilter = ServerFiltering->GetGroupFilteredOutObjects(Client->ConnectionIdOnServer);

	// Verify that object is not filtered out
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, true);

	// Add groups to group filter
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle0);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle1);

	// Filter out objects in the group
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle0, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle1, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	// Update filters
	Filter();

	// We now expect the object to be filtered out
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, false);

	// Remote Filter Trait to first group
	Server->ReplicationSystem->RemoveGroupFilter(GroupHandle0);

	// Update filters
	Filter();

	// We still  expect the object to be filtered out since the object is a member of two groups
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, false);

	// Removee Filter Trait from second group
	Server->ReplicationSystem->RemoveGroupFilter(GroupHandle1);

	// Update filters
	Filter();

	// We now expect the object to no longer be filtered out
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, true);
}


UE_NET_TEST_FIXTURE(FTestGroupsFixture, GroupFilterTestObjectListViaTrait)
{
	// Spawn object on server
	UReplicatedTestObject* ServerObject0 = Server->CreateObject(0, 0);
	const FNetRefHandle ServerHandle = ServerObject0->NetRefHandle;
	
	const uint32 ServerInternalIndex = Server->GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager().GetInternalIndex(ServerHandle);
	UE_NET_ASSERT_NE((uint32)InvalidInternalNetRefIndex, ServerInternalIndex);

	// Create groups
	FNetObjectGroupHandle GroupHandle0 = Server->ReplicationSystem->CreateGroup(NAME_None);
	FNetObjectGroupHandle GroupHandle1 = Server->ReplicationSystem->CreateGroup(NAME_None);
	FNetObjectGroupHandle GroupHandle2 = Server->ReplicationSystem->CreateGroup(NAME_None);
	UE_NET_ASSERT_TRUE(GroupHandle0.IsValid());
	UE_NET_ASSERT_TRUE(GroupHandle1.IsValid());
	UE_NET_ASSERT_TRUE(GroupHandle2.IsValid());

	// Add Objects to groups
	Server->ReplicationSystem->AddToGroup(GroupHandle0, ServerHandle);
	Server->ReplicationSystem->AddToGroup(GroupHandle1, ServerHandle);
	Server->ReplicationSystem->AddToGroup(GroupHandle2, ServerHandle);

	// Update filters
	Filter();

	const FNetBitArrayView GlobalGroupFilterList = ServerGroups->GetGroupFilteredOutObjects();

	// Object is not in a group filter yet
	UE_NET_ASSERT_FALSE(GlobalGroupFilterList.IsBitSet(ServerInternalIndex));
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, true);

	// Set two groups to start filtering
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle0);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle2);

	Filter();

	// Object should be in the group filter list now
	UE_NET_ASSERT_TRUE(GlobalGroupFilterList.IsBitSet(ServerInternalIndex));
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, false);

	// Remove the filter trait on the first group
	Server->ReplicationSystem->RemoveGroupFilter(GroupHandle0);

	Filter();

	// Object should still be in the group filter list
	UE_NET_ASSERT_TRUE(GlobalGroupFilterList.IsBitSet(ServerInternalIndex));
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, false);

	// Remove the filter trait on the second group
	Server->ReplicationSystem->RemoveGroupFilter(GroupHandle2);

	Filter();

	// Object should not be part of a filter group anymore
	UE_NET_ASSERT_FALSE(GlobalGroupFilterList.IsBitSet(ServerInternalIndex));
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, true);

	// Add the filter trait to the last group
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle1);

	Filter();

	// Object should be in the group list again.
	UE_NET_ASSERT_TRUE(GlobalGroupFilterList.IsBitSet(ServerInternalIndex));
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, false);
}


UE_NET_TEST_FIXTURE(FTestGroupsFixture, GroupFilterTestObjectListViaMembership)
{
	// Spawn object on server
	UReplicatedTestObject* ServerObject0 = Server->CreateObject(0, 0);
	const FNetRefHandle ServerHandle = ServerObject0->NetRefHandle;

	const uint32 ServerInternalIndex = Server->GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager().GetInternalIndex(ServerHandle);
	UE_NET_ASSERT_NE((uint32)InvalidInternalNetRefIndex, ServerInternalIndex);

	// Create groups
	FNetObjectGroupHandle GroupHandle0 = Server->ReplicationSystem->CreateGroup(NAME_None);
	FNetObjectGroupHandle GroupHandle1 = Server->ReplicationSystem->CreateGroup(NAME_None);
	FNetObjectGroupHandle GroupHandle2 = Server->ReplicationSystem->CreateGroup(NAME_None);
	UE_NET_ASSERT_TRUE(GroupHandle0.IsValid());
	UE_NET_ASSERT_TRUE(GroupHandle1.IsValid());
	UE_NET_ASSERT_TRUE(GroupHandle2.IsValid());

	// Add Objects to groups
	Server->ReplicationSystem->AddToGroup(GroupHandle0, ServerHandle);
	Server->ReplicationSystem->AddToGroup(GroupHandle1, ServerHandle);
	Server->ReplicationSystem->AddToGroup(GroupHandle2, ServerHandle);

	// Set two groups to start filtering
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle0);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle2);

	// Update filters
	Filter();

	const FNetBitArrayView GlobalGroupFilterList = ServerGroups->GetGroupFilteredOutObjects();

	// Object should be in a group filter
	UE_NET_ASSERT_TRUE(GlobalGroupFilterList.IsBitSet(ServerInternalIndex));
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, false);

	// Remove the object from the first filter group
	Server->ReplicationSystem->RemoveFromGroup(GroupHandle0, ServerHandle);

	Filter();

	// Should still be in the filtered list
	UE_NET_ASSERT_TRUE(GlobalGroupFilterList.IsBitSet(ServerInternalIndex));
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, false);

	// Remove the object from the second filter group
	Server->ReplicationSystem->RemoveFromGroup(GroupHandle2, ServerHandle);

	Filter();

	// Is no longer in a group with the filter trait
	UE_NET_ASSERT_FALSE(GlobalGroupFilterList.IsBitSet(ServerInternalIndex));
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, true);

	// Remove from the last non-filter group
	Server->ReplicationSystem->RemoveFromGroup(GroupHandle1, ServerHandle);

	Filter();

	// Still not filtered
	UE_NET_ASSERT_FALSE(GlobalGroupFilterList.IsBitSet(ServerInternalIndex));
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, true);

	// Add back to a filter group
	Server->ReplicationSystem->AddToGroup(GroupHandle2, ServerHandle);

	Filter();

	// Now filtered again
	UE_NET_ASSERT_TRUE(GlobalGroupFilterList.IsBitSet(ServerInternalIndex));
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, false);
}

// Group filtering tests
UE_NET_TEST_FIXTURE(FTestGroupsFixture, GroupFilterAPINotFilteredGroup)
{
	// Spawn object on server
	UReplicatedTestObject* ServerObject0 = Server->CreateObject(0, 0);
	UReplicatedTestObject* ServerObject1 = Server->CreateObject(0, 0);
	UReplicatedTestObject* ServerObject2 = Server->CreateObject(0, 0);

	// Add Objects to NotFilteredGroup
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject0->NetRefHandle);

	// Update filters
	Filter();

	const FNetBitArrayView ObjectGroupFilter = ServerFiltering->GetGroupFilteredOutObjects(Client->ConnectionIdOnServer);

	// We expect the filter to filter out the object	
	UE_NET_ASSERT_TRUE(ObjectGroupFilter.IsAnyBitSet());

	// Remove object from NotReplicatedNetObjectGroup
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject0->NetRefHandle);

	// Update filters
	Filter();

	// We now expect no filtering
	UE_NET_ASSERT_TRUE(ObjectGroupFilter.IsNoBitSet());
}

// Verify that group filtering on subobjects works as expected
UE_NET_TEST_FIXTURE(FTestGroupsFixture, GroupFilterAPISubObjects)
{
	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	const FNetRefHandle ServerHandle = ServerObject->NetRefHandle;

	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerHandle, 0, 0);	
	const FNetRefHandle ServerSubObjectHandle = ServerSubObject->NetRefHandle;
	
	// Create groups
	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle);

	// Add Object to group
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerHandle);

	// Update filters
	Filter();

	// Verify that both object and subobject are filtered out
	const FNetBitArrayView ObjectGroupFilter = ServerFiltering->GetGroupFilteredOutObjects(Client->ConnectionIdOnServer);

	const bool bExpectsReplicationToBeAllowed = false;
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, bExpectsReplicationToBeAllowed);
	VerifyObjectFilterStatus(ServerSubObjectHandle, Client->ConnectionIdOnServer, bExpectsReplicationToBeAllowed);
}

// Verify that group filtering on subobjects works as expected and that it is restored if subobject is removed same frame
UE_NET_TEST_FIXTURE(FTestGroupsFixture, GroupFilterAPIRemovingSubObjectsRestoresGroupFilteredOutObjects)
{
	// Create groups
	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle);

	// Update filters
	Filter();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	const FNetRefHandle ServerHandle = ServerObject->NetRefHandle;

	// Add Object to group
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerHandle);


	// Create sub object
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerHandle, 0, 0);
	const FNetRefHandle ServerSubObjectHandle = ServerSubObject->NetRefHandle;

	// destroy subobject in the same frame
	Server->DestroyObject(ServerSubObject);

	// Update filters
	Filter();

	// Verify that both object and subobject are filtered out
	const FNetBitArrayView ObjectGroupFilter = ServerFiltering->GetGroupFilteredOutObjects(Client->ConnectionIdOnServer);

	const bool bExpectsReplicationToBeAllowed = false;
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, bExpectsReplicationToBeAllowed);
}



UE_NET_TEST_FIXTURE(FTestGroupsFixture, GroupFilterAPILateAddedSubObjectIsFilteredOut)
{
	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	const FNetRefHandle ServerHandle = ServerObject->NetRefHandle;
	
	// Create groups
	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle);

	// Add Object to group
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerHandle);

	// Update filters
	Filter();

	// Add subobject
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerHandle, 0, 0);	
	const FNetRefHandle ServerSubObjectHandle = ServerSubObject->NetRefHandle;

	// Update filters
	Filter();

	// Verify that both object and subobject are filtered out
	const FNetBitArrayView ObjectGroupFilter = ServerFiltering->GetGroupFilteredOutObjects(Client->ConnectionIdOnServer);

	const bool bExpectsReplicationToBeAllowed = false;
	VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, bExpectsReplicationToBeAllowed);
	VerifyObjectFilterStatus(ServerSubObjectHandle, Client->ConnectionIdOnServer, bExpectsReplicationToBeAllowed);
}

UE_NET_TEST_FIXTURE(FTestGroupsFixture, GroupFilterAPISubObjectIsFilteredOutIfOwnerIsFilterIsChanged)
{
	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	const FNetRefHandle ServerHandle = ServerObject->NetRefHandle;

	// Add subobject
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerHandle, 0, 0);	
	const FNetRefHandle ServerSubObjectHandle = ServerSubObject->NetRefHandle;
	
	// Create groups
	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	FNetObjectGroupHandle SubObjectGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);

	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Allow);

	// Add Object to group
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerHandle);

	// Update filters
	Filter();

	// Verify that we allow replication for both object and subobject
	{
		const bool bExpectsReplicationToBeAllowed = true;
		VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, bExpectsReplicationToBeAllowed);
		VerifyObjectFilterStatus(ServerSubObjectHandle, Client->ConnectionIdOnServer, bExpectsReplicationToBeAllowed);
	}

	// Disallow group filter
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Disallow);

	// Update filters
	Filter();

	// Verify that both object and subobject are filtered out
	{
		const bool bExpectsReplicationToBeAllowed = false;
		VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, bExpectsReplicationToBeAllowed);
		VerifyObjectFilterStatus(ServerSubObjectHandle, Client->ConnectionIdOnServer, bExpectsReplicationToBeAllowed);
	}
}

UE_NET_TEST_FIXTURE(FTestGroupsFixture, GroupFilterAPISubObjectIsFilteredOutWithSeparateFilter)
{
	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	const FNetRefHandle ServerHandle = ServerObject->NetRefHandle;

	// Add subobject
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerHandle, 0, 0);	
	const FNetRefHandle ServerSubObjectHandle = ServerSubObject->NetRefHandle;
	
	// Create groups
	FNetObjectGroupHandle GroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	FNetObjectGroupHandle SubObjectGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);

	Server->ReplicationSystem->AddExclusionFilterGroup(GroupHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(SubObjectGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Allow);

	// Add Object to group
	Server->ReplicationSystem->AddToGroup(GroupHandle, ServerHandle);
	Server->ReplicationSystem->AddToGroup(SubObjectGroupHandle, ServerSubObjectHandle);

	// Update filters
	Filter();

	// Verify that we only allow replication for owner
	{
		VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, true);
		VerifyObjectFilterStatus(ServerSubObjectHandle, Client->ConnectionIdOnServer, false);
	}

	// Enable SubObjectGroupHandle
	Server->ReplicationSystem->SetGroupFilterStatus(SubObjectGroupHandle, ENetFilterStatus::Allow);

	// Update filters
	Filter();

	// Verify that both object and subobject are enabled
	{
		VerifyObjectFilterStatus(ServerHandle, Client->ConnectionIdOnServer, true);
		VerifyObjectFilterStatus(ServerSubObjectHandle, Client->ConnectionIdOnServer, true);
	}
}

}
