// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"

#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilterDefinitions.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectGridFilter.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectGroups.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectParentRelevancyGridFilter.h"
#include "Iris/ReplicationSystem/WorldLocations.h"

#include "Tests/ReplicationSystem/Filtering/TestFilteringObject.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"

namespace UE::Net::Private
{

class FTestParentRelevancyGridFilterFixture : public FReplicationSystemServerClientTestFixture
{
protected:
	virtual void SetUp() override
	{
		InitFilterDefinitions();
		FReplicationSystemServerClientTestFixture::SetUp();
		InitFilterHandles();

		Server->GetReplicationBridge()->SetExternalWorldLocationUpdateFunctor([this](FNetRefHandle, const UObject* Obj, FVector& OutLoc, float& OutCullDistance)
		{
			const FObjectWorldInfo& Info = GetWorldInfo(Obj);
			OutLoc = Info.Loc;
			OutCullDistance = Info.CullDistance;
		});
	}

	virtual void TearDown() override
	{
		FReplicationSystemServerClientTestFixture::TearDown();
		RestoreFilterDefinitions();
	}

private:
	void InitFilterDefinitions()
	{
		const UClass* DefinitionsClass = UNetObjectFilterDefinitions::StaticClass();
		const FProperty* DefinitionsProperty = DefinitionsClass->FindPropertyByName("NetObjectFilterDefinitions");
		check(DefinitionsProperty != nullptr);

		// Save current filter definitions and install only the filter under test.
		UNetObjectFilterDefinitions* FilterDefinitions = GetMutableDefault<UNetObjectFilterDefinitions>();
		DefinitionsProperty->CopyCompleteValue(&OriginalFilterDefinitions, (void*)(UPTRINT(FilterDefinitions) + DefinitionsProperty->GetOffset_ForInternal()));

		TArray<FNetObjectFilterDefinition> NewDefinitions;
		FNetObjectFilterDefinition& Def = NewDefinitions.Emplace_GetRef();
		Def.FilterName = "ParentRelevancyGrid";
		Def.ClassName = "/Script/IrisCore.NetObjectParentRelevancyGridFilter";
		Def.ConfigClassName = "/Script/IrisCore.NetObjectParentRelevancyGridFilterConfig";

		DefinitionsProperty->CopyCompleteValue((void*)(UPTRINT(FilterDefinitions) + DefinitionsProperty->GetOffset_ForInternal()), &NewDefinitions);

		// Add a profile that disables culling hysteresis
		UNetObjectGridFilterConfig* GridFilterConfig = GetMutableDefault<UNetObjectParentRelevancyGridFilterConfig>();
		OriginalFilterProfiles = GridFilterConfig->FilterProfiles;
		GridFilterConfig->FilterProfiles.Add(FNetObjectGridFilterProfile{
			.FilterProfileName = TEXT("NoHysteresis"),
			.FrameCountBeforeCulling = 1,
		});
	}

	void RestoreFilterDefinitions()
	{
		const UClass* DefinitionsClass = UNetObjectFilterDefinitions::StaticClass();
		const FProperty* DefinitionsProperty = DefinitionsClass->FindPropertyByName("NetObjectFilterDefinitions");
		UNetObjectFilterDefinitions* FilterDefinitions = GetMutableDefault<UNetObjectFilterDefinitions>();
		DefinitionsProperty->CopyCompleteValue((void*)(UPTRINT(FilterDefinitions) + DefinitionsProperty->GetOffset_ForInternal()), &OriginalFilterDefinitions);
		OriginalFilterDefinitions.Empty();

		FilterHandle = InvalidNetObjectFilterHandle;

		UNetObjectGridFilterConfig* GridFilterConfig = GetMutableDefault<UNetObjectParentRelevancyGridFilterConfig>();
		GridFilterConfig->FilterProfiles = OriginalFilterProfiles;
	}

	void InitFilterHandles()
	{
		FilterHandle = Server->GetReplicationSystem()->GetFilterHandle("ParentRelevancyGrid");
	}

public:
	struct FObjectWorldInfo
	{
		FVector Loc = FVector::ZeroVector;
		float CullDistance = 1500.f;
	};

	FObjectWorldInfo GetWorldInfo(const UObject* ReplicatedObject)
	{
		if (!ObjectWorldInfoMap.Contains(ReplicatedObject))
		{
			checkf(!WorldInfoToBeAssigned.IsEmpty(), TEXT("No info was pushed for assignation"));
			FObjectWorldInfo NewInfo = WorldInfoToBeAssigned[0];
			WorldInfoToBeAssigned.RemoveAt(0);
			ObjectWorldInfoMap.Add(ReplicatedObject, NewInfo);
		}
		return ObjectWorldInfoMap[ReplicatedObject];
	}

	void SetWorldInfo(const UObject* ReplicatedObject, const FObjectWorldInfo& WorldInfo)
	{
		ObjectWorldInfoMap.FindOrAdd(ReplicatedObject) = WorldInfo;
	}

	UReplicatedTestObject* CreateFilteredObject(FObjectWorldInfo Info)
	{
		WorldInfoToBeAssigned.Add(Info);

		UObjectReplicationBridge::FRootObjectReplicationParams Params;
		Params.bNeedsWorldLocationUpdate = true;
		Params.bIsDormant = false;
		Params.bUseClassConfigDynamicFilter = false;
		Params.bUseExplicitDynamicFilter = true;
		Params.ExplicitDynamicFilterName = FName("ParentRelevancyGrid");

		return Server->CreateObject(Params);
	}

	void SetClientViewAt(FReplicationSystemTestClient* Client, const FVector& Pos)
	{
		FReplicationView View;
		FReplicationView::FView& ViewEntry = View.Views.Emplace_GetRef();
		ViewEntry.Pos = Pos;
		Server->ReplicationSystem->SetReplicationView(Client->ConnectionIdOnServer, View);
	}

protected:
	TArray<FObjectWorldInfo> WorldInfoToBeAssigned;
	TMap<const UObject*, FObjectWorldInfo> ObjectWorldInfoMap;
	FNetObjectFilterHandle FilterHandle = InvalidNetObjectFilterHandle;

private:
	TArray<FNetObjectFilterDefinition> OriginalFilterDefinitions;
	TArray<FNetObjectGridFilterProfile> OriginalFilterProfiles;
};

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, ChildInRangePromotesAllAncestorsToRelevant)
{
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Root far from origin (out of cull range), Mid moderately far, Leaf at origin.
	UReplicatedTestObject* Root = CreateFilteredObject({ .Loc = FVector(50000.f, 50000.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Mid  = CreateFilteredObject({ .Loc = FVector(25000.f, 25000.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Leaf = CreateFilteredObject({ .Loc = FVector(0.f, 0.f, 0.f),         .CullDistance = 1500.f });

	// Wire the chain Root -> Mid -> Leaf via creation deps.
	Bridge->AddCreationDependencyLink(Root->NetRefHandle, Mid->NetRefHandle);
	Bridge->AddCreationDependencyLink(Mid->NetRefHandle, Leaf->NetRefHandle);

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector::ZeroVector);

	Server->UpdateAndSend({ Client });

	// Leaf is in cull range; Mid and Root are not but should still be relevant via propagation.
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Mid->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Root->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, NoChildInRangeKeepsAncestorsCulled)
{
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	UReplicatedTestObject* Root = CreateFilteredObject({ .Loc = FVector(50000.f, 50000.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Mid  = CreateFilteredObject({ .Loc = FVector(25000.f, 25000.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Leaf = CreateFilteredObject({ .Loc = FVector(10000.f, 10000.f, 0.f), .CullDistance = 1500.f });

	Bridge->AddCreationDependencyLink(Root->NetRefHandle, Mid->NetRefHandle);
	Bridge->AddCreationDependencyLink(Mid->NetRefHandle, Leaf->NetRefHandle);

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector::ZeroVector);

	Server->UpdateAndSend({ Client });

	// Nothing within cull range - nothing should be relevant.
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Mid->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Root->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, MidViewerLightsOnlyMidAndRoot)
{
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	UReplicatedTestObject* Root = CreateFilteredObject({ .Loc = FVector(50000.f, 0.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Mid  = CreateFilteredObject({ .Loc = FVector(25000.f, 0.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Leaf = CreateFilteredObject({ .Loc = FVector(0.f, 0.f, 0.f),     .CullDistance = 1500.f });

	Bridge->AddCreationDependencyLink(Root->NetRefHandle, Mid->NetRefHandle);
	Bridge->AddCreationDependencyLink(Mid->NetRefHandle, Leaf->NetRefHandle);

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector(25000.f, 0.f, 0.f));

	Server->UpdateAndSend({ Client });

	// Mid is in range; Root is propagated; Leaf is far from view.
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Mid->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Root->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, SiblingsSharingImmediateParentBothPromoteCorrectly)
{
	// Two sibling leaves share the chain Parent -> Grandparent -> Root. After the first leaf walks it, the second must take the all-immediates-visited skip and the end state must still have all five objects relevant.
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Root and Grandparent placed far from the viewer; Parent slightly closer but still out of cull range.
	UReplicatedTestObject* Root        = CreateFilteredObject({ .Loc = FVector(80000.f, 0.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Grandparent = CreateFilteredObject({ .Loc = FVector(60000.f, 0.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Parent      = CreateFilteredObject({ .Loc = FVector(40000.f, 0.f, 0.f), .CullDistance = 1500.f });
	// Two sibling leaves near origin, both in range.
	UReplicatedTestObject* LeafA = CreateFilteredObject({ .Loc = FVector(0.f, 0.f, 0.f),   .CullDistance = 1500.f });
	UReplicatedTestObject* LeafB = CreateFilteredObject({ .Loc = FVector(100.f, 0.f, 0.f), .CullDistance = 1500.f });

	Bridge->AddCreationDependencyLink(Root->NetRefHandle, Grandparent->NetRefHandle);
	Bridge->AddCreationDependencyLink(Grandparent->NetRefHandle, Parent->NetRefHandle);
	Bridge->AddCreationDependencyLink(Parent->NetRefHandle, LeafA->NetRefHandle);
	Bridge->AddCreationDependencyLink(Parent->NetRefHandle, LeafB->NetRefHandle);

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector::ZeroVector);

	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(LeafA->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(LeafB->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Parent->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Grandparent->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Root->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, DiamondGraphPromotesSharedAncestor)
{
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Diamond:  Grand <- ParentA <- Leaf
	//           Grand <- ParentB <- Leaf  (Leaf depends on both branches)
	UReplicatedTestObject* Grand   = CreateFilteredObject({ .Loc = FVector(50000.f, 0.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* ParentA = CreateFilteredObject({ .Loc = FVector(40000.f, 0.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* ParentB = CreateFilteredObject({ .Loc = FVector(40000.f, 100.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Leaf    = CreateFilteredObject({ .Loc = FVector(0.f, 0.f, 0.f),     .CullDistance = 1500.f });

	Bridge->AddCreationDependencyLink(Grand->NetRefHandle, ParentA->NetRefHandle);
	Bridge->AddCreationDependencyLink(Grand->NetRefHandle, ParentB->NetRefHandle);
	Bridge->AddCreationDependencyLink(ParentA->NetRefHandle, Leaf->NetRefHandle);
	Bridge->AddCreationDependencyLink(ParentB->NetRefHandle, Leaf->NetRefHandle);

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector::ZeroVector);

	Server->UpdateAndSend({ Client });

	// Leaf in range; both parents and the shared grandparent should be relevant.
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentA->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentB->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Grand->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, OrphanObjectBehavesLikePlainGridFilter)
{
	// Object with no creation dependencies - plain cull-distance behavior expected.
	UReplicatedTestObject* InRange  = CreateFilteredObject({ .Loc = FVector::ZeroVector, .CullDistance = 1500.f });
	UReplicatedTestObject* OutRange = CreateFilteredObject({ .Loc = FVector(50000.f, 0.f, 0.f), .CullDistance = 1500.f });

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector::ZeroVector);

	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(InRange->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(OutRange->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, ChildLeavingRangeAlsoCullsAncestors)
{
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	UReplicatedTestObject* Root = CreateFilteredObject({ .Loc = FVector(50000.f, 0.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Leaf = CreateFilteredObject({ .Loc = FVector::ZeroVector,        .CullDistance = 1500.f });

	Bridge->AddCreationDependencyLink(Root->NetRefHandle, Leaf->NetRefHandle);

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector::ZeroVector);

	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Root->NetRefHandle));

	// Move the leaf out of range. Hysteresis countdown will eventually drop both objects.
	SetWorldInfo(Leaf, FObjectWorldInfo{ .Loc = FVector(50000.f, 50000.f, 0.f), .CullDistance = 1500.f });
	Server->ReplicationSystem->MarkDirty(Leaf->NetRefHandle);

	// Run frames until hysteresis drops both objects, capped at 60.
	for (uint32 FrameIt = 0; FrameIt < 60; ++FrameIt)
	{
		Server->UpdateAndSend({ Client });

		if (!Client->IsResolvableNetRefHandle(Leaf->NetRefHandle) && !Client->IsResolvableNetRefHandle(Root->NetRefHandle))
		{
			break;
		}
	}

	// Leaf out of range with no other descendants of Root keeping it relevant - both should be culled.
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Root->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, LinearChainPromotesIntermediateWhenAlternatePathPreVisitsAncestor)
{
	// Regression: an alternate path Root -> AltParent -> AltLeaf marks Root visited; MainLeaf's chain Root -> Grandparent -> Parent -> MainLeaf must still promote Grandparent after the linear early-break stops on Root.
	//
	//                  Root
	//                 /    \
	//          Grandparent  AltParent
	//               |          |
	//             Parent     AltLeaf
	//               |
	//            MainLeaf
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	UReplicatedTestObject* Root = CreateFilteredObject({ .Loc = FVector(80000.f, 0.f, 0.f), .CullDistance = 1500.f });

	// Alternate path: Root -> AltParent -> AltLeaf. AltLeaf's processing will mark Root visited without ever promoting Grandparent.
	UReplicatedTestObject* AltParent = CreateFilteredObject({ .Loc = FVector(80000.f, 100.f, 0.f), .CullDistance = 1500.f });
	Bridge->AddCreationDependencyLink(Root->NetRefHandle, AltParent->NetRefHandle);

	UReplicatedTestObject* AltLeaf = CreateFilteredObject({ .Loc = FVector(0.f, 100.f, 0.f), .CullDistance = 1500.f });
	Bridge->AddCreationDependencyLink(AltParent->NetRefHandle, AltLeaf->NetRefHandle);

	// Main path: Root -> Grandparent -> Parent -> MainLeaf.
	UReplicatedTestObject* Grandparent = CreateFilteredObject({ .Loc = FVector(80000.f, 200.f, 0.f), .CullDistance = 1500.f });
	Bridge->AddCreationDependencyLink(Root->NetRefHandle, Grandparent->NetRefHandle);

	UReplicatedTestObject* Parent = CreateFilteredObject({ .Loc = FVector(80000.f, 300.f, 0.f), .CullDistance = 1500.f });
	Bridge->AddCreationDependencyLink(Grandparent->NetRefHandle, Parent->NetRefHandle);

	UReplicatedTestObject* MainLeaf = CreateFilteredObject({ .Loc = FVector(0.f, 0.f, 0.f), .CullDistance = 1500.f });
	Bridge->AddCreationDependencyLink(Parent->NetRefHandle, MainLeaf->NetRefHandle);

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector::ZeroVector);

	Server->UpdateAndSend({ Client });

	// Both leaves are in cull range; every shared/intermediate parent must be promoted by parent propagation.
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MainLeaf->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(AltLeaf->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Parent->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(AltParent->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Grandparent->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Root->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, NonLinearChainPromotesUnvisitedImmediateBranchesWhenOnlyLastImmediateWasPreVisited)
{
	// When a leaf has multiple immediate creation-dependency parents, the all-immediates-visited skip must only fire once every immediate parent has been promoted by an earlier walk.
	// PreVisitor promotes ParentC and RootC first; MultiParentLeaf must then walk its other immediate-parent branches (ParentA -> RootA, ParentB -> RootB) normally.
	//
	//    RootA     RootB     RootC
	//      |         |         |
	//   ParentA   ParentB   ParentC
	//        \      |      /    \
	//         \     |     /      PreVisitor
	//          \    |    /
	//        MultiParentLeaf
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Three private branches. Each Root has only one child (its own Parent) so the chains stay linear
	// from the perspective of any single descendant.
	UReplicatedTestObject* RootA = CreateFilteredObject({ .Loc = FVector(80000.f, 100.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* RootB = CreateFilteredObject({ .Loc = FVector(80000.f, 200.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* RootC = CreateFilteredObject({ .Loc = FVector(80000.f, 300.f, 0.f), .CullDistance = 1500.f });

	UReplicatedTestObject* ParentA = CreateFilteredObject({ .Loc = FVector(80000.f, 110.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* ParentB = CreateFilteredObject({ .Loc = FVector(80000.f, 210.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* ParentC = CreateFilteredObject({ .Loc = FVector(80000.f, 310.f, 0.f), .CullDistance = 1500.f });
	Bridge->AddCreationDependencyLink(RootA->NetRefHandle, ParentA->NetRefHandle);
	Bridge->AddCreationDependencyLink(RootB->NetRefHandle, ParentB->NetRefHandle);
	Bridge->AddCreationDependencyLink(RootC->NetRefHandle, ParentC->NetRefHandle);

	// PreVisitor processed first (lower index). Its chain only walks ParentC -> RootC.
	UReplicatedTestObject* PreVisitor = CreateFilteredObject({ .Loc = FVector(0.f, 300.f, 0.f), .CullDistance = 1500.f });
	Bridge->AddCreationDependencyLink(ParentC->NetRefHandle, PreVisitor->NetRefHandle);

	// MultiParentLeaf has three immediate creation-dep parents; the chain is non-linear and ParentC is the last
	// of the three to appear in the immediate-parent prefix.
	UReplicatedTestObject* MultiParentLeaf = CreateFilteredObject({ .Loc = FVector(0.f, 0.f, 0.f), .CullDistance = 1500.f });
	Bridge->AddCreationDependencyLink(ParentA->NetRefHandle, MultiParentLeaf->NetRefHandle);
	Bridge->AddCreationDependencyLink(ParentB->NetRefHandle, MultiParentLeaf->NetRefHandle);
	Bridge->AddCreationDependencyLink(ParentC->NetRefHandle, MultiParentLeaf->NetRefHandle);

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector::ZeroVector);

	Server->UpdateAndSend({ Client });

	// Every entry of every chain must be relevant - including RootA and RootB which are only reachable through the two immediate-parent branches that PreVisitor never touched.
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(PreVisitor->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MultiParentLeaf->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentA->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentB->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentC->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(RootA->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(RootB->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(RootC->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, ChainWalksThroughCreationDependencyParentOwnedByAnotherFilter)
{
	// Chain: Leaf -> Mid -> AlwaysRelevantRoot -> SuperRoot
	// AlwaysRelevantRoot is using a different filter.
	// BuildParentChain must walk through AlwaysRelevantRoot to keep SuperRoot in Leaf's chain and promote it too

	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	UReplicatedTestObject* SuperRoot = CreateFilteredObject({ .Loc = FVector(80000.f, 0.f, 0.f), .CullDistance = 1500.f });

	UReplicatedTestObject* AlwaysRelevantRoot = Server->CreateObject();
	Bridge->AddCreationDependencyLink(SuperRoot->NetRefHandle, AlwaysRelevantRoot->NetRefHandle);

	UReplicatedTestObject* Mid = CreateFilteredObject({ .Loc = FVector(80000.f, 100.f, 0.f), .CullDistance = 1500.f });
	Bridge->AddCreationDependencyLink(AlwaysRelevantRoot->NetRefHandle, Mid->NetRefHandle);

	UReplicatedTestObject* Leaf = CreateFilteredObject({ .Loc = FVector::ZeroVector, .CullDistance = 1500.f });
	Bridge->AddCreationDependencyLink(Mid->NetRefHandle, Leaf->NetRefHandle);

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector::ZeroVector);

	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Mid->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(AlwaysRelevantRoot->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(SuperRoot->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, LeafExcludedByGroupFilterMustNotPromoteAncestors)
{
	// A leaf that is excluded for this connection by a group filter must not pull its ancestors into scope.
	// Walking that leaf's parent chain would leak ancestors to a connection that should see none of the chain.
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	UReplicatedTestObject* Root = CreateFilteredObject({ .Loc = FVector(80000.f, 0.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Mid  = CreateFilteredObject({ .Loc = FVector(40000.f, 0.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Leaf = CreateFilteredObject({ .Loc = FVector::ZeroVector,        .CullDistance = 1500.f });

	Bridge->AddCreationDependencyLink(Root->NetRefHandle, Mid->NetRefHandle);
	Bridge->AddCreationDependencyLink(Mid->NetRefHandle, Leaf->NetRefHandle);

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector::ZeroVector);

	// Exclude the leaf from the test client via a group exclusion filter.
	FNetObjectGroupHandle ExclusionGroup = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(ExclusionGroup, Leaf->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(ExclusionGroup);
	Server->ReplicationSystem->SetGroupFilterStatus(ExclusionGroup, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Mid->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Root->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, AncestorExcludedByGroupFilterMustNotForcePromoteHigherAncestors)
{
	// When an intermediate ancestor is group-excluded for a connection, the chain walk must stop there.
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	UReplicatedTestObject* Root = CreateFilteredObject({ .Loc = FVector(80000.f, 0.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Mid  = CreateFilteredObject({ .Loc = FVector(40000.f, 0.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Leaf = CreateFilteredObject({ .Loc = FVector::ZeroVector,        .CullDistance = 1500.f });

	Bridge->AddCreationDependencyLink(Root->NetRefHandle, Mid->NetRefHandle);
	Bridge->AddCreationDependencyLink(Mid->NetRefHandle, Leaf->NetRefHandle);

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector::ZeroVector);

	// Exclude Mid (an ancestor) from the test client via a group exclusion filter.
	FNetObjectGroupHandle ExclusionGroup = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(ExclusionGroup, Mid->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(ExclusionGroup);
	Server->ReplicationSystem->SetGroupFilterStatus(ExclusionGroup, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Mid->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Root->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, LeafWithDiamondChainExcludedOnOneBranchPromotesNothing)
{
	// Non-linear case: a leaf with two immediate creation-dependency parents sharing a common Grandparent.
	//
	//          Grandparent
	//          /         \
	//      ParentA      ParentB
	//          \         /
	//             Leaf
	//
	// ParentA is excluded so the Leaf and it's other parents cannot be relevant since all the creation dependencies must be relevant.

	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	UReplicatedTestObject* Grandparent = CreateFilteredObject({ .Loc = FVector(80000.f, 0.f,   0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* ParentA     = CreateFilteredObject({ .Loc = FVector(40000.f, 0.f,   0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* ParentB     = CreateFilteredObject({ .Loc = FVector(40000.f, 100.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Leaf        = CreateFilteredObject({ .Loc = FVector::ZeroVector,          .CullDistance = 1500.f });

	Bridge->AddCreationDependencyLink(Grandparent->NetRefHandle, ParentA->NetRefHandle);
	Bridge->AddCreationDependencyLink(Grandparent->NetRefHandle, ParentB->NetRefHandle);
	Bridge->AddCreationDependencyLink(ParentB->NetRefHandle, Leaf->NetRefHandle);
	Bridge->AddCreationDependencyLink(ParentA->NetRefHandle, Leaf->NetRefHandle);

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector::ZeroVector);

	// Exclude only ParentA. Leaf's other immediate (ParentB) and Grandparent are not in any exclusion group.
	FNetObjectGroupHandle ExclusionGroup = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(ExclusionGroup, ParentA->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(ExclusionGroup);
	Server->ReplicationSystem->SetGroupFilterStatus(ExclusionGroup, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentA->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentB->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Grandparent->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, CreationDependencyAddedAfterReplicationStartRebuildsChain)
{
	// Validates objects running autonomously, then later add a dependency between them.
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	UReplicatedTestObject* Parent = CreateFilteredObject({ .Loc = FVector(50000.f, 0.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Leaf   = CreateFilteredObject({ .Loc = FVector::ZeroVector,        .CullDistance = 1500.f });

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector::ZeroVector);

	// Phase 1: no creation dependencies
	Server->UpdateAndSend({ Client });
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Parent->NetRefHandle));

	// Phase 2: add a dependency between them
	Bridge->AddCreationDependencyLink(Parent->NetRefHandle, Leaf->NetRefHandle);

	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Parent->NetRefHandle));

	// Phase 3: remove the dependency

	Bridge->RemoveCreationDependencyLink(Parent->NetRefHandle, Leaf->NetRefHandle);

	for (uint32 FrameIt = 0; FrameIt < 60; ++FrameIt)
	{
		Server->UpdateAndSend({ Client });
		if (!Client->IsResolvableNetRefHandle(Parent->NetRefHandle))
		{
			break;
		}
	}

	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Parent->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, CreationDependencyRemovedAfterReplicationStart)
{
	// Validates that removing a creation dependency post start replication works

	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	UReplicatedTestObject* Parent = CreateFilteredObject({ .Loc = FVector(50000.f, 0.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Leaf   = CreateFilteredObject({ .Loc = FVector::ZeroVector,        .CullDistance = 1500.f });

	Bridge->AddCreationDependencyLink(Parent->NetRefHandle, Leaf->NetRefHandle);

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector::ZeroVector);

	// Phase 1: Objects start with a dependency
	
	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Parent->NetRefHandle));

	// Phase 2: Remove the dependency
	Bridge->RemoveCreationDependencyLink(Parent->NetRefHandle, Leaf->NetRefHandle);

	for (uint32 FrameIt = 0; FrameIt < 60; ++FrameIt)
	{
		Server->UpdateAndSend({ Client });
		
		if (!Client->IsResolvableNetRefHandle(Parent->NetRefHandle))
		{
			break;
		}
	}

	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Parent->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestParentRelevancyGridFilterFixture, CreationDependencyChangeOnParentAffectsChildren)
{
	// Validates that changing the dependency of a parent forces the children of that parent to rebuild their cached parent list.
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	UReplicatedTestObject* Root = CreateFilteredObject({ .Loc = FVector(80000.f, 0.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Mid  = CreateFilteredObject({ .Loc = FVector(40000.f, 0.f, 0.f), .CullDistance = 1500.f });
	UReplicatedTestObject* Leaf = CreateFilteredObject({ .Loc = FVector::ZeroVector,        .CullDistance = 1500.f });

	Bridge->AddCreationDependencyLink(Root->NetRefHandle, Mid->NetRefHandle);
	Bridge->AddCreationDependencyLink(Mid->NetRefHandle, Leaf->NetRefHandle);

	FReplicationSystemTestClient* Client = CreateClient();
	SetClientViewAt(Client, FVector::ZeroVector);

	// Phase 1: baseline - the whole chain replicates.
	Server->UpdateAndSend({ Client });
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Mid->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Root->NetRefHandle));

	// Phase 2: introduce a new parent and make Mid depend on it too.
	UReplicatedTestObject* NewParent = CreateFilteredObject({ .Loc = FVector(0.f, 80000.f, 0.f), .CullDistance = 1500.f });
	Bridge->AddCreationDependencyLink(NewParent->NetRefHandle, Mid->NetRefHandle);

	Server->UpdateAndSend({ Client });

	// The new parent should be forced relevant by the Leaf
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(NewParent->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Root->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Mid->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Leaf->NetRefHandle));
}

} // end namespace UE::Net::Private
