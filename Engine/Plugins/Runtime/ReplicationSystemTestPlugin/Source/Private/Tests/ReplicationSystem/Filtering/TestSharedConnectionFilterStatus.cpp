// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "IrisTestMessageStreamOperators.h"
#include "Iris/IrisConstants.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationSystem/Filtering/SharedConnectionFilterStatus.h"
#include "Logging/LogScopedVerbosityOverride.h"

namespace UE::Net::Private
{

class FSharedConnectionFilterStatusTestFixture : public FNetworkAutomationTestSuiteFixture
{
};

class FSharedConnectionFilterStatusCollectionTestFixture : public FNetworkAutomationTestSuiteFixture
{
};

// FSharedConnectionFilterStatus tests
UE_NET_TEST_FIXTURE(FSharedConnectionFilterStatusTestFixture, ReplicationIsDisallowedByDefault)
{
	FSharedConnectionFilterStatus FilterStatus;
	UE_NET_ASSERT_EQ(FilterStatus.GetFilterStatus(), ENetFilterStatus::Disallow);
}

UE_NET_TEST_FIXTURE(FSharedConnectionFilterStatusTestFixture, ReplicationIsNeverAllowedOnInvalidConnection)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogIrisFiltering, ELogVerbosity::Fatal);
	FSharedConnectionFilterStatus FilterStatus;
	FilterStatus.SetFilterStatus(FConnectionHandle(InvalidConnectionId), ENetFilterStatus::Allow);
	UE_NET_ASSERT_EQ(FilterStatus.GetFilterStatus(), ENetFilterStatus::Disallow);
}

UE_NET_TEST_FIXTURE(FSharedConnectionFilterStatusTestFixture, ConnectionWithFilterStatusEstablishesParentConnection)
{
	constexpr uint32 ParentConnId = 1U;
	constexpr uint32 OtherParentConnId = 2U;
	FConnectionHandle ConnHandle0(ParentConnId);
	FConnectionHandle ConnHandle1(OtherParentConnId);

	FSharedConnectionFilterStatus FilterStatus;
	FilterStatus.SetFilterStatus(ConnHandle0, ENetFilterStatus::Disallow);
	UE_NET_ASSERT_EQ(FilterStatus.GetParentConnectionId(), ConnHandle0.GetParentConnectionId());

	// Trying to set filter status for a different parent connection ID should not modify the group as parent connection ID has already been established
	{
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogIrisFiltering, ELogVerbosity::Fatal);
		FilterStatus.SetFilterStatus(ConnHandle1, ENetFilterStatus::Allow);
	}
	UE_NET_ASSERT_EQ(FilterStatus.GetParentConnectionId(), ConnHandle0.GetParentConnectionId());
	UE_NET_ASSERT_EQ(FilterStatus.GetFilterStatus(), ENetFilterStatus::Disallow);
}

UE_NET_TEST_FIXTURE(FSharedConnectionFilterStatusTestFixture, EstablishedParentConnectionIsKeptWhenRemovingChildConnection)
{
	constexpr uint32 ParentConnId = 1U;
	constexpr uint32 ChildConnId = 1U;
	FConnectionHandle ConnHandle(ParentConnId, ChildConnId);

	FSharedConnectionFilterStatus FilterStatus;
	FilterStatus.SetFilterStatus(ConnHandle, ENetFilterStatus::Allow);
	FilterStatus.RemoveConnection(ConnHandle);
	UE_NET_ASSERT_EQ(FilterStatus.GetParentConnectionId(), ConnHandle.GetParentConnectionId());
}

UE_NET_TEST_FIXTURE(FSharedConnectionFilterStatusTestFixture, EstablishedParentConnectionIsRemovedWhenRemovingParentConnection)
{
	constexpr uint32 ParentConnId = 1U;
	constexpr uint32 ChildConnId = 1U;
	FConnectionHandle ConnHandle(ParentConnId, ChildConnId);

	FSharedConnectionFilterStatus FilterStatus;
	FilterStatus.SetFilterStatus(ConnHandle, ENetFilterStatus::Allow);
	FilterStatus.RemoveConnection(FConnectionHandle(ConnHandle.GetParentConnectionId()));
	UE_NET_ASSERT_EQ(FilterStatus.GetParentConnectionId(), InvalidConnectionId);
}

UE_NET_TEST_FIXTURE(FSharedConnectionFilterStatusTestFixture, ReplicationIsDisallowedWhenNoConnectionAllows)
{
	constexpr uint32 ParentConnectionId = 4711U;
	FSharedConnectionFilterStatus FilterStatus;
	for (const uint32 ChildConnectionId : {0, 1, 3, 2, 7})
	{
		FilterStatus.SetFilterStatus(FConnectionHandle(4711U, ChildConnectionId), ENetFilterStatus::Disallow);
		UE_NET_ASSERT_EQ(FilterStatus.GetFilterStatus(), ENetFilterStatus::Disallow);
	}
}

UE_NET_TEST_FIXTURE(FSharedConnectionFilterStatusTestFixture, ReplicationIsAllowedWhenOneOrMoreConnectionAllows)
{
	constexpr uint32 ParentConnectionId = 7512U;
	FSharedConnectionFilterStatus FilterStatus;
	// Note that the child connection ID currently needs a mix of odd and even numbers as we're using that information to choose between Allow and Disallow.
	for (const uint32 ChildConnectionId : {0, 1, 3, 2, 7})
	{
		FilterStatus.SetFilterStatus(FConnectionHandle(4711U, ChildConnectionId), (ChildConnectionId & 1U ? ENetFilterStatus::Disallow :  ENetFilterStatus::Allow));
	}

	UE_NET_ASSERT_EQ(FilterStatus.GetFilterStatus(), ENetFilterStatus::Allow);
}

UE_NET_TEST_FIXTURE(FSharedConnectionFilterStatusTestFixture, FilterStatusIsAdjustedWhenConnectionIsRemoved)
{
	constexpr uint32 ParentConnId = 1U;
	FConnectionHandle AllowReplicationConnHandle(ParentConnId, 1U);
	FConnectionHandle DisallowReplicationConnHandle(ParentConnId, 2U);

	// Test allowing first and disallowing second before removing the allow connection
	{
		FSharedConnectionFilterStatus FilterStatus;
		FilterStatus.SetFilterStatus(AllowReplicationConnHandle, ENetFilterStatus::Allow);
		FilterStatus.SetFilterStatus(DisallowReplicationConnHandle, ENetFilterStatus::Disallow);
		FilterStatus.RemoveConnection(AllowReplicationConnHandle);
		UE_NET_ASSERT_EQ(FilterStatus.GetFilterStatus(), ENetFilterStatus::Disallow);
	}

	// Test disallowing first and allowing second before removing the allow connection
	{
		FSharedConnectionFilterStatus FilterStatus;
		FilterStatus.SetFilterStatus(DisallowReplicationConnHandle, ENetFilterStatus::Disallow);
		FilterStatus.SetFilterStatus(AllowReplicationConnHandle, ENetFilterStatus::Allow);
		FilterStatus.RemoveConnection(AllowReplicationConnHandle);
		UE_NET_ASSERT_EQ(FilterStatus.GetFilterStatus(), ENetFilterStatus::Disallow);
	}
}

UE_NET_TEST_FIXTURE(FSharedConnectionFilterStatusTestFixture, FilterStatusIsAdjustedWhenParentConnectionIsRemoved)
{
	constexpr uint32 ParentConnId = 1U;
	FConnectionHandle AllowReplicationConnHandle(ParentConnId, 1U);

	FSharedConnectionFilterStatus FilterStatus;
	FilterStatus.SetFilterStatus(AllowReplicationConnHandle, ENetFilterStatus::Allow);
	// Removing the parent connection should act as removing all child connections too.
	FilterStatus.RemoveConnection(FConnectionHandle(ParentConnId));
	UE_NET_ASSERT_EQ(FilterStatus.GetFilterStatus(), ENetFilterStatus::Disallow);
}

// FSharedConnectionFilterStatusCollection tests
UE_NET_TEST_FIXTURE(FSharedConnectionFilterStatusCollectionTestFixture, ReplicationIsDisallowedByDefault)
{
	FSharedConnectionFilterStatusCollection Collection;
	for (const uint32 ParentConnectionId : {InvalidConnectionId, 1U, 4711U, 99U})
	{
		UE_NET_ASSERT_EQ(Collection.GetFilterStatus(ParentConnectionId), ENetFilterStatus::Disallow);
	}
}

UE_NET_TEST_FIXTURE(FSharedConnectionFilterStatusCollectionTestFixture, ReplicationIsAllowedIfAnyChildConnectionAllows)
{
	FSharedConnectionFilterStatusCollection Collection;

	constexpr uint32 ParentConnectionIds[] = {1, 4711, 99};
	struct FChildFilterStatus
	{
		uint32 ChildConnectionId;
		ENetFilterStatus FilterStatus;
	}
	ChildFilterStatuses[] =
	{
		{0, ENetFilterStatus::Allow},
		{3, ENetFilterStatus::Disallow},
		{2, ENetFilterStatus::Disallow},
	};

	for (const uint32 ParentConnectionId : ParentConnectionIds)
	{
		for (const FChildFilterStatus& ChildFilterStatus : ChildFilterStatuses)
		{
			Collection.SetFilterStatus(FConnectionHandle(ParentConnectionId, ChildFilterStatus.ChildConnectionId), ChildFilterStatus.FilterStatus);
		}
	}

	// Now that all filter statuses have been set we can verify the result
	for (const uint32 ParentConnectionId : ParentConnectionIds)
	{
		UE_NET_ASSERT_EQ(Collection.GetFilterStatus(ParentConnectionId), ENetFilterStatus::Allow);
	}
}

UE_NET_TEST_FIXTURE(FSharedConnectionFilterStatusCollectionTestFixture, FilterStatusIsAdjustedWhenConnectionIsRemoved)
{
	FSharedConnectionFilterStatusCollection Collection;

	constexpr uint32 ParentConnectionIds[] = {1, 4711, 99};
	struct FChildFilterStatus
	{
		uint32 ChildConnectionId;
		ENetFilterStatus FilterStatus;
	}
	ChildFilterStatuses[] =
	{
		{1, ENetFilterStatus::Allow},
		{3, ENetFilterStatus::Disallow},
		{2, ENetFilterStatus::Disallow},
	};

	// Set filter status
	for (const uint32 ParentConnectionId : ParentConnectionIds)
	{
		for (const FChildFilterStatus& ChildFilterStatus : ChildFilterStatuses)
		{
			Collection.SetFilterStatus(FConnectionHandle(ParentConnectionId, ChildFilterStatus.ChildConnectionId), ChildFilterStatus.FilterStatus);
		}
	}

	// Remove all connections
	for (const uint32 ParentConnectionId : ParentConnectionIds)
	{
		for (const FChildFilterStatus& ChildFilterStatus : ChildFilterStatuses)
		{
			Collection.RemoveConnection(FConnectionHandle(ParentConnectionId, ChildFilterStatus.ChildConnectionId));
		}
	}

	// Now that all filter operations have been performed we can verify the result
	for (const uint32 ParentConnectionId : ParentConnectionIds)
	{
		UE_NET_ASSERT_EQ(Collection.GetFilterStatus(ParentConnectionId), ENetFilterStatus::Disallow);
	}
}

UE_NET_TEST_FIXTURE(FSharedConnectionFilterStatusCollectionTestFixture, FilterStatusIsAdjustedWhenParentConnectionIsRemoved)
{
	FSharedConnectionFilterStatusCollection Collection;

	constexpr uint32 ParentConnectionIds[] = {1, 4711, 99};
	struct FChildFilterStatus
	{
		uint32 ChildConnectionId;
		ENetFilterStatus FilterStatus;
	}
	ChildFilterStatuses[] =
	{
		{1, ENetFilterStatus::Allow},
		{3, ENetFilterStatus::Disallow},
		{2, ENetFilterStatus::Disallow},
	};

	// Set filter status
	for (const uint32 ParentConnectionId : ParentConnectionIds)
	{
		for (const FChildFilterStatus& ChildFilterStatus : ChildFilterStatuses)
		{
			Collection.SetFilterStatus(FConnectionHandle(ParentConnectionId, ChildFilterStatus.ChildConnectionId), ChildFilterStatus.FilterStatus);
		}
	}

	// Remove all parent connections
	for (const uint32 ParentConnectionId : ParentConnectionIds)
	{
		Collection.RemoveConnection(FConnectionHandle(ParentConnectionId));
	}

	// Now that all filter operations have been performed we can verify the result
	for (const uint32 ParentConnectionId : ParentConnectionIds)
	{
		UE_NET_ASSERT_EQ(Collection.GetFilterStatus(ParentConnectionId), ENetFilterStatus::Disallow);
	}
}

}
