// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/ReplicationSystem/MultiReplicationSystemsTestFixture.h"
#include "Containers/Array.h"
#include "Containers/Map.h"

namespace UE::Net
{

void FMultiReplicationSystemsTestFixture::SetUp()
{
	FNetworkAutomationTestSuiteFixture::SetUp();

	// Fake what we normally get from config
	DataStreamUtil.SetUp();
	DataStreamUtil.AddDataStreamDefinition(TEXT("NetToken"), TEXT("/Script/IrisCore.NetTokenDataStream"));
	DataStreamUtil.AddDataStreamDefinition(TEXT("Replication"), TEXT("/Script/IrisCore.ReplicationDataStream"));
	DataStreamUtil.FixupDefinitions();
}

void FMultiReplicationSystemsTestFixture::TearDown()
{
	for (const TMap<FReplicationSystemTestServer*, TArray<FReplicationSystemTestClient*>>::ElementType& ServerAndClient : ServerClients)
	{
		FReplicationSystemTestServer* Server = ServerAndClient.Key;
		delete Server;
		for (FReplicationSystemTestClient* Client : ServerAndClient.Value)
		{
			delete Client;
		}
	}

	Servers.Empty();
	ServerClients.Empty();

	DataStreamUtil.TearDown();

	FNetworkAutomationTestSuiteFixture::TearDown();
}

void FMultiReplicationSystemsTestFixture::CreateServers(unsigned ServerCount, const FReplicationSystemTestNode::FReplicationSystemParamsOverride* ParamsOverride)
{
	for (unsigned It = 0, EndIt = ServerCount; It != EndIt; ++It)
	{
		CreateServer(ParamsOverride);
	}
}

void FMultiReplicationSystemsTestFixture::CreateSomeServers()
{
	CreateServers(DefaultServerCount);
}

TArrayView<FReplicationSystemTestServer*> FMultiReplicationSystemsTestFixture::GetAllServers()
{
	return MakeArrayView(Servers);
}

FReplicationSystemTestServer* FMultiReplicationSystemsTestFixture::CreateServer(const FReplicationSystemTestNode::FReplicationSystemParamsOverride* ParamsOverride)
{
	FReplicationSystemTestServer* Server = new FReplicationSystemTestServer(FReplicationSystemTestNode::DelaySetup);
	Server->Setup(true, GetName(), ParamsOverride);
	Servers.Add(Server);
	ServerClients.Emplace(Server);

	return Server;
}

void FMultiReplicationSystemsTestFixture::DestroyServer(FReplicationSystemTestServer* Server)
{
	if (TMap<FReplicationSystemTestServer*, TArray<FReplicationSystemTestClient*>>::ValueType* Clients = ServerClients.Find(Server))
	{
		for (FReplicationSystemTestClient* Client : *Clients)
		{
			delete Client;
		}
	}
	ServerClients.Remove(Server);

	Servers.RemoveSingle(Server);
	delete Server;
}

FReplicationSystemTestClient* FMultiReplicationSystemsTestFixture::CreateClientForServer(FReplicationSystemTestServer* Server)
{
	if (TMap<FReplicationSystemTestServer*, TArray<FReplicationSystemTestClient*>>::ValueType* Clients = ServerClients.Find(Server))
	{
		FReplicationSystemTestClient* Client = new FReplicationSystemTestClient(GetName()); 
		Clients->Add(Client);

		// The client needs a connection
		Client->LocalConnectionId = Client->AddConnection();

		// Auto connect to server
		Client->ConnectionIdOnServer = Server->AddConnection();

		return Client;
	}

	return nullptr;
}

TArrayView<FReplicationSystemTestClient*> FMultiReplicationSystemsTestFixture::GetClients(FReplicationSystemTestServer* Server)
{
	if (TMap<FReplicationSystemTestServer*, TArray<FReplicationSystemTestClient*>>::ValueType* Clients = ServerClients.Find(Server))
	{
		return MakeArrayView(*Clients);
	}

	return TArrayView<FReplicationSystemTestClient*>();
}

UTestReplicatedIrisObject* FMultiReplicationSystemsTestFixture::CreateObject(const UTestReplicatedIrisObject::FComponents& Components)
{
	UTestReplicatedIrisObject* CreatedObject = NewObject<UTestReplicatedIrisObject>();
	CreatedObject->AddComponents(Components);

	CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedObject));
	return CreatedObject;
}

void FMultiReplicationSystemsTestFixture::BeginReplication(UTestReplicatedIrisObject* Object)
{
	for (FReplicationSystemTestServer* Server : GetAllServers())
	{
		if (UReplicatedTestObjectBridge* ReplicationBridge = Server->GetReplicationBridge())
		{
			ReplicationBridge->BeginReplication(Object);
		}
	}
}

void FMultiReplicationSystemsTestFixture::EndReplication(UTestReplicatedIrisObject* Object)
{
	for (FReplicationSystemTestServer* Server : GetAllServers())
	{
		if (UReplicatedTestObjectBridge* ReplicationBridge = Server->GetReplicationBridge())
		{
			constexpr EEndReplicationFlags EndReplicationFlags = EEndReplicationFlags::Destroy | EEndReplicationFlags::DestroyNetHandle | EEndReplicationFlags::ClearNetPushId;
			ReplicationBridge->EndReplication(Object, EndReplicationFlags);
		}
	}
}

FNetRefHandle FMultiReplicationSystemsTestFixture::BeginReplication(FReplicationSystemTestServer* Server, UTestReplicatedIrisObject* Object)
{
	if (UReplicatedTestObjectBridge* ReplicationBridge = Server->GetReplicationBridge())
	{
		return ReplicationBridge->BeginReplication(Object);
	}

	return FNetRefHandle::GetInvalid();
}

FNetRefHandle FMultiReplicationSystemsTestFixture::BeginReplication(FReplicationSystemTestServer* Server, UTestReplicatedIrisObject* Object, const UObjectReplicationBridge::FRootObjectReplicationParams& Params)
{
	if (UReplicatedTestObjectBridge* ReplicationBridge = Server->GetReplicationBridge())
	{
		return ReplicationBridge->BeginReplication(Object, Params);
	}

	return FNetRefHandle::GetInvalid();
}

void FMultiReplicationSystemsTestFixture::EndReplication(FReplicationSystemTestServer* Server, UTestReplicatedIrisObject* Object)
{
	if (UReplicatedTestObjectBridge* ReplicationBridge = Server->GetReplicationBridge())
	{
		ReplicationBridge->EndReplication(Object);
	}
}

void FMultiReplicationSystemsTestFixture::FullSendAndDeliverUpdate()
{
	for (FReplicationSystemTestServer* Server : Servers)
	{
		Server->NetUpdate();
	}

	for (TMap<FReplicationSystemTestServer*, TArray<FReplicationSystemTestClient*>>::ElementType& ServerWithClients : ServerClients)
	{
		FReplicationSystemTestServer* Server = ServerWithClients.Key;
		for (FReplicationSystemTestClient* Client : ServerWithClients.Value)
		{
			Server->SendAndDeliverTo(Client, DeliverPacket);
		}
	}

	for (FReplicationSystemTestServer* Server : Servers)
	{
		Server->PostSendUpdate();
	}
}

void FMultiReplicationSystemsTestFixture::FullSendAndDeliverUpdateTwoPass()
{
	// PreSend & Send
	for (FReplicationSystemTestServer* Server : Servers)
	{
		TArray<FReplicationSystemTestClient*>* FoundClients = ServerClients.Find(Server);
		if (FoundClients && FoundClients->Num() > 0)
		{
			Server->NetUpdate();

			for (FReplicationSystemTestClient* Client : *FoundClients)
			{
				Server->SendTo(Client);
			}
		}
	}

	// PostSend
	for (FReplicationSystemTestServer* Server : Servers)
	{
		TArray<FReplicationSystemTestClient*>* FoundClients = ServerClients.Find(Server);
		if (FoundClients && FoundClients->Num() > 0)
		{
			Server->PostSendUpdate();
		}
	}

	// Deliver
	for (TMap<FReplicationSystemTestServer*, TArray<FReplicationSystemTestClient*>>::ElementType& ServerWithClients : ServerClients)
	{
		FReplicationSystemTestServer* Server = ServerWithClients.Key;
		for (FReplicationSystemTestClient* Client : ServerWithClients.Value)
		{
			Server->DeliverTo(Client, DeliverPacket);
		}
	}
}

}
