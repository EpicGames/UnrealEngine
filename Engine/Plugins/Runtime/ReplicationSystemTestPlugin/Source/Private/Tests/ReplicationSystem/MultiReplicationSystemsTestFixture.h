// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkAutomationTest.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Containers/ContainersFwd.h"

namespace UE::Net
{

class FMultiReplicationSystemsTestFixture : public FNetworkAutomationTestSuiteFixture
{
public:
	FMultiReplicationSystemsTestFixture() : FNetworkAutomationTestSuiteFixture() {}

protected:
	enum EConstants : unsigned
	{
		/** It's common for things to stop working beyond 2. */
		DefaultServerCount = 3U,
	};

	enum : bool
	{
		DoNotDeliverPacket = false,
		DeliverPacket = true,
	};

protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	/** Create a specific amount of servers. */
	void CreateServers(unsigned ServerCount, const FReplicationSystemTestNode::FReplicationSystemParamsOverride* ParamsOverride = nullptr);

	/** Creates the default amount of servers. */
	void CreateSomeServers();

	/** Returns a view of all created servers. */
	TArrayView<FReplicationSystemTestServer*> GetAllServers();

	/** Creates a server and returns it. */
	FReplicationSystemTestServer* CreateServer(const FReplicationSystemTestNode::FReplicationSystemParamsOverride* ParamsOverride = nullptr);

	/** Destroys a server and all of its clients */
	void DestroyServer(FReplicationSystemTestServer* Server);

	/** Creates a client that is connected to a specific server. */
	FReplicationSystemTestClient* CreateClientForServer(FReplicationSystemTestServer*);

	/** Returns a view of the clients connected to the server. */
	TArrayView<FReplicationSystemTestClient*> GetClients(FReplicationSystemTestServer*);

	/** Creates a UTestReplicatedIrisObject. It will not begin replicating unless a BeginReplication method is called. */
	UTestReplicatedIrisObject* CreateObject(const UTestReplicatedIrisObject::FComponents& Components);
	
	/** Begins replication of the object on all replication systems. */
	void BeginReplication(UTestReplicatedIrisObject* Object);

	/** Ends replication of the object on all replication systems. */
	void EndReplication(UTestReplicatedIrisObject* Object);

	/** Begins replication of the object on a specific replication system. */
	FNetRefHandle BeginReplication(FReplicationSystemTestServer* Server, UTestReplicatedIrisObject* Object);

	/** Begins replication of the object on a specific replication system with the given parameters. */
	FNetRefHandle BeginReplication(FReplicationSystemTestServer* Server, UTestReplicatedIrisObject* Object, const UObjectReplicationBridge::FRootObjectReplicationParams& Params);

	/** Ends replication of the object on a specific replication system. */
	void EndReplication(FReplicationSystemTestServer* Server, UTestReplicatedIrisObject* Object);

	/** Performs PreSendUpdate, SendAndDeliverTo and PostSendUpdate on all servers. Packets are delivered to all clients connected to the servers. */
	void FullSendAndDeliverUpdate();

	/**
	 * Performs PreSendUpdate and SendAndDeliverTo on each server one at a time.
	 * Then in a separate loop performs PostSendUpdate on each server.
	 * Then packets are delivered to all clients connected to the servers.
	 * This more closely resembles the order that occurs with UWorld and TickFlush/PostTickFlush.
	 */
	void FullSendAndDeliverUpdateTwoPass();

private:
	FDataStreamTestUtil DataStreamUtil;
	TArray<FReplicationSystemTestServer*> Servers;
	TMap<FReplicationSystemTestServer*, TArray<FReplicationSystemTestClient*>> ServerClients;
	TArray<TStrongObjectPtr<UObject>> CreatedObjects;
};

}
