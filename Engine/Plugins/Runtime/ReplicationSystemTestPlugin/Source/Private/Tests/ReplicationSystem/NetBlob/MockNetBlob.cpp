// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockNetBlob.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlobHandlerManager.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializationContext.h"

#ifndef UE_MAKEFOURCC
#define UE_MAKEFOURCC(ch0, ch1, ch2, ch3) ((uint32)(uint8)(ch0) | ((uint32)(uint8)(ch1) << 8U) | ((uint32)(uint8)(ch2) << 16U) | ((uint32)(uint8)(ch3) << 24U))
#endif

namespace UE::Net
{

FMockNetBlob::FMockNetBlob(const FNetBlobCreationInfo& CreationInfo)
: Super(CreationInfo)
, BlobBitCount(0)
{
}

void FMockNetBlob::SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	Writer->WriteBits(UE_MAKEFOURCC('O', 'B', 'J', ' '), 32U);
	Writer->WriteBits(BlobBitCount, 32U);
	Writer->Seek(Writer->GetPosBits() + BlobBitCount);
	Writer->WriteBits(UE_MAKEFOURCC('C', 'N', 'R', 'Y'), 32U);
}

void FMockNetBlob::DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	
	const uint32 Header = Reader->ReadBits(32U);
	if (Header != UE_MAKEFOURCC('O', 'B', 'J', ' '))
	{
		Context.SetError(GNetError_BitStreamError);
		return;
	}

	BlobBitCount = Reader->ReadBits(32U);
	Reader->Seek(Reader->GetPosBits() + BlobBitCount);

	const uint32 Canary = Reader->ReadBits(32U);
	if (Canary != UE_MAKEFOURCC('C', 'N', 'R', 'Y'))
	{
		Context.SetError(GNetError_BitStreamError);
		return;
	}
}

void FMockNetBlob::Serialize(FNetSerializationContext& Context) const
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(UE_MAKEFOURCC('L', 'O', 'N', 'E'), 32U);
	Writer->WriteBits(BlobBitCount, 32U);
	Writer->Seek(Writer->GetPosBits() + BlobBitCount);
	Writer->WriteBits(UE_MAKEFOURCC('C', 'N', 'R', 'Y'), 32U);
}

void FMockNetBlob::Deserialize(FNetSerializationContext& Context)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	
	const uint32 Header = Reader->ReadBits(32U);
	if (Header != UE_MAKEFOURCC('L', 'O', 'N', 'E'))
	{
		Context.SetError(GNetError_BitStreamError);
		return;
	}

	BlobBitCount = Reader->ReadBits(32U);
	Reader->Seek(Reader->GetPosBits() + BlobBitCount);

	const uint32 Canary = Reader->ReadBits(32U);
	if (Canary != UE_MAKEFOURCC('C', 'N', 'R', 'Y'))
	{
		Context.SetError(GNetError_BitStreamError);
		return;
	}
}

}

// UMockSequentialPartialNetBlobHandlerConfig
UMockSequentialPartialNetBlobHandlerConfig::UMockSequentialPartialNetBlobHandlerConfig()
{
	// Overriding MaxPartCount
	MaxPartCount = 32768U;
}

// UMockNetBlobHandler
UMockNetBlobHandler::UMockNetBlobHandler()
: CallCounts({})
{
}

UMockNetBlobHandler::~UMockNetBlobHandler()
{
}

TRefCountPtr<UE::Net::FNetBlob> UMockNetBlobHandler::CreateReliableNetBlob(uint32 PayloadBitCount) const
{
	using namespace UE::Net;

	FNetBlobCreationInfo CreationInfo;
	CreationInfo.Type = GetNetBlobType();
	CreationInfo.Flags = ENetBlobFlags::Reliable;
	FMockNetBlob* Blob = InternalCreateNetBlob(CreationInfo);
	Blob->SetPayloadBitCount(PayloadBitCount);
	return Blob;
}

TRefCountPtr<UE::Net::FNetBlob> UMockNetBlobHandler::CreateUnreliableNetBlob(uint32 PayloadBitCount) const
{
	using namespace UE::Net;

	FNetBlobCreationInfo CreationInfo;
	CreationInfo.Type = GetNetBlobType();
	CreationInfo.Flags = ENetBlobFlags::None;
	FMockNetBlob* Blob = InternalCreateNetBlob(CreationInfo);
	Blob->SetPayloadBitCount(PayloadBitCount);
	return Blob;
}

TRefCountPtr<UE::Net::FNetBlob> UMockNetBlobHandler::CreateNetBlob(const FNetBlobCreationInfo& CreationInfo) const
{
	++CallCounts.CreateNetBlob;
	return InternalCreateNetBlob(CreationInfo);
}

void UMockNetBlobHandler::OnNetBlobReceived(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& NetBlob)
{
	++CallCounts.OnNetBlobReceived;
}

UE::Net::FMockNetBlob* UMockNetBlobHandler::InternalCreateNetBlob(const FNetBlobCreationInfo& CreationInfo) const
{
	return new FMockNetBlob(CreationInfo);
}


// UMockSequentialPartialNetBlobHandler
UMockSequentialPartialNetBlobHandler::UMockSequentialPartialNetBlobHandler()
: CallCounts({})
{
}

UMockSequentialPartialNetBlobHandler::~UMockSequentialPartialNetBlobHandler()
{
}

void UMockSequentialPartialNetBlobHandler::Init(const FSequentialPartialNetBlobHandlerInitParams& InitParams)
{
	Super::Init(InitParams);
	UE::Net::FNetBlobAssemblerInitParams AssemblerInitParams;
	AssemblerInitParams.PartialNetBlobHandlerConfig = GetConfig();
	Assembler.Init(AssemblerInitParams);
}

TRefCountPtr<UE::Net::FNetBlob> UMockSequentialPartialNetBlobHandler::CreateNetBlob(const FNetBlobCreationInfo& CreationInfo) const
{
	++CallCounts.CreateNetBlob;
	return Super::CreateNetBlob(CreationInfo);
}

void UMockSequentialPartialNetBlobHandler::OnNetBlobReceived(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& NetBlob)
{
	++CallCounts.OnNetBlobReceived;
	Assembler.AddPartialNetBlob(Context, UE::Net::FNetRefHandle(), reinterpret_cast<const TRefCountPtr<UE::Net::FPartialNetBlob>&>(NetBlob));
	if (Assembler.IsReadyToAssemble())
	{
		const TRefCountPtr<FNetBlob>& AssembledBlob = Assembler.Assemble(Context);
		if (AssembledBlob.IsValid())
		{
			if (INetBlobReceiver* BlobReceiver = Context.GetNetBlobReceiver())
			{
				BlobReceiver->OnNetBlobReceived(Context, AssembledBlob);
			}
		}
	}
}
