// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/NetBlob/NetBlob.h"
#include "Iris/ReplicationSystem/NetBlob/SequentialPartialNetBlobHandler.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlobAssembler.h"
#include "MockNetBlob.generated.h"

namespace UE::Net
{

class FMockNetBlob : public FNetBlob
{
	typedef FNetBlob Super;

public:
	FMockNetBlob(const FNetBlobCreationInfo&);

	void SetPayloadBitCount(uint32 BitCount) { BlobBitCount = BitCount; }

private:
	virtual void SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const override;
	virtual void DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) override;

	virtual void Serialize(FNetSerializationContext& Context) const override;
	virtual void Deserialize(FNetSerializationContext& Context) override;

private:
	uint32 BlobBitCount;
};

}

UCLASS()
class UMockNetBlobHandler final : public UNetBlobHandler
{
	GENERATED_BODY()

	using FMockNetBlob = UE::Net::FMockNetBlob;

public:
	struct FCallCounts
	{
		uint32 CreateNetBlob;
		uint32 OnNetBlobReceived;
	};

	UMockNetBlobHandler();
	virtual ~UMockNetBlobHandler();

	TRefCountPtr<FNetBlob> CreateReliableNetBlob(uint32 PayloadBitCount) const;
	TRefCountPtr<FNetBlob> CreateUnreliableNetBlob(uint32 PayloadBitCount) const;

	const FCallCounts& GetFunctionCallCounts() const { return CallCounts; }
	void ResetFunctionCallCounts() { CallCounts = FCallCounts({}); }

private:
	virtual TRefCountPtr<FNetBlob> CreateNetBlob(const FNetBlobCreationInfo&) const override final;
	virtual void OnNetBlobReceived(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& NetBlob) override final;

	FMockNetBlob* InternalCreateNetBlob(const FNetBlobCreationInfo&) const;

	mutable FCallCounts CallCounts;
};

UCLASS()
class UMockSequentialPartialNetBlobHandlerConfig : public USequentialPartialNetBlobHandlerConfig
{
	GENERATED_BODY()

public:
	UMockSequentialPartialNetBlobHandlerConfig();
};

UCLASS()
class UMockSequentialPartialNetBlobHandler final : public USequentialPartialNetBlobHandler
{
	GENERATED_BODY()

public:
	struct FCallCounts
	{
		uint32 CreateNetBlob;
		uint32 OnNetBlobReceived;
	};

	UMockSequentialPartialNetBlobHandler();
	virtual ~UMockSequentialPartialNetBlobHandler();

	void Init(const FSequentialPartialNetBlobHandlerInitParams& InitParams);

	const USequentialPartialNetBlobHandlerConfig* GetConfig() const;

	const FCallCounts& GetFunctionCallCounts() const { return CallCounts; }
	void ResetFunctionCallCounts() { CallCounts = FCallCounts({}); }

private:
	virtual TRefCountPtr<FNetBlob> CreateNetBlob(const FNetBlobCreationInfo&) const override;
	virtual void OnNetBlobReceived(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& NetBlob) override;

	UE::Net::FNetBlobAssembler Assembler;
	mutable FCallCounts CallCounts;
};

inline const USequentialPartialNetBlobHandlerConfig* UMockSequentialPartialNetBlobHandler::GetConfig() const
{
	return CastChecked<const USequentialPartialNetBlobHandlerConfig>(Super::GetConfig(), ECastCheckedType::NullAllowed);
}