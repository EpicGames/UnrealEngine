// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockNetObjectAttachment.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializationContext.h"

#ifndef UE_MAKEFOURCC
#define UE_MAKEFOURCC(ch0, ch1, ch2, ch3) ((uint32)(uint8)(ch0) | ((uint32)(uint8)(ch1) << 8U) | ((uint32)(uint8)(ch2) << 16U) | ((uint32)(uint8)(ch3) << 24U))
#endif

namespace UE::Net
{

FMockNetObjectAttachment::FMockNetObjectAttachment(const FNetBlobCreationInfo& CreationInfo)
: Super(CreationInfo)
, BlobBitCount(0)
{
}

void FMockNetObjectAttachment::SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	SerializeSubObjectReference(Context, RefHandle);

	Writer->WriteBits(UE_MAKEFOURCC('O', 'B', 'J', ' '), 32U);
	Writer->WriteBits(BlobBitCount, 32U);
	Writer->Seek(Writer->GetPosBits() + BlobBitCount);
	Writer->WriteBits(UE_MAKEFOURCC('C', 'N', 'R', 'Y'), 32U);
}

void FMockNetObjectAttachment::DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	
	DeserializeSubObjectReference(Context, RefHandle);

	const uint32 Header = Reader->ReadBits(32U);
	if (Header != UE_MAKEFOURCC('O', 'B', 'J', ' '))
	{
		Context.SetError(GNetError_BitStreamError);
		return;
	}

	BlobBitCount = Reader->ReadBits(32U);
	Reader->Seek(Reader->GetPosBits() + BlobBitCount);

	const uint32 Canary = Reader->ReadBits(32U);
	if (Canary != UE_MAKEFOURCC('C', 'N', 'R', 'Y'))
	{
		Context.SetError(GNetError_BitStreamError);
		return;
	}
}

void FMockNetObjectAttachment::Serialize(FNetSerializationContext& Context) const
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	SerializeObjectReference(Context);

	Writer->WriteBits(UE_MAKEFOURCC('L', 'O', 'N', 'E'), 32U);
	Writer->WriteBits(BlobBitCount, 32U);
	Writer->Seek(Writer->GetPosBits() + BlobBitCount);
	Writer->WriteBits(UE_MAKEFOURCC('C', 'N', 'R', 'Y'), 32U);
}

void FMockNetObjectAttachment::Deserialize(FNetSerializationContext& Context)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	
	DeserializeObjectReference(Context);

	const uint32 Header = Reader->ReadBits(32U);
	if (Header != UE_MAKEFOURCC('L', 'O', 'N', 'E'))
	{
		Context.SetError(GNetError_BitStreamError);
		return;
	}

	BlobBitCount = Reader->ReadBits(32U);
	Reader->Seek(Reader->GetPosBits() + BlobBitCount);

	const uint32 Canary = Reader->ReadBits(32U);
	if (Canary != UE_MAKEFOURCC('C', 'N', 'R', 'Y'))
	{
		Context.SetError(GNetError_BitStreamError);
		return;
	}
}

}

// UMockNetBlobHandler
UMockNetObjectAttachmentHandler::UMockNetObjectAttachmentHandler()
: CallCounts({})
{
}

UMockNetObjectAttachmentHandler::~UMockNetObjectAttachmentHandler()
{
}

TRefCountPtr<UE::Net::FNetObjectAttachment> UMockNetObjectAttachmentHandler::CreateReliableNetObjectAttachment(uint32 PayloadBitCount) const
{
	using namespace UE::Net;

	FNetBlobCreationInfo CreationInfo;
	CreationInfo.Type = GetNetBlobType();
	CreationInfo.Flags = ENetBlobFlags::Reliable;
	FMockNetObjectAttachment* Blob = InternalCreateNetBlob(CreationInfo);
	Blob->SetPayloadBitCount(PayloadBitCount);
	return Blob;
}

TRefCountPtr<UE::Net::FNetObjectAttachment> UMockNetObjectAttachmentHandler::CreateUnreliableNetObjectAttachment(uint32 PayloadBitCount) const
{
	using namespace UE::Net;

	FNetBlobCreationInfo CreationInfo;
	CreationInfo.Type = GetNetBlobType();
	CreationInfo.Flags = ENetBlobFlags::None;
	FMockNetObjectAttachment* Blob = InternalCreateNetBlob(CreationInfo);
	Blob->SetPayloadBitCount(PayloadBitCount);
	return Blob;
}

TRefCountPtr<UE::Net::FNetBlob> UMockNetObjectAttachmentHandler::CreateNetBlob(const FNetBlobCreationInfo& CreationInfo) const
{
	++CallCounts.CreateNetBlob;
	return InternalCreateNetBlob(CreationInfo);
}

void UMockNetObjectAttachmentHandler::OnNetBlobReceived(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& NetBlob)
{
	++CallCounts.OnNetBlobReceived;
}

UE::Net::FMockNetObjectAttachment* UMockNetObjectAttachmentHandler::InternalCreateNetBlob(const FNetBlobCreationInfo& CreationInfo) const
{
	return new FMockNetObjectAttachment(CreationInfo);
}
