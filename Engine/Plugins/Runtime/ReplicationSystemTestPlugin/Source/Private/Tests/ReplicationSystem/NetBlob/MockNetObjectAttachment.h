// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/NetBlob/NetBlob.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlobHandler.h"
#include "MockNetObjectAttachment.generated.h"

namespace UE::Net
{

class FMockNetObjectAttachment : public FNetObjectAttachment
{
	typedef FNetObjectAttachment Super;

public:
	FMockNetObjectAttachment(const FNetBlobCreationInfo&);

	void SetPayloadBitCount(uint32 BitCount) { BlobBitCount = BitCount; }

private:
	virtual void SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const override;
	virtual void DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) override;

	virtual void Serialize(FNetSerializationContext& Context) const override;
	virtual void Deserialize(FNetSerializationContext& Context) override;

private:
	uint32 BlobBitCount;
};

}

UCLASS()
class UMockNetObjectAttachmentHandler final : public UNetBlobHandler
{
	GENERATED_BODY()

	using FMockNetObjectAttachment = UE::Net::FMockNetObjectAttachment;

public:
	struct FCallCounts
	{
		uint32 CreateNetBlob;
		uint32 OnNetBlobReceived;
	};

	UMockNetObjectAttachmentHandler();
	virtual ~UMockNetObjectAttachmentHandler();

	TRefCountPtr<UE::Net::FNetObjectAttachment> CreateReliableNetObjectAttachment(uint32 PayloadBitCount) const;
	TRefCountPtr<UE::Net::FNetObjectAttachment> CreateUnreliableNetObjectAttachment(uint32 PayloadBitCount) const;

	const FCallCounts& GetFunctionCallCounts() const { return CallCounts; }
	void ResetFunctionCallCounts() { CallCounts = FCallCounts({}); }

private:
	virtual TRefCountPtr<FNetBlob> CreateNetBlob(const FNetBlobCreationInfo&) const override final;
	virtual void OnNetBlobReceived(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& NetBlob) override final;

	FMockNetObjectAttachment* InternalCreateNetBlob(const FNetBlobCreationInfo&) const;

	mutable FCallCounts CallCounts;
};
