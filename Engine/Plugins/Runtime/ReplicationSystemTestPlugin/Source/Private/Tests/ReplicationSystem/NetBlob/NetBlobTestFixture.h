// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MockNetBlob.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlobHandlerDefinitions.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"

namespace UE::Net
{

class FNetBlobTestFixture : public FReplicationSystemServerClientTestFixture
{
	typedef FReplicationSystemServerClientTestFixture Super;

public:
	FNetBlobTestFixture()
	{
	}

protected:
	virtual void SetUp() override
	{
		StoreNetBlobHandlerDefinitions();
		SetupNetBlobHandlerDefinitions();
		Super::SetUp();

	}

	virtual void TearDown() override
	{
		Super::TearDown();
		DestroyNetBlobHandlers();
		RestoreNetBlobHandlerDefinitions();
	}

	struct FTestContext
	{
		FNetSerializationContext SerializationContext;
		Private::FInternalNetSerializationContext InternalSerializationContext;
	};

	void SetupTestContext(FTestContext& TestContext, FReplicationSystemTestNode* TestNode)
	{
		const bool bIsServer = TestNode->GetReplicationSystem()->IsServer();
		Private::FNetBlobHandlerManager* NetBlobHandlerManager = &TestNode->GetReplicationSystem()->GetReplicationSystemInternal()->GetNetBlobHandlerManager();

		TestContext.InternalSerializationContext = Private::FInternalNetSerializationContext(TestNode->GetReplicationSystem());
		TestContext.SerializationContext.SetLocalConnectionId(bIsServer ? 1U : static_cast<FReplicationSystemTestClient*>(TestNode)->LocalConnectionId);
		TestContext.SerializationContext.SetInternalContext(&TestContext.InternalSerializationContext);
		TestContext.SerializationContext.SetNetBlobReceiver(NetBlobHandlerManager);
	}

	// Calls must be prior to this SetUp() is called, such as in a derived class' SetUp().
	void AddMockNetBlobHandlerDefinition()
	{
 		HandlerDefinitions.Add({TEXT("MockNetBlobHandler"), });
	}

	// Calls must be prior to this SetUp() is called, such as in a derived class' SetUp().
	void AddNetBlobHandlerDefinitions(const FNetBlobHandlerDefinition* InHandlerDefinitions, uint32 HandlerDefinitionCount)
	{
		HandlerDefinitions.Append(InHandlerDefinitions, HandlerDefinitionCount);
	}

	bool CreateAndRegisterMockNetBlobHandler(UReplicationSystem* RepSys)
	{
		UMockNetBlobHandler* BlobHandler = NewObject<UMockNetBlobHandler>();
		if (RepSys->IsServer())
		{
			MockNetBlobHandler = TStrongObjectPtr<UMockNetBlobHandler>(BlobHandler);
		}
		return RegisterNetBlobHandler(RepSys, BlobHandler);
	}

	bool RegisterNetBlobHandler(UReplicationSystem* RepSys, UNetBlobHandler* Handler)
	{
		NetBlobHandlers.Add(TStrongObjectPtr<UNetBlobHandler>(Handler));
		Private::FNetBlobHandlerManager* NetBlobHandlerManager = &RepSys->GetReplicationSystemInternal()->GetNetBlobHandlerManager();
		return NetBlobHandlerManager->RegisterHandler(Handler);
	}

	// Blob is created on server.
	TRefCountPtr<FNetBlob> CreateReliableMockNetBlob(uint32 PayloadBitCount)
	{
		if (MockNetBlobHandler)
		{
			return MockNetBlobHandler->CreateReliableNetBlob(PayloadBitCount);
		}

		return nullptr;
	}

	// Blob is created on server.
	TRefCountPtr<FNetBlob> CreateUnreliableMockNetBlob(uint32 PayloadBitCount)
	{
		if (MockNetBlobHandler)
		{
			return MockNetBlobHandler->CreateUnreliableNetBlob(PayloadBitCount);
		}
		
		return nullptr;
	}

private:
	void StoreNetBlobHandlerDefinitions()
	{
		UNetBlobHandlerDefinitions* BlobHandlerDefinitions = GetMutableDefault<UNetBlobHandlerDefinitions>();
		OriginalHandlerDefinitions = BlobHandlerDefinitions->ReadWriteHandlerDefinitions();
	}

	void SetupNetBlobHandlerDefinitions()
	{
		UNetBlobHandlerDefinitions* BlobHandlerDefinitions = GetMutableDefault<UNetBlobHandlerDefinitions>();
		BlobHandlerDefinitions->ReadWriteHandlerDefinitions() = HandlerDefinitions;
	}

	void RestoreNetBlobHandlerDefinitions()
	{
		UNetBlobHandlerDefinitions* BlobHandlerDefinitions = GetMutableDefault<UNetBlobHandlerDefinitions>();
		BlobHandlerDefinitions->ReadWriteHandlerDefinitions() = OriginalHandlerDefinitions;
	}

	void DestroyNetBlobHandlers()
	{
		MockNetBlobHandler.Reset();
		NetBlobHandlers.Empty();
	}

// Members
protected:
	TStrongObjectPtr<UMockNetBlobHandler> MockNetBlobHandler;
	TArray<TStrongObjectPtr<UNetBlobHandler>> NetBlobHandlers;

private:
	TArray<FNetBlobHandlerDefinition> OriginalHandlerDefinitions;
	TArray<FNetBlobHandlerDefinition> HandlerDefinitions;
};

}
