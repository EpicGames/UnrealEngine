// Copyright Epic Games, Inc. All Rights Reserved.

#include "PartialNetBlobTestFixture.h"
#include "MockNetObjectAttachment.h"

namespace UE::Net
{

void FPartialNetBlobTestFixture::SetUp()
{
	AddNetBlobHandlerDefinitions();
	Super::SetUp();
	RegisterNetBlobHandlers(Server);
}

void FPartialNetBlobTestFixture::TearDown()
{
	Super::TearDown();
}

void FPartialNetBlobTestFixture::RegisterNetBlobHandlers(FReplicationSystemTestNode* Node)
{
	UReplicationSystem* RepSys = Node->GetReplicationSystem();
	const bool bIsServer = RepSys->IsServer();

	Private::FNetBlobHandlerManager* NetBlobHandlerManager = &RepSys->GetReplicationSystemInternal()->GetNetBlobHandlerManager();

	CreateAndRegisterMockNetBlobHandler(RepSys);

	{
		FSequentialPartialNetBlobHandlerInitParams InitParams = {};
		InitParams.ReplicationSystem = RepSys;
		InitParams.Config = GetDefault<UMockSequentialPartialNetBlobHandlerConfig>();
		UMockSequentialPartialNetBlobHandler* BlobHandler = NewObject<UMockSequentialPartialNetBlobHandler>();
		BlobHandler->Init(InitParams);
		const bool bPartialNetBlobHandlerWasRegistered = RegisterNetBlobHandler(RepSys, BlobHandler);
		check(bPartialNetBlobHandlerWasRegistered);

		if (bIsServer)
		{
			MockSequentialPartialNetBlobHandler = TStrongObjectPtr<UMockSequentialPartialNetBlobHandler>(BlobHandler);
		}
	}

	{
		UMockNetObjectAttachmentHandler* BlobHandler = NewObject<UMockNetObjectAttachmentHandler>();
		const bool bMockNetObjectAttachmentHandlerWasRegistered = RegisterNetBlobHandler(RepSys, BlobHandler);
		check(bMockNetObjectAttachmentHandlerWasRegistered);

		if (bIsServer)
		{
			MockNetObjectAttachmentHandler = TStrongObjectPtr<UMockNetObjectAttachmentHandler>(BlobHandler);
		}
		else
		{
			ClientMockNetObjectAttachmentHandler = TStrongObjectPtr<UMockNetObjectAttachmentHandler>(BlobHandler);
		}
	}
}

void FPartialNetBlobTestFixture::AddNetBlobHandlerDefinitions()
{
	AddMockNetBlobHandlerDefinition();
	const FNetBlobHandlerDefinition PartialNetBlobHandlerDefinitions[] = 
	{
		{TEXT("MockSequentialPartialNetBlobHandler"),},
		// Used for tests involving server and clients.
		{TEXT("MockNetObjectAttachmentHandler"),}, 
		{TEXT("PartialNetObjectAttachmentHandler"),}, 
		{TEXT("NetObjectBlobHandler"),}, 
	};
	Super::AddNetBlobHandlerDefinitions(PartialNetBlobHandlerDefinitions, UE_ARRAY_COUNT(PartialNetBlobHandlerDefinitions));
}

}
