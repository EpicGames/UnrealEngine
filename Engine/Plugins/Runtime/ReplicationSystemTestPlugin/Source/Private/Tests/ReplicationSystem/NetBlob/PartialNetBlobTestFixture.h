// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetBlobTestFixture.h"
#include "MockNetObjectAttachment.h"

namespace UE::Net
{

class FPartialNetBlobTestFixture : public FNetBlobTestFixture
{
	typedef FNetBlobTestFixture Super;

protected:
	virtual void SetUp() override;

	virtual void TearDown() override;

	void RegisterNetBlobHandlers(FReplicationSystemTestNode* Node);

private:
	void AddNetBlobHandlerDefinitions();

protected:
	TStrongObjectPtr<UMockSequentialPartialNetBlobHandler> MockSequentialPartialNetBlobHandler;
	TStrongObjectPtr<UMockNetObjectAttachmentHandler> MockNetObjectAttachmentHandler;
	TStrongObjectPtr<UMockNetObjectAttachmentHandler> ClientMockNetObjectAttachmentHandler;
};

}
