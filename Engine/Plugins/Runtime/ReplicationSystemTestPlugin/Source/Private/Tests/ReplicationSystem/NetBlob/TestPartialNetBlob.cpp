// Copyright Epic Games, Inc. All Rights Reserved.

#include "PartialNetBlobTestFixture.h"
#include "MockNetBlob.h"

namespace UE::Net::Private
{

UE_NET_TEST_FIXTURE(FPartialNetBlobTestFixture, CanSplitNetBlobNotInNeedOfSplitting)
{
	FTestContext ServerContext;
	SetupTestContext(ServerContext, Server);

	constexpr uint32 PayloadBitCount = 0U;
	const TRefCountPtr<FNetBlob>& Blob = CreateUnreliableMockNetBlob(PayloadBitCount);

	TArray<TRefCountPtr<FNetBlob>> PartialNetBlobs;
	MockSequentialPartialNetBlobHandler->SplitNetBlob(ServerContext.SerializationContext, Blob, PartialNetBlobs);
	UE_NET_ASSERT_EQ(PartialNetBlobs.Num(), 1);

	for (const TRefCountPtr<FNetBlob>& PartialNetBlob : PartialNetBlobs)
	{
		ServerContext.SerializationContext.GetNetBlobReceiver()->OnNetBlobReceived(ServerContext.SerializationContext, PartialNetBlob);
	}
	UE_NET_ASSERT_FALSE(ServerContext.SerializationContext.HasError());

	constexpr uint32 ExpectedPartialNetBlobReceiveCount = 1U;
	constexpr uint32 ExpectedMockNetBlobReceiveCount = 1U;
	UE_NET_ASSERT_EQ(MockSequentialPartialNetBlobHandler->GetFunctionCallCounts().OnNetBlobReceived, ExpectedPartialNetBlobReceiveCount);
	UE_NET_ASSERT_EQ(MockNetBlobHandler->GetFunctionCallCounts().OnNetBlobReceived, ExpectedMockNetBlobReceiveCount);
}

UE_NET_TEST_FIXTURE(FPartialNetBlobTestFixture, CanSplitNetBlobInNeedOfSplitting)
{
	FTestContext ServerContext;
	SetupTestContext(ServerContext, Server);

	constexpr uint32 PayloadBitCount = 262193U;
	const TRefCountPtr<FNetBlob>& Blob = CreateUnreliableMockNetBlob(PayloadBitCount);

	TArray<TRefCountPtr<FNetBlob>> PartialNetBlobs;
	MockSequentialPartialNetBlobHandler->SplitNetBlob(ServerContext.SerializationContext, Blob, PartialNetBlobs);
	UE_NET_ASSERT_GE(PartialNetBlobs.Num(), 2);

	for (const TRefCountPtr<FNetBlob>& PartialNetBlob : PartialNetBlobs)
	{
		ServerContext.SerializationContext.GetNetBlobReceiver()->OnNetBlobReceived(ServerContext.SerializationContext, PartialNetBlob);
	}

	const uint32 ExpectedPartialNetBlobReceiveCount = PartialNetBlobs.Num();
	constexpr uint32 ExpectedMockNetBlobReceiveCount = 1U;
	UE_NET_ASSERT_EQ(MockSequentialPartialNetBlobHandler->GetFunctionCallCounts().OnNetBlobReceived, ExpectedPartialNetBlobReceiveCount);
	UE_NET_ASSERT_EQ(MockNetBlobHandler->GetFunctionCallCounts().OnNetBlobReceived, ExpectedMockNetBlobReceiveCount);
}

UE_NET_TEST_FIXTURE(FPartialNetBlobTestFixture, CanSplitHugeNetBlob)
{
	FTestContext ServerContext;
	SetupTestContext(ServerContext, Server);

	constexpr uint32 PayloadBitCount = 384U*1024U*8U; // 384KiB
	const TRefCountPtr<FNetBlob>& Blob = CreateUnreliableMockNetBlob(PayloadBitCount);

	TArray<TRefCountPtr<FNetBlob>> PartialNetBlobs;
	MockSequentialPartialNetBlobHandler->SplitNetBlob(ServerContext.SerializationContext, Blob, PartialNetBlobs);
	UE_NET_ASSERT_GE(PartialNetBlobs.Num(), 2);

	for (const TRefCountPtr<FNetBlob>& PartialNetBlob : PartialNetBlobs)
	{
		ServerContext.SerializationContext.GetNetBlobReceiver()->OnNetBlobReceived(ServerContext.SerializationContext, PartialNetBlob);
	}

	const uint32 ExpectedPartialNetBlobReceiveCount = PartialNetBlobs.Num();
	constexpr uint32 ExpectedMockNetBlobReceiveCount = 1U;
	UE_NET_ASSERT_EQ(MockSequentialPartialNetBlobHandler->GetFunctionCallCounts().OnNetBlobReceived, ExpectedPartialNetBlobReceiveCount);
	UE_NET_ASSERT_EQ(MockNetBlobHandler->GetFunctionCallCounts().OnNetBlobReceived, ExpectedMockNetBlobReceiveCount);
}

UE_NET_TEST_FIXTURE(FPartialNetBlobTestFixture, MissingFirstPartialNetBlobCausesError)
{
	FTestContext ServerContext;
	SetupTestContext(ServerContext, Server);

	constexpr uint32 PayloadBitCount = 44444U;
	const TRefCountPtr<FNetBlob>& Blob = CreateReliableMockNetBlob(PayloadBitCount);

	TArray<TRefCountPtr<FNetBlob>> PartialNetBlobs;
	MockSequentialPartialNetBlobHandler->SplitNetBlob(ServerContext.SerializationContext, Blob, PartialNetBlobs);
	UE_NET_ASSERT_GE(PartialNetBlobs.Num(), 2);

	ServerContext.SerializationContext.GetNetBlobReceiver()->OnNetBlobReceived(ServerContext.SerializationContext, PartialNetBlobs[1]);
	UE_NET_ASSERT_TRUE(ServerContext.SerializationContext.HasError());
}

UE_NET_TEST_FIXTURE(FPartialNetBlobTestFixture, MissingArbitraryPartialNetBlobCausesError)
{
	FTestContext ServerContext;
	SetupTestContext(ServerContext, Server);

	constexpr uint32 PayloadBitCount = 44444U;
	const TRefCountPtr<FNetBlob>& Blob = CreateReliableMockNetBlob(PayloadBitCount);

	TArray<TRefCountPtr<FNetBlob>> PartialNetBlobs;
	MockSequentialPartialNetBlobHandler->SplitNetBlob(ServerContext.SerializationContext, Blob, PartialNetBlobs);
	UE_NET_ASSERT_GE(PartialNetBlobs.Num(), 3);

	ServerContext.SerializationContext.GetNetBlobReceiver()->OnNetBlobReceived(ServerContext.SerializationContext, PartialNetBlobs[0]);
	ServerContext.SerializationContext.GetNetBlobReceiver()->OnNetBlobReceived(ServerContext.SerializationContext, PartialNetBlobs[2]);
	UE_NET_ASSERT_TRUE(ServerContext.SerializationContext.HasError());
}

UE_NET_TEST_FIXTURE(FPartialNetBlobTestFixture, TestSplitBlobIsReceivedDespitePacketLoss)
{
	// This test will create a server and a client and then create a huge NetBlob to send.
	// The NetBlob is expected not to be able to be delivered in a single packet so packet loss
	// will be induced at the second packet.

	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 16000;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Deliver a packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 0U);

	// Lose a few packets
	for (auto It : {0, 1, 2, 3})
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DoNotDeliverPacket);
		Server->PostSendUpdate();
	}

	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 0U);

	// Deliver a packet. At this point we expect the entire original blob to have been received properly.
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);

	Server->DestroyObject(ServerObject);
}

#if 0 // Temporary disabled because it will fail for the wrong reason.
UE_NET_TEST_FIXTURE(FPartialNetBlobTestFixture, PartOfUnexpectedSizeCausesError)
{
	FTestContext ServerContext;
	SetupTestContext(ServerContext, Server);

	TArray<TRefCountPtr<FNetBlob>> PartialNetBlobs;
	TRefCountPtr<FNetBlob> WrongSizeSecondBlob;
	{
		const USequentialPartialNetBlobHandlerConfig* Config = MockSequentialPartialNetBlobHandler->GetConfig();

		// Guesstimate on blob overhead. The important thing is we get exactly as many parts as we wish.
		constexpr uint32 BlobOverheadBitCount = 128U;

		constexpr int32 PartCount = 3U;
		uint32 TotalPayloadBitCountForOriginalSplit = PartCount*Config->GetMaxPartBitCount() - BlobOverheadBitCount - 1U;
		const TRefCountPtr<FNetBlob>& Blob = CreateUnreliableMockNetBlob(TotalPayloadBitCountForOriginalSplit);
		MockSequentialPartialNetBlobHandler->SplitNetBlob(Blob, PartialNetBlobs);
		UE_NET_ASSERT_EQ(PartialNetBlobs.Num(), PartCount);

		// Create a second blob with two parts, where the second part will be sent to cause an error.
		constexpr int32 OtherPartCount = 2U;
		TArray<TRefCountPtr<FNetBlob>> OtherPartialNetBlobs;
		const uint32 TotalPayloadBitCountForWrongSizeBlob = OtherPartCount*Config->GetMaxPartBitCount() - BlobOverheadBitCount - 1U;
		const TRefCountPtr<FNetBlob>& OtherBlob = CreateUnreliableMockNetBlob(TotalPayloadBitCountForWrongSizeBlob);
		MockSequentialPartialNetBlobHandler->SplitNetBlob(OtherBlob, OtherPartialNetBlobs);
		UE_NET_ASSERT_EQ(OtherPartialNetBlobs.Num(), OtherPartCount);

		WrongSizeSecondBlob = MoveTemp(OtherPartialNetBlobs[1]);
	}


	ServerContext.SerializationContext.GetNetBlobReceiver()->OnNetBlobReceived(ServerContext.SerializationContext, PartialNetBlobs[0]);
	ServerContext.SerializationContext.GetNetBlobReceiver()->OnNetBlobReceived(ServerContext.SerializationContext, WrongSizeSecondBlob);
	UE_NET_ASSERT_TRUE(ServerContext.SerializationContext.HasError());
}
#endif

UE_NET_TEST_FIXTURE(FPartialNetBlobTestFixture, UnexpectedPartSequenceCausesError)
{
	FTestContext ServerContext;
	SetupTestContext(ServerContext, Server);

	TArray<TRefCountPtr<FNetBlob>> PartialNetBlobs;
	TRefCountPtr<FNetBlob> WrongSecondBlob;
	{
		const USequentialPartialNetBlobHandlerConfig* Config = MockSequentialPartialNetBlobHandler->GetConfig();

		// Guesstimate on blob overhead. The important thing is we get exactly as many parts as we wish.
		constexpr uint32 BlobOverheadBitCount = 128U;

		constexpr int32 PartCount = 3U;
		uint32 TotalPayloadBitCount = PartCount * Config->GetMaxPartBitCount() - BlobOverheadBitCount - 1U;
		const TRefCountPtr<FNetBlob>& Blob = CreateReliableMockNetBlob(TotalPayloadBitCount);
		MockSequentialPartialNetBlobHandler->SplitNetBlob(ServerContext.SerializationContext, Blob, PartialNetBlobs);
		UE_NET_ASSERT_EQ(PartialNetBlobs.Num(), PartCount);

		TArray<TRefCountPtr<FNetBlob>> OtherPartialNetBlobs;
		const TRefCountPtr<FNetBlob>& OtherBlob = CreateReliableMockNetBlob(TotalPayloadBitCount);
		MockSequentialPartialNetBlobHandler->SplitNetBlob(ServerContext.SerializationContext, OtherBlob, OtherPartialNetBlobs);
	
		WrongSecondBlob = MoveTemp(OtherPartialNetBlobs[1]);
	}


	ServerContext.SerializationContext.GetNetBlobReceiver()->OnNetBlobReceived(ServerContext.SerializationContext, PartialNetBlobs[0]);
	ServerContext.SerializationContext.GetNetBlobReceiver()->OnNetBlobReceived(ServerContext.SerializationContext, WrongSecondBlob);
	UE_NET_ASSERT_TRUE(ServerContext.SerializationContext.HasError());
}

}
