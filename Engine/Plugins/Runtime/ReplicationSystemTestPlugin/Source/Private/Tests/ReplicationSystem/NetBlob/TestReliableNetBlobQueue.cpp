// Copyright Epic Games, Inc. All Rights Reserved.

#include "PartialNetBlobTestFixture.h"
#include "MockNetBlob.h"
#include "Iris/ReplicationSystem/NetBlob/ReliableNetBlobQueue.h"
#include "Iris/ReplicationSystem/NetBlob/PartialNetObjectAttachmentHandler.h"
#include "AutoRTFM.h"

namespace UE::Net::Private
{

class FTestReliableNetBlobQueue : public FPartialNetBlobTestFixture
{
protected:
	TRefCountPtr<FNetObjectAttachment> CreateReliableNetObjectAttachmentWithPartCount(uint32 PartCount);
};

UE_NET_TEST_FIXTURE(FTestReliableNetBlobQueue, CanSendMoreReliableBlobsThanReliableWindowAllows)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Create huge attachment
	constexpr uint32 BlobPartCount = FReliableNetBlobQueue::MaxUnackedBlobCount + 3U;
	{
		const TRefCountPtr<FNetObjectAttachment>& Attachment = CreateReliableNetObjectAttachmentWithPartCount(BlobPartCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Deliver the entirety of the attachment.
	for (SIZE_T TryIt = 0, TryEndIt = BlobPartCount; TryIt != TryEndIt; ++TryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();

		if (ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived > 0)
		{
			break;
		}
	}

	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);
}

UE_NET_TEST_FIXTURE(FTestReliableNetBlobQueue, CanSendMoreReliableBlobsThanReliableWindowAllowsDuringPacketLossCondition)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Perform the test with different packets being lost.
	for (uint32 PacketNumberToDrop : {0U, 1U})
	{
		// Start with a clean slate for each test.
		ClientMockNetObjectAttachmentHandler->ResetFunctionCallCounts();

		// Create huge attachment
		constexpr uint32 BlobPartCount = FReliableNetBlobQueue::MaxUnackedBlobCount + 3U;
		{
			const TRefCountPtr<FNetObjectAttachment>& Attachment = CreateReliableNetObjectAttachmentWithPartCount(BlobPartCount);
			FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
			Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
		}

		// Deliver as much as possible. We stop when no more packets are written.
		const auto& ConnectionInfo = Server->GetConnectionInfo(Client->ConnectionIdOnServer);
		SIZE_T PacketCount = ConnectionInfo.WrittenPackets.Count();
		for (SIZE_T TryIt = 0, TryEndIt = BlobPartCount; TryIt != TryEndIt; ++TryIt)
		{
			Server->NetUpdate();
			Server->SendTo(Client);
			Server->PostSendUpdate();

			const SIZE_T NewPacketCount = ConnectionInfo.WrittenPackets.Count();
			if (NewPacketCount == PacketCount)
			{
				break;
			}

			PacketCount = NewPacketCount;
		}

		// Now do the receive logic on the client. Pretend the first packet was lost but deliver the rest of them.
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, (PacketIt == PacketNumberToDrop ? DoNotDeliverPacket : DeliverPacket));
		}

		// Send the rest of the blobs.
		for (SIZE_T TryIt = 0, TryEndIt = BlobPartCount; TryIt != TryEndIt; ++TryIt)
		{
			Server->NetUpdate();
			Server->SendAndDeliverTo(Client, DeliverPacket);
			Server->PostSendUpdate();

			if (ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived > 0)
			{
				break;
			}
		}

		UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);
	}
}

TRefCountPtr<FNetObjectAttachment> FTestReliableNetBlobQueue::CreateReliableNetObjectAttachmentWithPartCount(uint32 PartCount)
{
	if (PartCount == 0)
	{
		return TRefCountPtr<FNetObjectAttachment>();
	}

	if (PartCount == 1)
	{
		return MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(1);
	}

	// Figure out how many bits can fit in a part.
	{
		const UPartialNetObjectAttachmentHandlerConfig* PartialNetObjectAttachmentHandlerConfig = GetDefault<UPartialNetObjectAttachmentHandlerConfig>();
		
		check(PartCount <= PartialNetObjectAttachmentHandlerConfig->GetMaxPartCount());

		const uint32 BitsPerPart = PartialNetObjectAttachmentHandlerConfig->GetMaxPartBitCount();
		const uint32 TotalBitCount = (PartCount - 1U)*BitsPerPart + 1U;
		return MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(TotalBitCount);
	}
}

#if UE_AUTORTFM
UE_NET_TEST_FIXTURE(FTestReliableNetBlobQueue, RefCountingNetBlobsInsideAutoRTFMTransactions)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Perform the test with Committing or Aborting the AutoRTFM transaction
	for (bool bCommitTransaction : {false, true})
	{
		// Start with a clean slate for each test.
		ClientMockNetObjectAttachmentHandler->ResetFunctionCallCounts();

		// Create attachment with a single part count
		constexpr uint32 BlobPartCount = 1;
		{
			const TRefCountPtr<FNetObjectAttachment>& Attachment = CreateReliableNetObjectAttachmentWithPartCount(BlobPartCount);
			FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
			Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);

			// Now take take a reference to this attachment inside an AutoRTFM scope. Whether it succeeds or not, we should
			// end up with the same RefCount afterwards
			int32 InitialAttachmentRefCount = const_cast<TRefCountPtr<FNetObjectAttachment>&>(Attachment).GetRefCount();
			AutoRTFM::Transact([&]
			{
				TRefCountPtr<FNetObjectAttachment> NewRef(Attachment);
			
				UE_NET_ASSERT_GT(NewRef.GetRefCount(), 0U);
			
				if(!bCommitTransaction)
				{
					AutoRTFM::AbortTransaction();
				}
			});
			int32 FinalAttachmentRefCount = const_cast<TRefCountPtr<FNetObjectAttachment>&>(Attachment).GetRefCount();

			UE_NET_ASSERT_EQ(InitialAttachmentRefCount, FinalAttachmentRefCount);
		}

		// Deliver as much as possible. We stop when no more packets are written.
		const auto& ConnectionInfo = Server->GetConnectionInfo(Client->ConnectionIdOnServer);
		SIZE_T PacketCount = ConnectionInfo.WrittenPackets.Count();
		for (SIZE_T TryIt = 0, TryEndIt = BlobPartCount; TryIt != TryEndIt; ++TryIt)
		{
			Server->UpdateAndSend({ Client });

			const SIZE_T NewPacketCount = ConnectionInfo.WrittenPackets.Count();
			if (NewPacketCount == PacketCount)
			{
				break;
			}

			PacketCount = NewPacketCount;
		}

		// Now do the receive logic on the client. 
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, DeliverPacket);
		}

		// Make sure the blob was received on the client and there are no attachments waiting to be sent
		UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);
		UE_NET_ASSERT_EQ(Server->GetReplicationSystem()->GetReplicationSystemInternal()->GetNetBlobManager().HasAnyUnprocessedReliableAttachments(), false);
	}
}
#endif // UE_AUTORTFM

}
