// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockNetObjectPrioritizer.h"
#include <limits>

UMockNetObjectPrioritizer::UMockNetObjectPrioritizer()
: CallStatus({})
, AddedCount(0U)
{
}

void UMockNetObjectPrioritizer::Init(FNetObjectPrioritizerInitParams& Params)
{
	++CallStatus.CallCounts.Init;

	CallStatus.SuccessfulCallCounts.Init += Cast<UMockNetObjectPrioritizerConfig>(Params.Config) != nullptr;

	AddedIndices.Init(Params.CurrentMaxInternalIndex);
}

void UMockNetObjectPrioritizer::OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex)
{
	AddedIndices.SetNumBits(NewMaxInternalIndex);
}

bool UMockNetObjectPrioritizer::AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectPrioritizerAddObjectParams& Params)
{
	++CallStatus.CallCounts.AddObject;
	if (CallSetup.AddObject.ReturnValue == true)
	{
		bool bIsProperCall = true;

		// Validate the passed info is zeroed
		bIsProperCall &= Params.OutInfo.Data[0] == 0;
		bIsProperCall &= Params.OutInfo.Data[1] == 0;
		bIsProperCall &= Params.OutInfo.Data[2] == 0;
		bIsProperCall &= Params.OutInfo.Data[3] == 0;
		
		CallStatus.SuccessfulCallCounts.AddObject += bIsProperCall;

		// Store object index in info so we can check it's passed to RemoveObject
		Params.OutInfo.Data[0] = uint16(ObjectIndex);
		Params.OutInfo.Data[1] = uint16(ObjectIndex);
		Params.OutInfo.Data[2] = uint16(ObjectIndex);
		Params.OutInfo.Data[3] = uint16(ObjectIndex);

		AddedIndices.SetBit(ObjectIndex);
		++AddedCount;

		// Check if object has the NetTest_Priority RepTag which the priority should be read from.
		UE::Net::FRepTagFindInfo RepTagInfo;
		if (UE::Net::FindRepTag(Params.Protocol, RepTag_NetTest_Priority, RepTagInfo))
		{
			// Warning: Using the internal state requires the proper NetSerializer and dequantization to get the value.
			ObjectToPriorityOffset.Add(ObjectIndex, RepTagInfo.InternalStateAbsoluteOffset);
			ObjectToPriority.Add(ObjectIndex, std::numeric_limits<float>::quiet_NaN());
		}
	}

	return CallSetup.AddObject.ReturnValue;
}

void UMockNetObjectPrioritizer::RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectPrioritizationInfo& Info)
{
	++CallStatus.CallCounts.RemoveObject;

	// If this object wasn't added to us it shouldn't be removed either.
	if (AddedIndices.IsBitSet(ObjectIndex))
	{
		bool bIsProperCall = true;

		// Validate the passed info is as we set it
		bIsProperCall &= Info.Data[0] == uint16(ObjectIndex);
		bIsProperCall &= Info.Data[1] == uint16(ObjectIndex);
		bIsProperCall &= Info.Data[2] == uint16(ObjectIndex);
		bIsProperCall &= Info.Data[3] == uint16(ObjectIndex);

		CallStatus.SuccessfulCallCounts.RemoveObject += bIsProperCall;

		AddedIndices.ClearBit(ObjectIndex);
		--AddedCount;

		ObjectToPriority.Remove(ObjectIndex);
		ObjectToPriorityOffset.Remove(ObjectIndex);
	}
}

void UMockNetObjectPrioritizer::UpdateObjects(FNetObjectPrioritizerUpdateParams& Params)
{
	++CallStatus.CallCounts.UpdateObjects;

	bool bIsProperCall = true;

	// We expect all objects to get passed in a single call. This might not be true though.
	bIsProperCall &= Params.ObjectCount == AddedCount;
	CallStatus.SuccessfulCallCounts.UpdateObjects += bIsProperCall;
	for (const uint32 ObjectIndex : MakeArrayView(Params.ObjectIndices, Params.ObjectCount))
	{
		if (UPTRINT* InternalStateOffset = ObjectToPriorityOffset.Find(ObjectIndex))
		{
			// Warning: This is generally not safe, not even for primitive types.
			const float Priority = *reinterpret_cast<const float*>((*Params.StateBuffers)[ObjectIndex] + *InternalStateOffset);
			ObjectToPriority.Emplace(ObjectIndex, Priority);
		}
	}
}

void UMockNetObjectPrioritizer::Prioritize(FNetObjectPrioritizationParams& Params)
{
	++CallStatus.CallCounts.Prioritize;

	bool bIsProperCall = true;

	// We expect all objects to get passed in a single call. This might not be true though.
	bIsProperCall &= Params.ObjectCount == AddedCount;

	// Validate that we only need to prioritize objects that were added, that they're in order and unique.
	UE::Net::FInternalNetRefIndex LastObjectIndex = 0;
	for (UE::Net::FInternalNetRefIndex ObjectIndex : MakeArrayView(Params.ObjectIndices, Params.ObjectCount))
	{
		bIsProperCall &= (AddedIndices.IsBitSet(ObjectIndex));
		bIsProperCall &= (ObjectIndex > LastObjectIndex);

		LastObjectIndex = ObjectIndex;

		// Prioritize
		if (const float* Priority = ObjectToPriority.Find(ObjectIndex))
		{
			Params.Priorities[ObjectIndex] = *Priority;
		}
		else
		{
			Params.Priorities[ObjectIndex] = DefaultPriority;
		}
	}

	CallStatus.SuccessfulCallCounts.Prioritize += bIsProperCall;
}

float UMockNetObjectPrioritizer::GetPriority(UE::Net::FInternalNetRefIndex ObjectIndex) const
{
	check(AddedIndices.IsBitSet(ObjectIndex));
	if (const float* Priority = ObjectToPriority.Find(ObjectIndex))
	{
		return *Priority;
	}

	return DefaultPriority;
}

