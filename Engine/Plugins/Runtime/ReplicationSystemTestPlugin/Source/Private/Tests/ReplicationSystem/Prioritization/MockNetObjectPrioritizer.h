// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/Prioritization/NetObjectPrioritizer.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/ReplicationSystem/RepTag.h"
#include "MockNetObjectPrioritizer.generated.h"

namespace UE::Net
{
	typedef uint32 FInternalNetRefIndex;
}

const UE::Net::FRepTag RepTag_NetTest_Priority = 0x22C6880A98E628EDULL;

UCLASS()
class UMockNetObjectPrioritizerConfig : public UNetObjectPrioritizerConfig
{
	GENERATED_BODY()

};

UCLASS()
class UMockNetObjectPrioritizer : public UNetObjectPrioritizer
{
	GENERATED_BODY()

public:
	struct FFunctionCallStatus
	{
		struct
		{
			uint32 Init;
			uint32 AddObject;
			uint32 RemoveObject;
			uint32 UpdateObjects;
			uint32 Prioritize;
		} CallCounts, SuccessfulCallCounts;
	};

	struct FFunctionCallSetup
	{
		FFunctionCallSetup() : AddObject({}) {}

		// AddObject
		struct FAddObject
		{
			bool ReturnValue;
		};

		FAddObject AddObject;
	};

	inline void SetFunctionCallSetup(const FFunctionCallSetup& Setup) { CallSetup = Setup; }
	inline const FFunctionCallStatus& GetFunctionCallStatus() const { return CallStatus; }
	inline void ResetFunctionCallStatus() { CallStatus = FFunctionCallStatus({}); }

	float GetPriority(UE::Net::FInternalNetRefIndex ObjectIndex) const;

protected:
	virtual void Init(FNetObjectPrioritizerInitParams& Params) override;
	virtual void Deinit() override {}
	virtual void OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex) override;
	virtual bool AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectPrioritizerAddObjectParams& Params) override;
	virtual void RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectPrioritizationInfo& Info) override;
	virtual void UpdateObjects(FNetObjectPrioritizerUpdateParams&) override;
	virtual void Prioritize(FNetObjectPrioritizationParams&) override;

private:
	UMockNetObjectPrioritizer();

	FFunctionCallStatus CallStatus;
	FFunctionCallSetup CallSetup;
	UE::Net::FNetBitArray AddedIndices;
	SIZE_T AddedCount;
	TMap<UE::Net::FInternalNetRefIndex, UPTRINT> ObjectToPriorityOffset;
	TMap<UE::Net::FInternalNetRefIndex, float> ObjectToPriority;

	static constexpr float DefaultPriority = 1.0f;
};
