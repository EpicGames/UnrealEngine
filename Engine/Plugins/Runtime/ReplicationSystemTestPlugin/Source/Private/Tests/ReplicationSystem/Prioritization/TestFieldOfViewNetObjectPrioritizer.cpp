// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/ReplicationSystem/Prioritization/TestFieldOfViewNetObjectPrioritizer.h"
#include "Tests/ReplicationSystem/Prioritization/TestNetObjectPrioritizerFixture.h"


UFieldOfViewNetObjectPrioritizerTestConfig::UFieldOfViewNetObjectPrioritizerTestConfig()
{
	// Intentioally left blank. Do not remove.
}

UFieldOfViewNetObjectPrioritizerForConeTestConfig::UFieldOfViewNetObjectPrioritizerForConeTestConfig()
: Super()
{
	InnerSpherePriority = 0.0f;
	OuterSpherePriority = 0.0f;
	LineOfSightPriority = 0.0f;
}

namespace UE::Net::Private
{

class FTestFieldOfViewNetObjectPrioritizer : public FTestNetObjectPrioritizerFixture
{
protected:
	using Super = FTestNetObjectPrioritizerFixture;

	virtual void SetUp() override
	{
		Super::SetUp();
		InitFoVNetObjectPrioritizer();
	}

	virtual void TearDown() override
	{
		FoVPrioritizerHandle = InvalidNetObjectPrioritizerHandle;
		Super::TearDown();
	}

private:
	// Called by Super::SetUp()
	virtual void GetPrioritizerDefinitions(TArray<FNetObjectPrioritizerDefinition>& InPrioritizerDefinitions) override
	{
		// Standard FieldOfView prioritizer
		{
			FNetObjectPrioritizerDefinition& PrioritizerDef = InPrioritizerDefinitions.Emplace_GetRef();
			PrioritizerDef.PrioritizerName = FName("FoVPrioritizer");
			PrioritizerDef.ClassName = FName("/Script/IrisCore.FieldOfViewNetObjectPrioritizer");
			PrioritizerDef.ConfigClassName = FName("/Script/ReplicationSystemTestPlugin.FieldOfViewNetObjectPrioritizerTestConfig");
		}

		// Prioritizer for cone test
		{
			FNetObjectPrioritizerDefinition& PrioritizerDef = InPrioritizerDefinitions.Emplace_GetRef();
			PrioritizerDef.PrioritizerName = FName("ConePrioritizer");
			PrioritizerDef.ClassName = FName("/Script/IrisCore.FieldOfViewNetObjectPrioritizer");
			PrioritizerDef.ConfigClassName = FName("/Script/ReplicationSystemTestPlugin.FieldOfViewNetObjectPrioritizerForConeTestConfig");
		}
	}

	void InitFoVNetObjectPrioritizer()
	{
		FoVPrioritizerHandle = Server->ReplicationSystem->GetPrioritizerHandle(FName("FoVPrioritizer"));
	}

protected:
	FNetObjectPrioritizerHandle FoVPrioritizerHandle = InvalidNetObjectPrioritizerHandle;
	static constexpr float SixtyDegreeFovInRadians = FMath::DegreesToRadians(60.0f);
};

}

namespace UE::Net::Private
{

UE_NET_TEST_FIXTURE(FTestFieldOfViewNetObjectPrioritizer, ViewPositionSameAsObjectGivesHighestPriority)
{
	const UFieldOfViewNetObjectPrioritizerTestConfig* PrioritizerConfig = GetDefault<UFieldOfViewNetObjectPrioritizerTestConfig>();

	const FVector TestPosition(-1000, 5000, -10000);

	TArray<FVector> WorldLocations;
	WorldLocations.Add(TestPosition);

	FReplicationView View = MakeReplicationView(TestPosition, FVector::ForwardVector, FMath::DegreesToRadians(60.0f));
	const FPrioritizationResult& Result = PrioritizeWorldLocations(View, FoVPrioritizerHandle, WorldLocations);

	// Can adjust to max all priorities in the config, but typically one sets the inner sphere priority to the max wanted priority.
	const float MaxPriority = PrioritizerConfig->InnerSpherePriority;
	UE_NET_ASSERT_EQ(Result.Priorities[0], MaxPriority);
}

UE_NET_TEST_FIXTURE(FTestFieldOfViewNetObjectPrioritizer, ViewPositionFarAwayFromObjectGivesLowPriority)
{
	const UFieldOfViewNetObjectPrioritizerTestConfig* PrioritizerConfig = GetDefault<UFieldOfViewNetObjectPrioritizerTestConfig>();

	const FVector TestPosition(-1000, 5000, -10000);

	TArray<FVector> WorldLocations;
	WorldLocations.Add(TestPosition);

	// Can adjust to max all priorities in the config, but typically one sets the inner sphere priority to the max wanted priority.
	const float MaxPriority = PrioritizerConfig->InnerSpherePriority;

	// One view
	{
		FReplicationView View = MakeReplicationView(TestPosition, FVector::ForwardVector, SixtyDegreeFovInRadians);

		const FPrioritizationResult& Result = PrioritizeWorldLocations(View, FoVPrioritizerHandle, WorldLocations);
		UE_NET_ASSERT_EQ(Result.Priorities[0], MaxPriority);
	}

	// Two views
	{
		FReplicationView View = MakeReplicationView(TestPosition, FVector::ForwardVector, SixtyDegreeFovInRadians);
		View.Views.Add(MakeReplicationView(FVector::ZeroVector, FVector::ForwardVector, SixtyDegreeFovInRadians).Views[0]);

		const FPrioritizationResult& Result = PrioritizeWorldLocations(View, FoVPrioritizerHandle, WorldLocations);
		UE_NET_ASSERT_EQ(Result.Priorities[0], MaxPriority);
	}

	// Three views
	{
		FReplicationView View = MakeReplicationView(-TestPosition, FVector::ForwardVector, SixtyDegreeFovInRadians);
		View.Views.Add(MakeReplicationView(TestPosition, FVector::ForwardVector, SixtyDegreeFovInRadians).Views[0]);
		View.Views.Add(MakeReplicationView(FVector::ZeroVector, FVector::ForwardVector, SixtyDegreeFovInRadians).Views[0]);

		const FPrioritizationResult& Result = PrioritizeWorldLocations(View, FoVPrioritizerHandle, WorldLocations);
		UE_NET_ASSERT_EQ(Result.Priorities[0], MaxPriority);
	}
}

UE_NET_TEST_FIXTURE(FTestFieldOfViewNetObjectPrioritizer, ViewPositionVeryFarAwayFromObjectGivesLowestPriority)
{
	const UFieldOfViewNetObjectPrioritizerTestConfig* PrioritizerConfig = GetDefault<UFieldOfViewNetObjectPrioritizerTestConfig>();

	const FVector TestPosition(0, 0, 0);

	TArray<FVector> WorldLocations;
	WorldLocations.Add(TestPosition);

	const float OutsideLength = FMath::Max(PrioritizerConfig->ConeLength, PrioritizerConfig->OuterSphereRadius) + 1.0f;
	const float MinPriority = PrioritizerConfig->OutsidePriority;

	// Test one view
	{
		FReplicationView View = MakeReplicationView(TestPosition + FVector(OutsideLength, 0, 0), FVector::ForwardVector, SixtyDegreeFovInRadians);

		const FPrioritizationResult& Result = PrioritizeWorldLocations(View, FoVPrioritizerHandle, WorldLocations);
		UE_NET_ASSERT_EQ(Result.Priorities[0], MinPriority);
	}

	// Test two views
	{
		FReplicationView View = MakeReplicationView(TestPosition + FVector::ForwardVector*OutsideLength, FVector::ForwardVector, SixtyDegreeFovInRadians);
		// Add second view even further away
		View.Views.Add(MakeReplicationView(TestPosition + FVector::ForwardVector*(2.0f*OutsideLength), FVector::ForwardVector, SixtyDegreeFovInRadians).Views[0]);

		const FPrioritizationResult& Result = PrioritizeWorldLocations(View, FoVPrioritizerHandle, WorldLocations);
		UE_NET_ASSERT_EQ(Result.Priorities[0], MinPriority);
	}

	// Test more than two views.
	{
		FReplicationView View = MakeReplicationView(TestPosition + FVector::ForwardVector*OutsideLength, FVector::ForwardVector, SixtyDegreeFovInRadians);
		// Add a couple of views
		View.Views.Add(MakeReplicationView(TestPosition + FVector::ForwardVector*(2.0f*OutsideLength), FVector::ForwardVector, SixtyDegreeFovInRadians).Views[0]);
		View.Views.Add(MakeReplicationView(TestPosition - FVector::ForwardVector*(4.0f*OutsideLength), FVector::ForwardVector, SixtyDegreeFovInRadians).Views[0]);

		const FPrioritizationResult& Result = PrioritizeWorldLocations(View, FoVPrioritizerHandle, WorldLocations);
		UE_NET_ASSERT_EQ(Result.Priorities[0], MinPriority);
	}
}

UE_NET_TEST_FIXTURE(FTestFieldOfViewNetObjectPrioritizer, ViewPositionInConeGivesReasonablePriority)
{
	const UFieldOfViewNetObjectPrioritizerForConeTestConfig* PrioritizerConfig = GetDefault<UFieldOfViewNetObjectPrioritizerForConeTestConfig>();
	
	const FNetObjectPrioritizerHandle ConePrioritizerHandle = Server->ReplicationSystem->GetPrioritizerHandle(FName("ConePrioritizer"));

	TArray<FVector> WorldLocations;
	// Start location is where the cone starts scaling the priority down to MinConePriority
	WorldLocations.Add(FVector(0, 0, 0));
	WorldLocations.Add(FVector(PrioritizerConfig->InnerConeLength, 0, 0));
	// Something between inner and total length, closer to the inner
	WorldLocations.Add(FVector(PrioritizerConfig->InnerConeLength + 0.25f*(PrioritizerConfig->ConeLength - PrioritizerConfig->InnerConeLength), 0, 0));
	// Something between inner and total length, closer to the total
	WorldLocations.Add(FVector(PrioritizerConfig->ConeLength - 0.25f*(PrioritizerConfig->ConeLength - PrioritizerConfig->InnerConeLength), 0, 0));
	// End location is where the cone ends
	WorldLocations.Add(FVector(PrioritizerConfig->ConeLength, 0, 0));
	WorldLocations.Add(FVector(PrioritizerConfig->ConeLength + 10.0f, 0, 0));

	FReplicationView View = MakeReplicationView(FVector::ZeroVector, FVector::ForwardVector, SixtyDegreeFovInRadians);
	const FPrioritizationResult& Result = PrioritizeWorldLocations(View, ConePrioritizerHandle, WorldLocations);

	UE_NET_ASSERT_EQ(Result.Priorities[0], PrioritizerConfig->MaxConePriority);
	UE_NET_ASSERT_TRUE(FMath::IsNearlyEqual(Result.Priorities[1], PrioritizerConfig->MaxConePriority, 0.01f));
	UE_NET_ASSERT_LT(Result.Priorities[2], Result.Priorities[1]);
	UE_NET_ASSERT_GT(Result.Priorities[2], Result.Priorities[3]);
	UE_NET_ASSERT_LT(Result.Priorities[3], Result.Priorities[2]);
	UE_NET_ASSERT_GT(Result.Priorities[3], Result.Priorities[4]);
	UE_NET_ASSERT_TRUE(FMath::IsNearlyEqual(Result.Priorities[4], PrioritizerConfig->MinConePriority, 0.01f));
	UE_NET_ASSERT_EQ(Result.Priorities[5], PrioritizerConfig->OutsidePriority);
}

UE_NET_TEST_FIXTURE(FTestFieldOfViewNetObjectPrioritizer, VisualizeFieldOfViewNetObjectPrioritizer)
{
	if (!FPlatformMisc::IsDebuggerPresent())
	{
		UE_NET_LOG("Not running VisualizeFieldOfViewNetObjectPrioritizer due to debugger not present.");
		return;
	}

	const UFieldOfViewNetObjectPrioritizerTestConfig* PrioritizerConfig = GetDefault<UFieldOfViewNetObjectPrioritizerTestConfig>();

	// The produced image needs to be viewed in a debugger capable of it or stored to a file and viewed in an external viewer.
	FVisualizationParams VisualizationParams;
	const FVector ViewPos(-15000, 10000, 5000);
	const FVector ViewDir = FVector::ForwardVector;
	VisualizationParams.View = MakeReplicationView(ViewPos, ViewDir, SixtyDegreeFovInRadians);
	VisualizationParams.PrioritizationBox.Min = ViewPos + FVector(-1, -1, 0)*(PrioritizerConfig->OuterSphereRadius + 100.0f);
	VisualizationParams.PrioritizationBox.Max = ViewPos + FVector(PrioritizerConfig->ConeLength + 100.0f, PrioritizerConfig->OuterSphereRadius + 100.0f, 0.0f);
	// 20 units per pixel means 100/20=5 pixels per meter.
	VisualizationParams.UnitsPerPixel = 20.0f;
	const FNetObjectPrioritizerImage& Image = Super::Visualize(FoVPrioritizerHandle, VisualizationParams);
	UE_NET_ASSERT_FALSE(Image.GreyScaleData.IsEmpty());
}

}
