// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/Prioritization/FieldOfViewNetObjectPrioritizer.h"
#include "TestFieldOfViewNetObjectPrioritizer.generated.h"

UCLASS()
class UFieldOfViewNetObjectPrioritizerTestConfig : public UFieldOfViewNetObjectPrioritizerConfig
{
	GENERATED_BODY()

public:
	UFieldOfViewNetObjectPrioritizerTestConfig();

};

// The cone test config will zero out all non-cone priorities.
UCLASS()
class UFieldOfViewNetObjectPrioritizerForConeTestConfig : public UFieldOfViewNetObjectPrioritizerConfig
{
	GENERATED_BODY()

public:
	UFieldOfViewNetObjectPrioritizerForConeTestConfig();

};
