// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/ReplicationSystem/Prioritization/TestNetObjectCountLimiter.h"
#include "Tests/ReplicationSystem/Prioritization/TestNetObjectPrioritizerFixture.h"

UNetObjectCountLimiterFillTestConfig::UNetObjectCountLimiterFillTestConfig()
{
	 Mode = ENetObjectCountLimiterMode::Fill;
	 MaxObjectCount = 1;
	 bEnableOwnedObjectsFastLane = true;
}

namespace UE::Net::Private
{

class FTestNetObjectCountLimiter : public FTestNetObjectPrioritizerFixture
{
protected:
	using Super = FTestNetObjectPrioritizerFixture;

	virtual void SetUp() override
	{
		Super::SetUp();
		InitNetObjectCountLimiter();
	}

	virtual void TearDown() override
	{
		NetObjectCountLimiterInFillModeHandle = InvalidNetObjectPrioritizerHandle;
		Super::TearDown();
	}

private:
	// Called by Super::SetUp()
	virtual void GetPrioritizerDefinitions(TArray<FNetObjectPrioritizerDefinition>& InPrioritizerDefinitions) override
	{
		// NetObjectCountLimiter in Fill mode.
		{
			FNetObjectPrioritizerDefinition& PrioritizerDef = InPrioritizerDefinitions.Emplace_GetRef();
			PrioritizerDef.PrioritizerName = FName("NetObjectCountLimiterInFillMode");
			PrioritizerDef.ClassName = FName("/Script/IrisCore.NetObjectCountLimiter");
			PrioritizerDef.ConfigClassName = FName("/Script/ReplicationSystemTestPlugin.NetObjectCountLimiterFillTestConfig");
		}
	}

	void InitNetObjectCountLimiter()
	{
		NetObjectCountLimiterInFillModeHandle = Server->ReplicationSystem->GetPrioritizerHandle(FName("NetObjectCountLimiterInFillMode"));
	}

protected:
	struct FTestObjects
	{
		TArray<FNetRefHandle> ServerNetRefHandles;
		TArray<UTestReplicatedIrisObject*> ServerObjects;
		TArray<UTestReplicatedIrisObject*> ClientObjects;
	};

	// Creates the specified number of objects and returns true if it succeeded. Makes sure the client objects exist too. Expects a client to exist to be able to resolve the client objects.
	bool CreateObjects(FTestObjects& OutObjects, uint32 ObjectCount, FNetObjectPrioritizerHandle PrioritizerHandle);

	// Waits for all server objects to be created on the client.
	bool WaitForClientObjectCreation(const TArray<FNetRefHandle>& ServerNetRefHandles) const;

	FNetObjectPrioritizerHandle NetObjectCountLimiterInFillModeHandle = InvalidNetObjectPrioritizerHandle;
};

}

namespace UE::Net::Private
{

UE_NET_TEST_FIXTURE(FTestNetObjectCountLimiter, PrioritizerExists)
{
	UE_NET_ASSERT_NE(NetObjectCountLimiterInFillModeHandle, InvalidNetObjectPrioritizerHandle);
}

UE_NET_TEST_FIXTURE(FTestNetObjectCountLimiter, FillLimitsNumberOfReplicatedObjects)
{
	const UNetObjectCountLimiterFillTestConfig* PrioritizerConfig = GetDefault<UNetObjectCountLimiterFillTestConfig>();

	FReplicationSystemTestClient* Client = CreateClient();

	// Create a bunch of objects, at least more than the number of objects the prioritizer is set to replicate per frame.
	FTestObjects TestObjects;
	const uint32 FrameCountToTest = 4U;
	const uint32 ObjectsToCreateCount = FrameCountToTest * PrioritizerConfig->MaxObjectCount;
	const bool bWereObjectsCreated = CreateObjects(TestObjects, ObjectsToCreateCount, NetObjectCountLimiterInFillModeHandle);
	UE_NET_ASSERT_TRUE(bWereObjectsCreated);

	// Modify all objects. Only MaxObjectCount should be updated per frame, but all of them should be updated eventually.
	{
		const int32 PrevIntAValue = TestObjects.ServerObjects[0]->IntA;
		const int32 NewIntAValue = PrevIntAValue + 1;
		for (UTestReplicatedIrisObject* ServerObject : TestObjects.ServerObjects)
		{
			ServerObject->IntA = NewIntAValue;
		}

		uint32 ObjectCountWithUpdatedValue = 0;
		for (uint32 FrameIt = 0, FrameEndIt = FrameCountToTest; FrameIt < FrameEndIt; ++FrameIt)
		{
			Server->UpdateAndSend({ Client });

			ObjectCountWithUpdatedValue = 0;
			const uint32 ExpectedObjectCountWithUpdatedValue = FrameIt + 1;
			for (const UTestReplicatedIrisObject* ClientObject : TestObjects.ClientObjects)
			{
				if (ClientObject->IntA == NewIntAValue)
				{
					++ObjectCountWithUpdatedValue;
				}
			}

			UE_NET_ASSERT_EQ(ObjectCountWithUpdatedValue, ExpectedObjectCountWithUpdatedValue);
		}
	}
}

UE_NET_TEST_FIXTURE(FTestNetObjectCountLimiter, FillReplicatesTheLeastRecentlyReplicatedObject)
{
	const UNetObjectCountLimiterFillTestConfig* PrioritizerConfig = GetDefault<UNetObjectCountLimiterFillTestConfig>();

	FReplicationSystemTestClient* Client = CreateClient();

	// Create a bunch of objects, at least more than the number of objects the prioritizer is set to replicate per frame.
	FTestObjects TestObjects;
	const uint32 FrameCountToTest = 5U;
	const uint32 ObjectsToCreateCount = FrameCountToTest * PrioritizerConfig->MaxObjectCount;
	const bool bWereObjectsCreated = CreateObjects(TestObjects, ObjectsToCreateCount, NetObjectCountLimiterInFillModeHandle);
	UE_NET_ASSERT_TRUE(bWereObjectsCreated);

	// Modify a single object for some time. Then modify an additional MaxObjectCount objects and make sure only the newly updated objects are replicated.
	{
		const int FirstIndexToTest[2] = { 0, static_cast<int32>(ObjectsToCreateCount) - 1 };
		const int DirectionOfNextIndices[2] = { +1, -1 };

		for (uint32 TestIt = 0, TestEndIt = UE_ARRAY_COUNT(FirstIndexToTest); TestIt < TestEndIt; ++TestIt)
		{
			for (uint32 FrameIt = 0, FrameEndIt = FrameCountToTest; FrameIt < FrameEndIt; ++FrameIt)
			{
				const int32 PrevIntAValue = TestObjects.ServerObjects[FirstIndexToTest[TestIt]]->IntA;
				const int32 NewIntAValue = PrevIntAValue + 1;

				// In the first few frames we'll modify the same object over and over again.
				if (FrameIt < FrameEndIt - 1U)
				{
					UTestReplicatedIrisObject* ServerObject = TestObjects.ServerObjects[FirstIndexToTest[TestIt]];
					ServerObject->IntA = NewIntAValue;

					Server->UpdateAndSend({ Client });

					const UTestReplicatedIrisObject* ClientObject = TestObjects.ClientObjects[FirstIndexToTest[TestIt]];
					UE_NET_ASSERT_EQ(ClientObject->IntA, NewIntAValue);
				}
				// In the last test frame we modify the first object as well as an additional MaxObjectCount objects. Only the latter should be replicated.
				else
				{
					// Modify values
					{
						UTestReplicatedIrisObject* ServerObject = TestObjects.ServerObjects[FirstIndexToTest[TestIt]];
						ServerObject->IntA = NewIntAValue;

						int32 NextIndexToTest = FirstIndexToTest[TestIt];
						for (uint32 ExtraObjectIt = 0, ExtraObjectEndIt = PrioritizerConfig->MaxObjectCount; ExtraObjectIt < ExtraObjectEndIt; ++ExtraObjectIt)
						{
							NextIndexToTest += DirectionOfNextIndices[TestIt];

							UTestReplicatedIrisObject* ExtraServerObject = TestObjects.ServerObjects[NextIndexToTest];
							ExtraServerObject->IntA = NewIntAValue;
						}
					}

					Server->UpdateAndSend({ Client });

					// Verify values
					{
						// The first object should not be replicated as the other ones were less recently replicated.
						const UTestReplicatedIrisObject* ClientObject = TestObjects.ClientObjects[FirstIndexToTest[TestIt]];
						UE_NET_ASSERT_EQ(ClientObject->IntA, PrevIntAValue);

						int32 NextIndexToTest = FirstIndexToTest[TestIt];
						for (uint32 ExtraObjectIt = 0, ExtraObjectEndIt = PrioritizerConfig->MaxObjectCount; ExtraObjectIt < ExtraObjectEndIt; ++ExtraObjectIt)
						{
							NextIndexToTest += DirectionOfNextIndices[TestIt];

							UTestReplicatedIrisObject* ExtraClientObject = TestObjects.ClientObjects[NextIndexToTest];
							UE_NET_ASSERT_EQ(ExtraClientObject->IntA, NewIntAValue);
						}
					}

					// Flush last change to the always modified object to not mess up with the second iteration of the test.
					{
						Server->UpdateAndSend({ Client });

						const UTestReplicatedIrisObject* ClientObject = TestObjects.ClientObjects[FirstIndexToTest[TestIt]];
						UE_NET_ASSERT_EQ(ClientObject->IntA, NewIntAValue);
					}
				}
			}
		}
	}
}

UE_NET_TEST_FIXTURE(FTestNetObjectCountLimiter, FillAlwaysReplicatesOwnedObjectAndTheLeastRecentlyReplicatedOnes)
{
	const UNetObjectCountLimiterFillTestConfig* PrioritizerConfig = GetDefault<UNetObjectCountLimiterFillTestConfig>();
	UE_NET_ASSERT_TRUE(PrioritizerConfig->bEnableOwnedObjectsFastLane);

	FReplicationSystemTestClient* Client = CreateClient();

	// Create a bunch of objects, at least more than the number of objects the prioritizer is set to replicate per frame.
	FTestObjects TestObjects;
	const uint32 FrameCountToTest = 10U;
	const uint32 ObjectsToCreateCount = FrameCountToTest * PrioritizerConfig->MaxObjectCount;
	const bool bWereObjectsCreated = CreateObjects(TestObjects, ObjectsToCreateCount, NetObjectCountLimiterInFillModeHandle);
	UE_NET_ASSERT_TRUE(bWereObjectsCreated);

	// Set an owner to an object
	constexpr uint32 OwnedObjectIndex = 0;
	Server->ReplicationSystem->SetOwningNetConnection(TestObjects.ServerObjects[OwnedObjectIndex]->NetRefHandle, Client->ConnectionIdOnServer);
	// Owner changes doesn't mark object as dirty and thus won't propagate to prioritizer requiring it until something updates. So let's modify a property to make that happen.
	TestObjects.ServerObjects[OwnedObjectIndex]->IntA++;

	// Add a push based subobject to each object.
	TArray<FNetRefHandle> ServerSubObjectNetRefHandles;
	TArray<UTestReplicatedIrisObject*> ServerSubObjects;
	TArray<UTestReplicatedIrisPushModelComponentWithObjectReference*> ServerComponents;
	TArray<UTestReplicatedIrisObject*> ClientSubObjects;
	TArray<UTestReplicatedIrisPushModelComponentWithObjectReference*> ClientComponents;

	for (FNetRefHandle OwnerNetRefHandle : TestObjects.ServerNetRefHandles)
	{
		// It's not obvious by any means but the component type with object reference has a push based property
		UTestReplicatedIrisObject* SubObject = Server->CreateSubObject(OwnerNetRefHandle, { .ObjectReferenceComponentCount = 1U });
		ServerSubObjectNetRefHandles.Add(SubObject->NetRefHandle);
		ServerSubObjects.Add(SubObject);
		ServerComponents.Add(SubObject->ObjectReferenceComponents[0].Get());
	}

	// Wait for all subobjects to be created
	{
		const bool bAreAllSubObjectsCreated = WaitForClientObjectCreation(ServerSubObjectNetRefHandles);
		UE_NET_ASSERT_TRUE(bAreAllSubObjectsCreated);

		for (FNetRefHandle SubObjectRefHandle : ServerSubObjectNetRefHandles)
		{
			UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectRefHandle));
			ClientSubObjects.Add(ClientSubObject);
			ClientComponents.Add(ClientSubObject->ObjectReferenceComponents[0].Get());
		}
	}

	// The test will dirty all subobjects every frame but verify that ownly the owned object and MaxObjectCount other objects are replicated each frame.
	// Test is agnostic to which order objects are replicated, such as internal index assignment.
	FNetBitArray UpdatedObjects(ObjectsToCreateCount);
	for (unsigned TestIt = 0, TestEndIt = 2*FrameCountToTest; TestIt < TestEndIt; ++TestIt)
	{
		// Modify all values
		for (UTestReplicatedIrisPushModelComponentWithObjectReference* ServerComp : ServerComponents)
		{
			ServerComp->ModifyIntA();
		}

		Server->UpdateAndSend({ Client });

		uint32 UpdatedObjectCount = 0;
		for (uint32 ObjectIndex = 0, ObjectEndIndex = static_cast<uint32>(ClientComponents.Num()); ObjectIndex < ObjectEndIndex; ++ObjectIndex)
		{
			const UTestReplicatedIrisPushModelComponentWithObjectReference* ServerComponent = ServerComponents[ObjectIndex];
			const UTestReplicatedIrisPushModelComponentWithObjectReference* ClientComponent = ClientComponents[ObjectIndex];
			// The owned object should always be replicated thanks to owner fast lane.
			if (ObjectIndex == OwnedObjectIndex)
			{
				UE_NET_ASSERT_EQ(ClientComponent->IntA, ServerComponent->IntA);
				UpdatedObjects.SetBit(ObjectIndex);
				++UpdatedObjectCount;
			}
			else
			{
				if (ClientComponent->IntA == ServerComponent->IntA)
				{
					UE_NET_ASSERT_FALSE(UpdatedObjects.GetBit(ObjectIndex));
					UpdatedObjects.SetBit(ObjectIndex);
					++UpdatedObjectCount;
					// Reset updated objects tracking if we've updated all objects
					if (UpdatedObjects.FindFirstZero() == FNetBitArray::InvalidIndex)
					{
						UpdatedObjects.ClearAllBits();
					}
				}
			}
		}

		UE_NET_ASSERT_EQ(UpdatedObjectCount, PrioritizerConfig->MaxObjectCount + 1U);
	}
}

// FTestNetObjectCountLimiter implementation
bool FTestNetObjectCountLimiter::CreateObjects(FTestObjects& OutObjects, uint32 ObjectCount, FNetObjectPrioritizerHandle PrioritizerHandle)
{
	// Need a client to have the client objects resolved.
	if (Clients.IsEmpty() || Clients[0] == nullptr)
	{
		return false;
	}

	OutObjects.ServerNetRefHandles.SetNum(ObjectCount);
	OutObjects.ServerObjects.SetNum(ObjectCount);
	OutObjects.ClientObjects.SetNum(ObjectCount);

	for (uint32 ObjIt = 0, ObjEndIt = ObjectCount; ObjIt < ObjEndIt; ++ObjIt)
	{
		UTestReplicatedIrisObject* ServerObject = Server->CreateObject({ .IrisComponentCount = 0 });
		OutObjects.ServerObjects[ObjIt] = ServerObject;
		OutObjects.ServerNetRefHandles[ObjIt] = ServerObject->NetRefHandle;

		Server->ReplicationSystem->SetPrioritizer(ServerObject->NetRefHandle, NetObjectCountLimiterInFillModeHandle);
	}

	const bool bAreAllObjectsCreated = WaitForClientObjectCreation(OutObjects.ServerNetRefHandles);
	if (!bAreAllObjectsCreated)
	{
		return false;
	}

	// Fill in the client objects
	FReplicationSystemTestClient* Client = Clients[0];
	for (uint32 ObjIt = 0, ObjEndIt = ObjectCount; ObjIt < ObjEndIt; ++ObjIt)
	{
		UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(OutObjects.ServerNetRefHandles[ObjIt]));
		OutObjects.ClientObjects[ObjIt] = ClientObject;
	}

	return true;
}

bool FTestNetObjectCountLimiter::WaitForClientObjectCreation(const TArray<FNetRefHandle>& ServerNetRefHandles) const
{
	if (ServerNetRefHandles.IsEmpty())
	{
		return true;
	}

	// Need a client to have the client objects resolved.
	if (Clients.IsEmpty() || Clients[0] == nullptr)
	{
		return false;
	}

	FReplicationSystemTestClient* Client = Clients[0];

	// Replicate all objects. Due to ReplicationWriter create priority all objects will be replicated and created regardless of which prioritizer is used.
	{
		TArray<UObject*> ClientObjects;
		ClientObjects.SetNumZeroed(ServerNetRefHandles.Num());

		bool bAreSomeObjectsNotCreated = true;
		int32 TryCount = ServerNetRefHandles.Num();
		while (bAreSomeObjectsNotCreated && TryCount >= 0)
		{
			bAreSomeObjectsNotCreated = false;
			--TryCount;

			Server->UpdateAndSend({ Client });

			for (uint32 ObjIt = 0, ObjEndIt = static_cast<uint32>(ServerNetRefHandles.Num()); ObjIt < ObjEndIt; ++ObjIt)
			{
				if (ClientObjects[ObjIt] == nullptr)
				{
					UObject* ClientObject = Client->GetReplicationBridge()->GetReplicatedObject(ServerNetRefHandles[ObjIt]);
					if (ClientObject == nullptr)
					{
						bAreSomeObjectsNotCreated = true;
						continue;
					}
					else
					{
						ClientObjects[ObjIt] = ClientObject;
					}
				}
			}
		}

		return !bAreSomeObjectsNotCreated;
	}
}

}
