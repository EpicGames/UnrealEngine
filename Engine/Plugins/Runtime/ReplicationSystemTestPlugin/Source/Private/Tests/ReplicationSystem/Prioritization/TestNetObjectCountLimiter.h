// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/Prioritization/NetObjectCountLimiter.h"
#include "TestNetObjectCountLimiter.generated.h"

// Config for NetObjectCuuntLimiter in Fill mode.
UCLASS()
class UNetObjectCountLimiterFillTestConfig : public UNetObjectCountLimiterConfig
{
	GENERATED_BODY()

public:
	UNetObjectCountLimiterFillTestConfig();
};
