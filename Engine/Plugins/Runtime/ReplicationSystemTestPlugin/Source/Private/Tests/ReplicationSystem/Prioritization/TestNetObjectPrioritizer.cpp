// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/Prioritization/NetObjectPrioritizerDefinitions.h"
#include "Templates/AlignmentTemplates.h"
#include "Tests/ReplicationSystem/Prioritization/MockNetObjectPrioritizer.h"
#include "Tests/ReplicationSystem/Prioritization/TestPrioritizationObject.h"
#include "Tests/ReplicationSystem/ReplicationSystemConfigOverrideTestFixture.h"
#include "Tests/ReplicationSystem/ReplicationSystemTestFixture.h"

namespace UE::Net::Private
{

class FTestNetObjectPrioritizerAPIFixtureBase
{
public:
	void InitNetObjectPrioritizerDefinitions()
	{
		const UClass* NetObjectPrioritizerDefinitionsClass = UNetObjectPrioritizerDefinitions::StaticClass();
		const FProperty* DefinitionsProperty = NetObjectPrioritizerDefinitionsClass->FindPropertyByName(TEXT("NetObjectPrioritizerDefinitions"));
		check(DefinitionsProperty != nullptr);

		// Save NetObjectPrioritizerDefinitions CDO state.
		UNetObjectPrioritizerDefinitions* PrioritizerDefinitions = GetMutableDefault<UNetObjectPrioritizerDefinitions>();
		DefinitionsProperty->CopyCompleteValue(&OriginalPrioritizerDefinitions, (void*)(UPTRINT(PrioritizerDefinitions) + DefinitionsProperty->GetOffset_ForInternal()));

		// Modify definitions to only include our mock prioritizers. Ugly... 
		TArray<FNetObjectPrioritizerDefinition> NewPrioritizerDefinitions;
		FNetObjectPrioritizerDefinition& MockDefinition = NewPrioritizerDefinitions.Emplace_GetRef();
		MockDefinition.PrioritizerName = FName("MockPrioritizer");
		MockDefinition.ClassName = FName("/Script/ReplicationSystemTestPlugin.MockNetObjectPrioritizer");
		MockDefinition.ConfigClassName = FName("/Script/ReplicationSystemTestPlugin.MockNetObjectPrioritizerConfig");
		DefinitionsProperty->CopyCompleteValue((void*)(UPTRINT(PrioritizerDefinitions) + DefinitionsProperty->GetOffset_ForInternal()), &NewPrioritizerDefinitions);
	}

	void RestoreNetObjectPrioritizerDefinitions()
	{
		// Restore NetObjectPrioritizerDefinitions CDO state from the saved state.
		const UClass* NetObjectPrioritizerDefinitionsClass = UNetObjectPrioritizerDefinitions::StaticClass();
		const FProperty* DefinitionsProperty = NetObjectPrioritizerDefinitionsClass->FindPropertyByName(TEXT("NetObjectPrioritizerDefinitions"));
		UNetObjectPrioritizerDefinitions* PrioritizerDefinitions = GetMutableDefault<UNetObjectPrioritizerDefinitions>();
		DefinitionsProperty->CopyCompleteValue((void*)(UPTRINT(PrioritizerDefinitions) + DefinitionsProperty->GetOffset_ForInternal()), &OriginalPrioritizerDefinitions);
		OriginalPrioritizerDefinitions.Empty();

		MockNetObjectPrioritizer = nullptr;
		MockPrioritizerHandle = InvalidNetObjectPrioritizerHandle;
	}

	void InitMockNetObjectPrioritizers(const UReplicationSystem* ReplicationSystem)
	{
		MockNetObjectPrioritizer = ExactCast<UMockNetObjectPrioritizer>(ReplicationSystem->GetPrioritizer(FName("MockPrioritizer")));
		MockPrioritizerHandle = ReplicationSystem->GetPrioritizerHandle(FName("MockPrioritizer"));
	}

protected:
	UMockNetObjectPrioritizer* MockNetObjectPrioritizer = nullptr;
	FNetObjectPrioritizerHandle MockPrioritizerHandle = InvalidNetObjectPrioritizerHandle;

private:
	TArray<FNetObjectPrioritizerDefinition> OriginalPrioritizerDefinitions;
};

class FTestNetObjectPrioritizerAPIFixture : public FReplicationSystemTestFixture, public FTestNetObjectPrioritizerAPIFixtureBase
{
protected:
	virtual void SetUp() override
	{
		InitNetObjectPrioritizerDefinitions();
		FReplicationSystemTestFixture::SetUp();
		InitMockNetObjectPrioritizers(ReplicationSystem);

		NetRefHandleManager = &ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
	}

	virtual void TearDown() override
	{
		FReplicationSystemTestFixture::TearDown();
		RestoreNetObjectPrioritizerDefinitions();

		NetRefHandleManager = nullptr;
	}

protected:
	FNetRefHandleManager* NetRefHandleManager = nullptr;
	static constexpr float FakeDeltaTime = 0.0334f;
};

class FTestNetObjectPrioritizationEdgeCaseFixture : public FReplicationSystemConfigOverrideRPCTestFixture, public FTestNetObjectPrioritizerAPIFixtureBase
{
protected:
	virtual void SetUp() override
	{
		InitNetObjectPrioritizerDefinitions();
		FReplicationSystemConfigOverrideRPCTestFixture::SetUp();

		CreateServer({.MaxReplicatedObjectCount = MaxObjectCount, .InitialNetObjectListCount = MaxObjectCount});
		ReplicationSystem = Server->GetReplicationSystem();
		InitMockNetObjectPrioritizers(ReplicationSystem);
	}

	virtual void TearDown() override
	{
		FReplicationSystemConfigOverrideRPCTestFixture::TearDown();
		RestoreNetObjectPrioritizerDefinitions();

		ReplicationSystem  = nullptr;
	}

protected:
	// This constant needs to match FReplicationPrioritization::FPrioritizerBatchHelper::MaxObjectCountPerBatch for tests to be effective
	static constexpr uint32 BatchObjectCount = 1024;
	static constexpr uint32 MaxObjectCount = Align(BatchObjectCount + 1, 64);

	UReplicationSystem* ReplicationSystem = nullptr;
};

UE_NET_TEST_FIXTURE(FTestNetObjectPrioritizerAPIFixture, CanGetPrioritizerHandle)
{
	UE_NET_ASSERT_NE(MockPrioritizerHandle, InvalidNetObjectPrioritizerHandle);
}

UE_NET_TEST_FIXTURE(FTestNetObjectPrioritizerAPIFixture, PrioritizerInitWasCalled)
{
	const UMockNetObjectPrioritizer::FFunctionCallStatus& FunctionCallStatus = MockNetObjectPrioritizer->GetFunctionCallStatus();
	UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.Init, 1U);
	UE_NET_ASSERT_EQ(FunctionCallStatus.SuccessfulCallCounts.Init, 1U);
}

UE_NET_TEST_FIXTURE(FTestNetObjectPrioritizerAPIFixture, PrioritizerAddObjectSucceedsAndRemoveObjectIsCalledWhenObjectIsDestroyed)
{
	UTestReplicatedIrisObject* TestObject = CreateObject(0, 0);
	UE_NET_ASSERT_NE(TestObject, nullptr);

	// Check AddObject is called. At this point RemoveObject should not be called.
	{
		MockNetObjectPrioritizer->ResetFunctionCallStatus();

		// We want the add call to succeed
		{
			UMockNetObjectPrioritizer::FFunctionCallSetup CallSetup;
			CallSetup.AddObject.ReturnValue = true;
			MockNetObjectPrioritizer->SetFunctionCallSetup(CallSetup);
		}

		const FNetRefHandle NetRefHandle = ReplicationBridge->BeginReplication(TestObject);
		ReplicationSystem->SetPrioritizer(NetRefHandle, MockPrioritizerHandle);

		// We don't know if prioritizer changes are batched, but we assume everything is setup correctly in PreSendUpdate at least.
		ReplicationSystem->NetUpdate(FakeDeltaTime);

		const UMockNetObjectPrioritizer::FFunctionCallStatus& FunctionCallStatus = MockNetObjectPrioritizer->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.AddObject, 1U);
		UE_NET_ASSERT_EQ(FunctionCallStatus.SuccessfulCallCounts.AddObject, 1U);
		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.RemoveObject, 0U);

		ReplicationSystem->PostSendUpdate();
	}

	// Check RemoveObject was called. AddObject should not have been called more times at this point.
	{
		MockNetObjectPrioritizer->ResetFunctionCallStatus();

		ReplicationBridge->EndReplication(TestObject);
		ReplicationSystem->NetUpdate(FakeDeltaTime);

		const UMockNetObjectPrioritizer::FFunctionCallStatus& FunctionCallStatus = MockNetObjectPrioritizer->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.RemoveObject, 1U);
		UE_NET_ASSERT_EQ(FunctionCallStatus.SuccessfulCallCounts.RemoveObject, 1U);
		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.AddObject, 0U);

		ReplicationSystem->PostSendUpdate();
	}
}

UE_NET_TEST_FIXTURE(FTestNetObjectPrioritizerAPIFixture, PrioritizerAddObjectFailsAndRemoveObjectIsNotCalledWhenObjectIsDestroyed)
{
	UTestReplicatedIrisObject* TestObject = CreateObject(0, 0);
	UE_NET_ASSERT_NE(TestObject, nullptr);

	// Check AddObject is called and fails. At this point RemoveObject should not be called.
	{
		MockNetObjectPrioritizer->ResetFunctionCallStatus();

		// We want the add call to fail
		{
			UMockNetObjectPrioritizer::FFunctionCallSetup CallSetup;
			CallSetup.AddObject.ReturnValue = false;
			MockNetObjectPrioritizer->SetFunctionCallSetup(CallSetup);
		}

		const FNetRefHandle NetRefHandle = ReplicationBridge->BeginReplication(TestObject);
		ReplicationSystem->SetPrioritizer(NetRefHandle, MockPrioritizerHandle);

		// We don't know if prioritizer changes are batched, but we assume everything is setup correctly in PreSendUpdate at least.
		ReplicationSystem->NetUpdate(FakeDeltaTime);

		const UMockNetObjectPrioritizer::FFunctionCallStatus& FunctionCallStatus = MockNetObjectPrioritizer->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.AddObject, 1U);
		UE_NET_ASSERT_EQ(FunctionCallStatus.SuccessfulCallCounts.AddObject, 0U);
		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.RemoveObject, 0U);

		ReplicationSystem->PostSendUpdate();
	}

	// Check RemoveObject is not called when object is destroyed.
	{
		MockNetObjectPrioritizer->ResetFunctionCallStatus();

		ReplicationBridge->EndReplication(TestObject);
		ReplicationSystem->NetUpdate(FakeDeltaTime);

		const UMockNetObjectPrioritizer::FFunctionCallStatus& FunctionCallStatus = MockNetObjectPrioritizer->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.RemoveObject, 0U);

		ReplicationSystem->PostSendUpdate();
	}
}

UE_NET_TEST_FIXTURE(FTestNetObjectPrioritizerAPIFixture, SwitchingPrioritizersCallsRemoveObjectOnPreviousPrioritizer)
{
	UTestReplicatedIrisObject* TestObject = CreateObject(0, 0);
	UE_NET_ASSERT_NE(TestObject, nullptr);

	// Check AddObject is called and succeeds.
	{
		MockNetObjectPrioritizer->ResetFunctionCallStatus();

		// We want the add call to succeed
		{
			UMockNetObjectPrioritizer::FFunctionCallSetup CallSetup;
			CallSetup.AddObject.ReturnValue = true;
			MockNetObjectPrioritizer->SetFunctionCallSetup(CallSetup);
		}

		const FNetRefHandle NetRefHandle = ReplicationBridge->BeginReplication(TestObject);
		ReplicationSystem->SetPrioritizer(NetRefHandle, MockPrioritizerHandle);

		// We don't know if prioritizer changes are batched, but we assume everything is setup correctly in PreSendUpdate at least.
		ReplicationSystem->NetUpdate(FakeDeltaTime);

		const UMockNetObjectPrioritizer::FFunctionCallStatus& FunctionCallStatus = MockNetObjectPrioritizer->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.AddObject, 1U);
		UE_NET_ASSERT_EQ(FunctionCallStatus.SuccessfulCallCounts.AddObject, 1U);

		ReplicationSystem->PostSendUpdate();
	}

	// Check RemoveObject is called when switching prioritizers.
	{
		MockNetObjectPrioritizer->ResetFunctionCallStatus();

		ReplicationSystem->SetStaticPriority(TestObject->NetRefHandle, 1.0f);

		ReplicationSystem->NetUpdate(FakeDeltaTime);

		const UMockNetObjectPrioritizer::FFunctionCallStatus& FunctionCallStatus = MockNetObjectPrioritizer->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.RemoveObject, 1U);
		UE_NET_ASSERT_EQ(FunctionCallStatus.SuccessfulCallCounts.RemoveObject, 1U);

		ReplicationSystem->PostSendUpdate();
	}
}

UE_NET_TEST_FIXTURE(FTestNetObjectPrioritizerAPIFixture, PrioritizeIsNotCalledWhenNoObjectsAreAddedToIt)
{
	UTestReplicatedIrisObject* TestObject = CreateObject(0, 0);
	UE_NET_ASSERT_NE(TestObject, nullptr);

	{
		MockNetObjectPrioritizer->ResetFunctionCallStatus();

		const FNetRefHandle NetRefHandle = ReplicationBridge->BeginReplication(TestObject);
		ReplicationSystem->SetStaticPriority(NetRefHandle, 1.0f);

		ReplicationSystem->NetUpdate(FakeDeltaTime);

		const UMockNetObjectPrioritizer::FFunctionCallStatus& FunctionCallStatus = MockNetObjectPrioritizer->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.AddObject, 0U);
		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.Prioritize, 0U);

		ReplicationSystem->PostSendUpdate();
	}
}

UE_NET_TEST_FIXTURE(FTestNetObjectPrioritizerAPIFixture, PrioritizeIsNotCalledWhenThereAreNoConnections)
{
	UTestReplicatedIrisObject* TestObject = CreateObject(0, 0);
	UE_NET_ASSERT_NE(TestObject, nullptr);

	{
		MockNetObjectPrioritizer->ResetFunctionCallStatus();

		// We want the add call to succeed
		{
			UMockNetObjectPrioritizer::FFunctionCallSetup CallSetup;
			CallSetup.AddObject.ReturnValue = true;
			MockNetObjectPrioritizer->SetFunctionCallSetup(CallSetup);
		}

		const FNetRefHandle NetRefHandle = ReplicationBridge->BeginReplication(TestObject);
		ReplicationSystem->SetPrioritizer(NetRefHandle, MockPrioritizerHandle);

		ReplicationSystem->NetUpdate(FakeDeltaTime);

		const UMockNetObjectPrioritizer::FFunctionCallStatus& FunctionCallStatus = MockNetObjectPrioritizer->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(FunctionCallStatus.SuccessfulCallCounts.AddObject, 1U);
		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.Prioritize, 0U);

		ReplicationSystem->PostSendUpdate();
	}
}

UE_NET_TEST_FIXTURE(FTestNetObjectPrioritizerAPIFixture, PrioritizeIsCalledWhenThereAreConnections)
{
	// Setup a couple of connections with valid views
	{
		ReplicationSystem->AddConnection(13);
		ReplicationSystem->AddConnection(37);

		FReplicationView View;
		View.Views.AddDefaulted();
		ReplicationSystem->SetReplicationView(13, View);
		ReplicationSystem->SetReplicationView(37, View);
	}

	UTestReplicatedIrisObject* TestObject1 = CreateObject(0, 0);
	UTestReplicatedIrisObject* TestObject2 = CreateObject(0, 0);
	UE_NET_ASSERT_NE(TestObject1, nullptr);
	UE_NET_ASSERT_NE(TestObject2, nullptr);

	{
		MockNetObjectPrioritizer->ResetFunctionCallStatus();

		// We want the add calls to succeed
		{
			UMockNetObjectPrioritizer::FFunctionCallSetup CallSetup;
			CallSetup.AddObject.ReturnValue = true;
			MockNetObjectPrioritizer->SetFunctionCallSetup(CallSetup);
		}

		const FNetRefHandle NetRefHandle1 = ReplicationBridge->BeginReplication(TestObject1);
		ReplicationSystem->SetPrioritizer(NetRefHandle1, MockPrioritizerHandle);

		const FNetRefHandle NetRefHandle2 = ReplicationBridge->BeginReplication(TestObject2);
		ReplicationSystem->SetPrioritizer(NetRefHandle2, MockPrioritizerHandle);

		ReplicationSystem->NetUpdate(FakeDeltaTime);

		const UMockNetObjectPrioritizer::FFunctionCallStatus& FunctionCallStatus = MockNetObjectPrioritizer->GetFunctionCallStatus();
		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.Prioritize, 2U);
		UE_NET_ASSERT_EQ(FunctionCallStatus.SuccessfulCallCounts.Prioritize, 2U);

		ReplicationSystem->PostSendUpdate();
	}

	// Remove connections
	{
		ReplicationSystem->RemoveConnection(13);
		ReplicationSystem->RemoveConnection(37);
	}
}

UE_NET_TEST_FIXTURE(FTestNetObjectPrioritizerAPIFixture, UpdateObjectsIsCalledForDirtyObject)
{
	UTestReplicatedIrisObject* TestObject1 = CreateObject(0, 0);
	UE_NET_ASSERT_NE(TestObject1, nullptr);

	// Add object to prioritizer
	{
		// We want the add calls to succeed
		{
			UMockNetObjectPrioritizer::FFunctionCallSetup CallSetup;
			CallSetup.AddObject.ReturnValue = true;
			MockNetObjectPrioritizer->SetFunctionCallSetup(CallSetup);
		}

		const FNetRefHandle NetRefHandle1 = ReplicationBridge->BeginReplication(TestObject1);
		ReplicationSystem->SetPrioritizer(NetRefHandle1, MockPrioritizerHandle);

		ReplicationSystem->NetUpdate(FakeDeltaTime);
		ReplicationSystem->PostSendUpdate();
	}

	// Dirty object and check that UpdateObjects is called
	{
		MockNetObjectPrioritizer->ResetFunctionCallStatus();

		TestObject1->IntA ^= 1;

		ReplicationSystem->NetUpdate(FakeDeltaTime);

		const UMockNetObjectPrioritizer::FFunctionCallStatus& FunctionCallStatus = MockNetObjectPrioritizer->GetFunctionCallStatus();

		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.UpdateObjects, 1U);
		UE_NET_ASSERT_EQ(FunctionCallStatus.SuccessfulCallCounts.UpdateObjects, 1U);

		ReplicationSystem->PostSendUpdate();
	}
}

UE_NET_TEST_FIXTURE(FTestNetObjectPrioritizerAPIFixture, UpdateObjectsIsNotCalledWhenNoObjectIsDirty)
{
	UTestReplicatedIrisObject* TestObject1 = CreateObject(0, 0);
	UE_NET_ASSERT_NE(TestObject1, nullptr);

	// Add object to prioritizer
	{
		// We want the add calls to succeed
		{
			UMockNetObjectPrioritizer::FFunctionCallSetup CallSetup;
			CallSetup.AddObject.ReturnValue = true;
			MockNetObjectPrioritizer->SetFunctionCallSetup(CallSetup);
		}

		const FNetRefHandle NetRefHandle1 = ReplicationBridge->BeginReplication(TestObject1);
		ReplicationSystem->SetPrioritizer(NetRefHandle1, MockPrioritizerHandle);

		ReplicationSystem->NetUpdate(FakeDeltaTime);
		ReplicationSystem->PostSendUpdate();
	}

	// Do not dirty object and check that UpdateObjects is not called
	{
		MockNetObjectPrioritizer->ResetFunctionCallStatus();

		ReplicationSystem->NetUpdate(FakeDeltaTime);

		const UMockNetObjectPrioritizer::FFunctionCallStatus& FunctionCallStatus = MockNetObjectPrioritizer->GetFunctionCallStatus();

		UE_NET_ASSERT_EQ(FunctionCallStatus.CallCounts.UpdateObjects, 0U);

		ReplicationSystem->PostSendUpdate();
	}
}

UE_NET_TEST_FIXTURE(FTestNetObjectPrioritizerAPIFixture, NativeIrisObjectGetsPriorityFromStart)
{
	UTestPrioritizationNativeIrisObject* Object = CreateObject<UTestPrioritizationNativeIrisObject>();

	{
		UMockNetObjectPrioritizer::FFunctionCallSetup CallSetup;
		CallSetup.AddObject.ReturnValue = true;
		MockNetObjectPrioritizer->SetFunctionCallSetup(CallSetup);
	}

	constexpr float ObjectPriority = 1.3056640625E10f;
	Object->SetPriority(ObjectPriority);

	const FNetRefHandle NetRefHandle = ReplicationBridge->BeginReplication(Object);
	ReplicationSystem->SetPrioritizer(NetRefHandle, MockPrioritizerHandle);

	ReplicationSystem->NetUpdate(FakeDeltaTime);
	ReplicationSystem->PostSendUpdate();

	UE::Net::FInternalNetRefIndex InternalIndex = NetRefHandleManager->GetInternalIndex(NetRefHandle);
	const float Priority = MockNetObjectPrioritizer->GetPriority(InternalIndex);
	UE_NET_ASSERT_EQ(Priority, ObjectPriority);
}

UE_NET_TEST_FIXTURE(FTestNetObjectPrioritizerAPIFixture, NativeIrisObjectGetsUpdatedPriority)
{
	UTestPrioritizationNativeIrisObject* Object = CreateObject<UTestPrioritizationNativeIrisObject>();

	{
		UMockNetObjectPrioritizer::FFunctionCallSetup CallSetup;
		CallSetup.AddObject.ReturnValue = true;
		MockNetObjectPrioritizer->SetFunctionCallSetup(CallSetup);
	}

	Object->SetPriority(2.0f);

	const FNetRefHandle NetRefHandle = ReplicationBridge->BeginReplication(Object);
	ReplicationSystem->SetPrioritizer(NetRefHandle, MockPrioritizerHandle);

	// Update priority
	constexpr float UpdatedPriority = 4711;
	Object->SetPriority(UpdatedPriority);

	ReplicationSystem->NetUpdate(FakeDeltaTime);
	ReplicationSystem->PostSendUpdate();

	UE::Net::FInternalNetRefIndex InternalIndex = NetRefHandleManager->GetInternalIndex(NetRefHandle);
	const float Priority = MockNetObjectPrioritizer->GetPriority(InternalIndex);
	UE_NET_ASSERT_EQ(Priority, UpdatedPriority);
}

UE_NET_TEST_FIXTURE(FTestNetObjectPrioritizerAPIFixture, ObjectGetsPriorityFromStart)
{
	UTestPrioritizationObject* Object = CreateObject<UTestPrioritizationObject>();

	{
		UMockNetObjectPrioritizer::FFunctionCallSetup CallSetup;
		CallSetup.AddObject.ReturnValue = true;
		MockNetObjectPrioritizer->SetFunctionCallSetup(CallSetup);
	}

	constexpr float ObjectPriority = 1.3056640625E10f;
	Object->SetPriority(ObjectPriority);

	const FNetRefHandle NetRefHandle = ReplicationBridge->BeginReplication(Object);
	ReplicationSystem->SetPrioritizer(NetRefHandle, MockPrioritizerHandle);

	ReplicationSystem->NetUpdate(FakeDeltaTime);
	ReplicationSystem->PostSendUpdate();

	UE::Net::FInternalNetRefIndex InternalIndex = NetRefHandleManager->GetInternalIndex(NetRefHandle);
	const float Priority = MockNetObjectPrioritizer->GetPriority(InternalIndex);
	UE_NET_ASSERT_EQ(Priority, ObjectPriority);
}

UE_NET_TEST_FIXTURE(FTestNetObjectPrioritizerAPIFixture, ObjectGetsUpdatedPriority)
{
	UTestPrioritizationObject* Object = CreateObject<UTestPrioritizationObject>();

	{
		UMockNetObjectPrioritizer::FFunctionCallSetup CallSetup;
		CallSetup.AddObject.ReturnValue = true;
		MockNetObjectPrioritizer->SetFunctionCallSetup(CallSetup);
	}

	Object->SetPriority(2.0f);

	const FNetRefHandle NetRefHandle = ReplicationBridge->BeginReplication(Object);
	ReplicationSystem->SetPrioritizer(NetRefHandle, MockPrioritizerHandle);

	// Update priority
	constexpr float UpdatedPriority = 4711;
	Object->SetPriority(UpdatedPriority);

	ReplicationSystem->NetUpdate(FakeDeltaTime);
	ReplicationSystem->PostSendUpdate();

	UE::Net::FInternalNetRefIndex InternalIndex = NetRefHandleManager->GetInternalIndex(NetRefHandle);
	const float Priority = MockNetObjectPrioritizer->GetPriority(InternalIndex);
	UE_NET_ASSERT_EQ(Priority, UpdatedPriority);
}

// Edge case tests
UE_NET_TEST_FIXTURE(FTestNetObjectPrioritizationEdgeCaseFixture, PrioritizationSystemHandlesObjectCountEdgeCaseGracefully)
{
	// First let's create the edge case scenario where we have BatchObjectCount objects and the last object is using the highest index available.
	{
		MockNetObjectPrioritizer->ResetFunctionCallStatus();

		// We want the add calls to succeed
		{
			UMockNetObjectPrioritizer::FFunctionCallSetup CallSetup;
			CallSetup.AddObject.ReturnValue = true;
			MockNetObjectPrioritizer->SetFunctionCallSetup(CallSetup);
		}

		// 1. First create objects that we'll later destroy so we get exactly 
		TArray<UTestReplicatedIrisObject*> ObjectsToDestroy;
		static_assert(MaxObjectCount > BatchObjectCount, "If MaxObjectCount <= BatchObjectCount we're unable to create the right amount of objects");
		{
			const uint32 ObjectCountToDestroy = MaxObjectCount - BatchObjectCount - 1;
			ObjectsToDestroy.Reserve(ObjectCountToDestroy);
			for (unsigned It = 0, EndIt = ObjectCountToDestroy; It < EndIt; ++It)
			{
				UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
				ObjectsToDestroy.Add(ServerObject);
			}
		}

		// 2. Then create BatchObjectCount objects using the same prioritizer.
		for (unsigned It = 0, EndIt = BatchObjectCount; It < EndIt; ++It)
		{
			UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
			ReplicationSystem->SetPrioritizer(ServerObject->NetRefHandle, MockPrioritizerHandle);
		}

		// 3. Destroy the first objects so we have exactly BatchObjectCount using the highest available internal indices
		for (UTestReplicatedIrisObject* Object : ObjectsToDestroy)
		{
			Server->DestroyObject(Object);
		}
	}
	
	// Setup a connection with a valid view
	{
		ReplicationSystem->AddConnection(13);

		FReplicationView View;
		View.Views.AddDefaulted();
		ReplicationSystem->SetReplicationView(13, View);
	}

	// There should be no ensures or checks when updating.
	FTestEnsureScope EnsureScope;

	ReplicationSystem->NetUpdate(0.1f);
	ReplicationSystem->PostSendUpdate();

	UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 0);
}

}
