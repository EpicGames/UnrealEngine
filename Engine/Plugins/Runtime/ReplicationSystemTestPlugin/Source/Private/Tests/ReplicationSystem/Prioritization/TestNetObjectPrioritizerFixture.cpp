// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/ReplicationSystem/Prioritization/TestNetObjectPrioritizerFixture.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"
#include "Tests/ReplicationSystem/Prioritization/TestPrioritizationObject.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"

namespace UE::Net
{

void FTestNetObjectPrioritizerFixture::SetUp()
{
	PrioritizerDefinitions.Reset();
	GetPrioritizerDefinitions(PrioritizerDefinitions);

	InitNetObjectPrioritizerDefinitions();

	FReplicationSystemServerClientTestFixture::SetUp();
}

void FTestNetObjectPrioritizerFixture::TearDown()
{
	FReplicationSystemServerClientTestFixture::TearDown();
	RestoreNetObjectPrioritizerDefinitions();
}

void FTestNetObjectPrioritizerFixture::GetPrioritizerDefinitions(TArray<FNetObjectPrioritizerDefinition>& InPrioritizerDefinitions)
{
}

void FTestNetObjectPrioritizerFixture::InitNetObjectPrioritizerDefinitions()
{
	const UClass* NetObjectPrioritizerDefinitionsClass = UNetObjectPrioritizerDefinitions::StaticClass();
	const FProperty* DefinitionsProperty = NetObjectPrioritizerDefinitionsClass->FindPropertyByName("NetObjectPrioritizerDefinitions");
	check(DefinitionsProperty != nullptr);

	// Save CDO state.
	UNetObjectPrioritizerDefinitions* DefaultPrioritizerDefinitions = GetMutableDefault<UNetObjectPrioritizerDefinitions>();
	DefinitionsProperty->CopyCompleteValue(&OriginalPrioritizerDefinitions, (void*)(UPTRINT(DefaultPrioritizerDefinitions) + DefinitionsProperty->GetOffset_ForInternal()));

	// Modify definitions to include the desired prioritizers.
	DefinitionsProperty->CopyCompleteValue((void*)(UPTRINT(DefaultPrioritizerDefinitions) + DefinitionsProperty->GetOffset_ForInternal()), &PrioritizerDefinitions);
}

void FTestNetObjectPrioritizerFixture::RestoreNetObjectPrioritizerDefinitions()
{
	const UClass* NetObjectPrioritizerDefinitionsClass = UNetObjectPrioritizerDefinitions::StaticClass();
	const FProperty* DefinitionsProperty = NetObjectPrioritizerDefinitionsClass->FindPropertyByName("NetObjectPrioritizerDefinitions");

	UNetObjectPrioritizerDefinitions* DefaultPrioritizerDefinitions = GetMutableDefault<UNetObjectPrioritizerDefinitions>();
	DefinitionsProperty->CopyCompleteValue((void*)(UPTRINT(DefaultPrioritizerDefinitions) + DefinitionsProperty->GetOffset_ForInternal()), &OriginalPrioritizerDefinitions);
}

FReplicationView FTestNetObjectPrioritizerFixture::MakeReplicationView(const FVector& ViewPos, const FVector& ViewDir, float ViewRadians)
{
	FReplicationView ReplicationView;
	FReplicationView::FView& View = ReplicationView.Views.Emplace_GetRef();
	View.Pos = ViewPos;
	View.Dir = ViewDir;
	View.FoVRadians = ViewRadians;
	return ReplicationView;
}

FTestNetObjectPrioritizerFixture::FPrioritizationResult FTestNetObjectPrioritizerFixture::PrioritizeWorldLocations(const FReplicationView& View, FNetObjectPrioritizerHandle PrioritizerHandle, TConstArrayView<FVector> WorldLocations)
{
	FReplicationSystemTestClient* Client = CreateClient();

	Server->GetReplicationSystem()->SetReplicationView(Client->ConnectionIdOnServer, View);

	// Need to set the world location update function to be able to set world locations.
	Server->GetReplicationBridge()->SetExternalWorldLocationUpdateFunctor([&](FNetRefHandle NetHandle, const UObject* Object, FVector& OutLocation, float& OutCullDistance)
		{
			if (const UTestPrioritizationWithWorldLocationObject* WorldLocObject = Cast<UTestPrioritizationWithWorldLocationObject>(Object))
			{
				OutLocation = WorldLocObject->GetWorldLocation();
				OutCullDistance = WorldLocObject->GetNetCullDistance();
			}
		});

	// Create objects but do not begin replication yet
	TArray<TObjectPtr<UTestPrioritizationWithWorldLocationObject>> WorldLocObjects;
	for (const FVector& WorldLocation : WorldLocations)
	{
		UTestPrioritizationWithWorldLocationObject* Object = NewObject<UTestPrioritizationWithWorldLocationObject>();
		Object->SetWorldLocation(WorldLocation);
		WorldLocObjects.Add(Object);
	}

	// Begin replication for all created objects
	{
		UObjectReplicationBridge::FRootObjectReplicationParams CreateNetRefHandleParams = { .bNeedsWorldLocationUpdate = true };

		for (UTestPrioritizationWithWorldLocationObject* Object : WorldLocObjects)
		{
			Object->NetRefHandle = Server->GetReplicationBridge()->BeginReplication(Object, CreateNetRefHandleParams);
			Server->ReplicationSystem->SetPrioritizer(Object->NetRefHandle, PrioritizerHandle);
		}
	}

	FPrioritizationResult TestResult;
	TestResult.Priorities.Reserve(WorldLocObjects.Num());

	// Prioritize.
	{
		Private::FReplicationPrioritization& ReplicationPrioritization = Server->GetReplicationSystem()->GetReplicationSystemInternal()->GetPrioritization();
		Private::FNetRefHandleManager& NetRefHandleManager = Server->GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager();

		Server->NetUpdate();

		{
			// Retrieve the priorities. These are floats for internal indices so we need to map the NetRefHandles to internal indices to construct the return value.
			TConstArrayView<float> Priorities = ReplicationPrioritization.GetPrioritiesForConnection(Client->ConnectionIdOnServer);
			for (UTestPrioritizationWithWorldLocationObject* Object : WorldLocObjects)
			{
				float Priority = 0.0f;
				FInternalNetRefIndex NetRefIndex = NetRefHandleManager.GetInternalIndex(Object->NetRefHandle);
				if (NetRefIndex != InvalidInternalNetRefIndex)
				{
					Priority = Priorities[NetRefIndex];
				}

				TestResult.Priorities.Add(Priority);
			}
		}

		Server->PostSendUpdate();
	}

	// Restore/reset world location update.
	Server->GetReplicationBridge()->SetExternalWorldLocationUpdateFunctor({});

	// Cleanup
	{
		DestroyClient(Client);

		for (UTestPrioritizationWithWorldLocationObject* Object : WorldLocObjects)
		{
			Server->ReplicationBridge->EndReplication(Object, EEndReplicationFlags::Destroy);
			Object->MarkAsGarbage();
		}

		WorldLocObjects.Empty();
	}

	return TestResult;
}

FTestNetObjectPrioritizerFixture::FPrioritizationResult FTestNetObjectPrioritizerFixture::PrioritizeObjects(const FReplicationView& View, TConstArrayView<FNetRefHandle> NetRefHandles)
{
	FReplicationSystemTestClient* Client = CreateClient();

	FPrioritizationResult TestResult;
	TestResult.Priorities.Reserve(NetRefHandles.Num());

	// Prioritize.
	{
		Private::FReplicationPrioritization& ReplicationPrioritization = Server->GetReplicationSystem()->GetReplicationSystemInternal()->GetPrioritization();
		Private::FNetRefHandleManager& NetRefHandleManager = Server->GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager();

		Server->NetUpdate();

		{
			// Retrieve the priorities. These are floats for internal indices so we need to map the NetRefHandles to internal indices to construct the return value.
			TConstArrayView<float> Priorities = ReplicationPrioritization.GetPrioritiesForConnection(Client->ConnectionIdOnServer);
			for (const FNetRefHandle& NetRefHandle : NetRefHandles)
			{
				float Priority = 0.0f;
				FInternalNetRefIndex NetRefIndex = NetRefHandleManager.GetInternalIndex(NetRefHandle);
				if (NetRefIndex != InvalidInternalNetRefIndex)
				{
					Priority = Priorities[NetRefIndex];
				}

				TestResult.Priorities.Add(Priority);
			}
		}

		Server->PostSendUpdate();
	}

	DestroyClient(Client);

	return TestResult;
}

FNetObjectPrioritizerImage FTestNetObjectPrioritizerFixture::Visualize(FNetObjectPrioritizerHandle PrioritizerHandle, const FVisualizationParams& Params)
{

	const double LocationDelta = Params.UnitsPerPixel;

	FReplicationSystemTestClient* Client = CreateClient();

	Server->GetReplicationSystem()->SetReplicationView(Client->ConnectionIdOnServer, Params.View);

	// Need to set the world location update function to be able to set world locations.
	Server->GetReplicationBridge()->SetExternalWorldLocationUpdateFunctor([&](FNetRefHandle NetHandle, const UObject* Object, FVector& OutLocation, float& OutCullDistance)
		{
			if (const UTestPrioritizationWithWorldLocationObject* WorldLocObject = Cast<UTestPrioritizationWithWorldLocationObject>(Object))
			{
				OutLocation = WorldLocObject->GetWorldLocation();
				OutCullDistance = WorldLocObject->GetNetCullDistance();
			}
		});

	FNetObjectPrioritizerImage Image;
	Image.ImageWidth = IntCastChecked<uint32>(int64((Params.PrioritizationBox.Max.X + LocationDelta - Params.PrioritizationBox.Min.X)/LocationDelta));
	Image.ImageHeight = IntCastChecked<uint32>(int64((Params.PrioritizationBox.Max.Y + LocationDelta - Params.PrioritizationBox.Min.Y)/LocationDelta));
	Image.GreyScaleData.SetNumZeroed(IntCastChecked<int64>(Image.ImageWidth*Image.ImageHeight));
	
	// Create objects for one horizontal line. UObject creation is slow.
	TArray<TObjectPtr<UTestPrioritizationWithWorldLocationObject>> WorldLocObjects;
	WorldLocObjects.Reserve(IntCastChecked<int32>(Image.ImageWidth));

	for (double X = Params.PrioritizationBox.Min.X; X <= Params.PrioritizationBox.Max.X; X += LocationDelta)
	{
		UTestPrioritizationWithWorldLocationObject* Object = NewObject<UTestPrioritizationWithWorldLocationObject>(GetTransientPackage(), FName(), RF_Transient);
		WorldLocObjects.Add(Object);
	}

	// Begin replication for all created objects
	{
		UObjectReplicationBridge::FRootObjectReplicationParams CreateNetRefHandleParams = { .bNeedsWorldLocationUpdate = true };

		for (UTestPrioritizationWithWorldLocationObject* Object : WorldLocObjects)
		{
			Object->NetRefHandle = Server->GetReplicationBridge()->BeginReplication(Object, CreateNetRefHandleParams);
			Server->ReplicationSystem->SetPrioritizer(Object->NetRefHandle, PrioritizerHandle);
		}
	}

	FPrioritizationResult PrioResult;
	PrioResult.Priorities.Reserve(WorldLocObjects.Num());

	// Loop through the entire box, creating one image line at a time.
	int32 ImageX = 0;
	int32 ImageY = 0;
	for (double Z = Params.PrioritizationBox.Min.Z; Z <= Params.PrioritizationBox.Max.Z; Z += LocationDelta)
	{
		ImageY = 0;
		for (double Y = Params.PrioritizationBox.Min.Y; Y <= Params.PrioritizationBox.Max.Y; Y += LocationDelta, ++ImageY)
		{
			ImageX = 0;
			for (double X = Params.PrioritizationBox.Min.X; X <= Params.PrioritizationBox.Max.X; X += LocationDelta, ++ImageX)
			{
				UTestPrioritizationWithWorldLocationObject* Object = WorldLocObjects[ImageX];
				Object->SetWorldLocation(FVector(X, Y, Z));
				Server->ReplicationSystem->ForceNetUpdate(Object->NetRefHandle);
			}

			// Prioritize.
			{
				Private::FReplicationPrioritization& ReplicationPrioritization = Server->GetReplicationSystem()->GetReplicationSystemInternal()->GetPrioritization();
				Private::FNetRefHandleManager& NetRefHandleManager = Server->GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager();

				Server->NetUpdate();

				{
					// Retrieve the priorities. These are floats for internal indices so we need to map the NetRefHandles to internal indices to construct the return value.
					TConstArrayView<float> Priorities = ReplicationPrioritization.GetPrioritiesForConnection(Client->ConnectionIdOnServer);
					for (UTestPrioritizationWithWorldLocationObject* Object : WorldLocObjects)
					{
						float Priority = 0.0f;
						FInternalNetRefIndex NetRefIndex = NetRefHandleManager.GetInternalIndex(Object->NetRefHandle);
						if (NetRefIndex != InvalidInternalNetRefIndex)
						{
							Priority = Priorities[NetRefIndex];
						}

						PrioResult.Priorities.Add(Priority);
					}
				}

				Server->PostSendUpdate();
			}

			// Update image data.
			for (uint32 Pixel = 0, PixelEnd = Image.ImageWidth; Pixel < PixelEnd; ++Pixel)
			{
				const uint32 PixelIndex = ImageY * Image.ImageWidth + Pixel;
				Image.GreyScaleData[PixelIndex] = FMath::Max(Image.GreyScaleData[PixelIndex], static_cast<uint8>(int32(255*FMath::Clamp(PrioResult.Priorities[PixelIndex], 0.0f, 1.0f))));
			}
		}
	}

	// Restore/reset world location update.
	Server->GetReplicationBridge()->SetExternalWorldLocationUpdateFunctor({});

	// Cleanup
	{
		DestroyClient(Client);

		for (UTestPrioritizationWithWorldLocationObject* Object : WorldLocObjects)
		{
			Server->ReplicationBridge->EndReplication(Object, EEndReplicationFlags::Destroy);
			Object->MarkAsGarbage();
		}

		WorldLocObjects.Empty();
	}

	if (Params.bGarbageCollectObjects)
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

	return Image;
}

}
