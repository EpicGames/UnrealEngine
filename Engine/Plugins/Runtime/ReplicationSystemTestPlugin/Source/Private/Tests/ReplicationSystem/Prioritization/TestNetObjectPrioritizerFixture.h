// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Iris/ReplicationSystem/Prioritization/NetObjectPrioritizer.h"
#include "Iris/ReplicationSystem/Prioritization/NetObjectPrioritizerDefinitions.h"
#include "Containers/ContainersFwd.h"
#include "UObject/NameTypes.h"

namespace UE::Net
{

struct FNetObjectPrioritizerImage
{
	// Width of image, in pixels.
	uint32 ImageWidth = 0;
	// Height of image, in pixels.
	uint32 ImageHeight = 0;
	// Image data where the value is mapped linearly from priority to value. 255 indicates a priority of 1.0f and above and 0.0f indicates a priority of 0.0f.
	TArray64<uint8> GreyScaleData;
};

class FTestNetObjectPrioritizerFixture : public FReplicationSystemServerClientTestFixture
{
protected:
	virtual void SetUp() override;

	virtual void TearDown() override;

	/** Subclasses need to override to get their prioritizers setup properly. */
	virtual void GetPrioritizerDefinitions(TArray<FNetObjectPrioritizerDefinition>& InPrioritizerDefinitions);

	static FReplicationView MakeReplicationView(const FVector& ViewPos, const FVector& ViewDir, float ViewRadians);

	struct FPrioritizationResult
	{
		// Array with priorities for each object tested
		TArray<float> Priorities;
	};

	/** Creates objects with specified world locations and prioritizes them using prioritizer named PrioritizerName. */
	FPrioritizationResult PrioritizeWorldLocations(const FReplicationView& View, FNetObjectPrioritizerHandle PrioritizerHandle, TConstArrayView<FVector> WorldLocations);

	/** Prioritizes objects with passed NetRefHandles. It is assumed the handles already have the appropritate prioritizer set. */
	FPrioritizationResult PrioritizeObjects(const FReplicationView& View, TConstArrayView<FNetRefHandle> NetRefHandles);

	struct FVisualizationParams
	{
		// View to be used to perform the prioritization and visualization.
		FReplicationView View;
		// The bounding box to prioritize. Keep it 2D unless you really have use for 3D prioritization for the image. X coords will be used for image width, Y coords will be image height.
		FBox PrioritizationBox;
		// Scaling factor. 1 UE unit is 1 cm. Unless very large images is desired or a very small box to be prioritized a value of 100 or more may be useful. 
		float UnitsPerPixel = 100.0f;
		// Visualization requires a lot of UObjects to be created. It's wise to perform garbage collection after the visualization has finished.
		bool bGarbageCollectObjects = true;
	};

	/**
	 * Produce grey scale image depicting how the prioritizer with the given name prioritizes. For the purpose of visualization priority values will be clamped to [0, 1.0f]
	 * and produce values in range [0, 255] such that a grey scale image is produced.
	 * If you want a clean picture make sure not to add any objects to the system prior to calling Visualize.
	 * NOTE: Very slow.
	 */
	FNetObjectPrioritizerImage Visualize(FNetObjectPrioritizerHandle PrioritizerHandle, const FVisualizationParams& Params);

private:
	void InitNetObjectPrioritizerDefinitions();

	void RestoreNetObjectPrioritizerDefinitions();

private:
	TArray<FNetObjectPrioritizerDefinition> OriginalPrioritizerDefinitions;
	TArray<FNetObjectPrioritizerDefinition> PrioritizerDefinitions;
};

}
