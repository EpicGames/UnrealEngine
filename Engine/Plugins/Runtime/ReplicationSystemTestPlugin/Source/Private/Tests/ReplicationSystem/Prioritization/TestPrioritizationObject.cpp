// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestPrioritizationObject.h"
#include "MockNetObjectPrioritizer.h" // For RepTag_NetTest_Priority
#include "Iris/ReplicationState/ReplicationStateDescriptorImplementationMacros.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Net/UnrealNetwork.h"


void UTestPrioritizationNativeIrisObject::SetPriority(float Priority)
{
	PriorityState.SetPriority(Priority);
}

void UTestPrioritizationNativeIrisObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	using namespace UE::Net;

	PriorityState.RegisterReplicationFragments(Context, RegistrationFlags);

}

UTestPrioritizationNativeIrisObject::FPriorityStateWrapper::FPriorityStateWrapper()
: Fragment(*this, State) 
{
}

void UTestPrioritizationNativeIrisObject::FPriorityStateWrapper::SetPriority(float Priority)
{
	State.SetPriority(Priority);
}

void UTestPrioritizationNativeIrisObject::FPriorityStateWrapper::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	Fragment.Register(Context);
}

void UTestPrioritizationNativeIrisObject::FPriorityStateWrapper::ApplyReplicationState(const FTestPrioritizationObjectPriorityNativeIrisState& InState, UE::Net::FReplicationStateApplyContext& Context)
{
	State = InState;
}

// UTestPrioritizationObject
void UTestPrioritizationObject::SetPriority(float Priority)
{
	NetTest_Priority = Priority;
}

void UTestPrioritizationObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	// Build descriptors and allocate PropertyReplicationFragments for this object
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

void UTestPrioritizationObject::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
	DOREPLIFETIME(ThisClass, NetTest_Priority);
}

// FTestPrioritizationObjectPriorityNativeIrisState

constexpr UE::Net::FReplicationStateMemberChangeMaskDescriptor FTestPrioritizationObjectPriorityNativeIrisState::sReplicationStateChangeMaskDescriptors[1];

IRIS_BEGIN_SERIALIZER_DESCRIPTOR(FTestPrioritizationObjectPriorityNativeIrisState)
IRIS_SERIALIZER_DESCRIPTOR(UE::Net::FFloatNetSerializer, nullptr)
IRIS_END_SERIALIZER_DESCRIPTOR()

IRIS_BEGIN_TRAITS_DESCRIPTOR(FTestPrioritizationObjectPriorityNativeIrisState)
IRIS_TRAITS_DESCRIPTOR(UE::Net::EReplicationStateMemberTraits::None)
IRIS_END_TRAITS_DESCRIPTOR()

IRIS_BEGIN_INTERNAL_TYPE_INFO(FTestPrioritizationObjectPriorityNativeIrisState)
IRIS_INTERNAL_TYPE_INFO(UE::Net::FFloatNetSerializer)
IRIS_END_INTERNAL_TYPE_INFO()

IRIS_BEGIN_MEMBER_DESCRIPTOR(FTestPrioritizationObjectPriorityNativeIrisState)
IRIS_MEMBER_DESCRIPTOR(FTestPrioritizationObjectPriorityNativeIrisState, Priority, 0)
IRIS_END_MEMBER_DESCRIPTOR()

IRIS_BEGIN_MEMBER_DEBUG_DESCRIPTOR(FTestPrioritizationObjectPriorityNativeIrisState)
IRIS_MEMBER_DEBUG_DESCRIPTOR(FTestPrioritizationObjectPriorityNativeIrisState, Priority)
IRIS_END_MEMBER_DEBUG_DESCRIPTOR()

IRIS_BEGIN_TAG_DESCRIPTOR(FTestPrioritizationObjectPriorityNativeIrisState)
IRIS_TAG_DESCRIPTOR(RepTag_NetTest_Priority, 0)
IRIS_END_TAG_DESCRIPTOR()

IRIS_BEGIN_FUNCTION_DESCRIPTOR(FTestPrioritizationObjectPriorityNativeIrisState)
IRIS_END_FUNCTION_DESCRIPTOR()

IRIS_BEGIN_REFERENCE_DESCRIPTOR(FTestPrioritizationObjectPriorityNativeIrisState)
IRIS_END_REFERENCE_DESCRIPTOR()

IRIS_IMPLEMENT_CONSTRUCT_AND_DESTRUCT(FTestPrioritizationObjectPriorityNativeIrisState)
IRIS_IMPLEMENT_REPLICATIONSTATEDESCRIPTOR(FTestPrioritizationObjectPriorityNativeIrisState)
