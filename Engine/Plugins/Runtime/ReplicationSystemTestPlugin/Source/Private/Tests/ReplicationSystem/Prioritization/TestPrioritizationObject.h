// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkAutomationTest.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "TestPrioritizationObject.generated.h"

namespace UE::Net
{
	struct FReplicationInstanceProtocol;
}

/*
 * Object used for testing the prioritization system.
 *
 */
#define IRIS_GENERATED_SECTION_FOR_FTestPrioritizationObjectPriorityNativeIrisState() \
private: \
	static constexpr UE::Net::FReplicationStateMemberChangeMaskDescriptor sReplicationStateChangeMaskDescriptors[] = { {0,1} }; \
	IRIS_DECLARE_COMMON(); \
public: \
	IRIS_ACCESS_BY_VALUE(Priority, float, 0);


class FTestPrioritizationObjectPriorityNativeIrisState
{
	IRIS_GENERATED_SECTION_FOR_FTestPrioritizationObjectPriorityNativeIrisState();

private:	
	float Priority = 0.0f;
};

UCLASS()
class UTestPrioritizationNativeIrisObject : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	void SetPriority(float Priority);

protected:
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

private:
	class FPriorityStateWrapper
	{
	public:
		FPriorityStateWrapper();

		void SetPriority(float Priority);

		void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags);
		void ApplyReplicationState(const FTestPrioritizationObjectPriorityNativeIrisState& State, UE::Net::FReplicationStateApplyContext& Context);

	private:
		FTestPrioritizationObjectPriorityNativeIrisState State;
		UE::Net::TReplicationFragment<FPriorityStateWrapper, FTestPrioritizationObjectPriorityNativeIrisState> Fragment;
	};

	FPriorityStateWrapper PriorityState;
};

UCLASS()
class UTestPrioritizationObject : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	void SetPriority(float Priority);

protected:
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

private:
	UPROPERTY(Replicated)
	float NetTest_Priority;
};

UCLASS()
class UTestPrioritizationWithWorldLocationObject : public UTestPrioritizationObject
{
	GENERATED_BODY()

public:
	void SetWorldLocation(const FVector& InWorldLocation)
	{
		WorldLocation = InWorldLocation;
	}

	const FVector& GetWorldLocation() const
	{
		return WorldLocation;
	}

	void SetNetCullDistance(float InNetCullDistance)
	{
		NetCullDistance = InNetCullDistance;
	}

	float GetNetCullDistance() const
	{
		return NetCullDistance;
	}

private:
	FVector WorldLocation = FVector::ZeroVector;
	float NetCullDistance = 0.0f;
};

