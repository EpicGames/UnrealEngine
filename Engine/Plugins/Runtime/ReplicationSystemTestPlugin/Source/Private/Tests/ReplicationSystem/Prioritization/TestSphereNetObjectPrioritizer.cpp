// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/ReplicationSystem/Prioritization/TestSphereNetObjectPrioritizer.h"
#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Algo/Reverse.h"
#include "HAL/PlatformMath.h"
#include "Misc/MemStack.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/Prioritization/NetObjectPrioritizerDefinitions.h"
#include "Tests/ReplicationSystem/ReplicationSystemTestFixture.h"


USphereNetObjectPrioritizerForTest::USphereNetObjectPrioritizerForTest()
{
}

void USphereNetObjectPrioritizerForTest::PrioritizeWithParams(USphereNetObjectPrioritizerForTest::FPrioritizeParams& Params)
{
	ensure(Params.InPositions.Num() == Params.OutPriorities.Num());

	if (Params.InPositions.Num() == 0)
	{
		return;
	}

	FMemory::Memzero(Params.OutPriorities.GetData(), Params.OutPriorities.Num()*sizeof(float));

	FMemStack& Mem = FMemStack::Get();
	FMemMark MemMark(Mem);

	// Arbitrary number. In the test framework we can go for a high number since we're not optimizing for memory.
	constexpr uint32 MaxBatchObjectCount = 4096U;
	const uint32 ObjectCount = static_cast<uint32>(Params.InPositions.Num());

	// Setup minimal prioritization params needed for the SetupBatchParams call.
	FNetObjectPrioritizationParams PrioritizationParams = {};
	PrioritizationParams.View = Params.View;

	uint32 BatchObjectCount = FPlatformMath::Min((ObjectCount + 3U) & ~3U, MaxBatchObjectCount);
	FBatchParams BatchParams;
	SetupBatchParams(BatchParams, PrioritizationParams, BatchObjectCount, Mem);

	for (uint32 ObjectIt = 0, ObjectEndIt = ObjectCount; ObjectIt < ObjectEndIt; )
	{
		const uint32 CurrentBatchObjectCount = FMath::Min(ObjectEndIt - ObjectIt, MaxBatchObjectCount);

		BatchParams.ObjectCount = CurrentBatchObjectCount;
		
		// Instead of PrepareBatch we copy positions and clear priorities.
		{
			const FVector* FirstPos = Params.InPositions.GetData();
			for (const FVector& Pos : MakeArrayView(Params.InPositions.GetData() + ObjectIt, CurrentBatchObjectCount))
			{
				const int32 Index = static_cast<int32>(&Pos - FirstPos);
				BatchParams.Positions[Index] = VectorLoadFloat3_W0(&Params.InPositions[Index]);
			}

			FPlatformMemory::Memcpy(&BatchParams.Priorities[ObjectIt], &Params.OutPriorities[ObjectIt], CurrentBatchObjectCount*sizeof(float));
		}
		
		PrioritizeBatch(BatchParams);
		
		// Instead of FinishBatch we copy priorities.
		{
			FPlatformMemory::Memcpy(&Params.OutPriorities[ObjectIt], &BatchParams.Priorities[ObjectIt], CurrentBatchObjectCount*sizeof(float));
		}

		ObjectIt += CurrentBatchObjectCount;
	}
}

namespace UE::Net::Private
{

class FTestSphereNetObjectPrioritizer : public FReplicationSystemTestFixture
{
public:
	FTestSphereNetObjectPrioritizer() : SphereNetObjectPrioritizer(nullptr) {}

protected:
	virtual void SetUp() override
	{
		InitNetObjectPrioritizerDefinitions();
		FReplicationSystemTestFixture::SetUp();
		InitSphereNetObjectPrioritizer();

		NetRefHandleManager = &ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
	}

	virtual void TearDown() override
	{
		FReplicationSystemTestFixture::TearDown();
		RestoreNetObjectPrioritizerDefinitions();

		NetRefHandleManager = nullptr;
	}

	static UE::Net::FReplicationView MakeReplicationView(const FVector& ViewPos, const FVector& ViewDir, float ViewRadians)
	{
		FReplicationView ReplicationView;
		FReplicationView::FView& View = ReplicationView.Views.Emplace_GetRef();
		View.Pos = ViewPos;
		View.Dir = ViewDir;
		View.FoVRadians = ViewRadians;
		return ReplicationView;
	}

private:
	void InitNetObjectPrioritizerDefinitions()
	{
		const UClass* NetObjectPrioritizerDefinitionsClass = UNetObjectPrioritizerDefinitions::StaticClass();
		const FProperty* DefinitionsProperty = NetObjectPrioritizerDefinitionsClass->FindPropertyByName(TEXT("NetObjectPrioritizerDefinitions"));
		check(DefinitionsProperty != nullptr);

		// Save NetObjectPrioritizerDefinitions CDO state.
		UNetObjectPrioritizerDefinitions* PrioritizerDefinitions = GetMutableDefault<UNetObjectPrioritizerDefinitions>();
		DefinitionsProperty->CopyCompleteValue(&OriginalPrioritizerDefinitions, (void*)(UPTRINT(PrioritizerDefinitions) + DefinitionsProperty->GetOffset_ForInternal()));

		// Modify definitions to only include our mock prioritizer. Ugly... 
		TArray<FNetObjectPrioritizerDefinition> NewPrioritizerDefinitions;
		FNetObjectPrioritizerDefinition& SphereDefinition = NewPrioritizerDefinitions.Emplace_GetRef();
		SphereDefinition.PrioritizerName = TEXT("SpherePrioritizer");
		SphereDefinition.ClassName = TEXT("/Script/ReplicationSystemTestPlugin.SphereNetObjectPrioritizerForTest");
		SphereDefinition.ConfigClassName = TEXT("/Script/IrisCore.SphereNetObjectPrioritizerConfig");
		DefinitionsProperty->CopyCompleteValue((void*)(UPTRINT(PrioritizerDefinitions) + DefinitionsProperty->GetOffset_ForInternal()), &NewPrioritizerDefinitions);
	}

	void RestoreNetObjectPrioritizerDefinitions()
	{
		// Restore NetObjectPrioritizerDefinitions CDO state from the saved state.
		const UClass* NetObjectPrioritizerDefinitionsClass = UNetObjectPrioritizerDefinitions::StaticClass();
		const FProperty* DefinitionsProperty = NetObjectPrioritizerDefinitionsClass->FindPropertyByName(TEXT("NetObjectPrioritizerDefinitions"));
		UNetObjectPrioritizerDefinitions* PrioritizerDefinitions = GetMutableDefault<UNetObjectPrioritizerDefinitions>();
		DefinitionsProperty->CopyCompleteValue((void*)(UPTRINT(PrioritizerDefinitions) + DefinitionsProperty->GetOffset_ForInternal()), &OriginalPrioritizerDefinitions);
		OriginalPrioritizerDefinitions.Empty();

		SpherePrioritizerHandle = InvalidNetObjectPrioritizerHandle;
		SphereNetObjectPrioritizer = nullptr;
	}

	void InitSphereNetObjectPrioritizer()
	{
		SphereNetObjectPrioritizer = ExactCast<USphereNetObjectPrioritizerForTest>(ReplicationSystem->GetPrioritizer(TEXT("SpherePrioritizer")));
		SphereNetObjectPrioritizerConfig = SphereNetObjectPrioritizer->GetConfig();
		SpherePrioritizerHandle = ReplicationSystem->GetPrioritizerHandle(FName("SpherePrioritizer"));
	}

protected:
	USphereNetObjectPrioritizerForTest* SphereNetObjectPrioritizer = nullptr;
	const USphereNetObjectPrioritizerConfig* SphereNetObjectPrioritizerConfig = nullptr;
	FNetObjectPrioritizerHandle SpherePrioritizerHandle = InvalidNetObjectPrioritizerHandle;
	FNetRefHandleManager* NetRefHandleManager = nullptr;
	static constexpr float FakeDeltaTime = 0.0334f;

private:
	TArray<FNetObjectPrioritizerDefinition> OriginalPrioritizerDefinitions;
};

}

namespace UE::Net::Private
{

UE_NET_TEST_FIXTURE(FTestSphereNetObjectPrioritizer, ViewPositionSameAsObjectGivesHighestPriority)
{
	USphereNetObjectPrioritizerForTest::FPrioritizeParams PrioParams;

	// Use an arbitrary position for the object.
	const FVector TestPosition(-10000, 5000000, -1000000);
	float OutPriority;

	PrioParams.InPositions = MakeArrayView(&TestPosition, 1);
	PrioParams.OutPriorities = MakeArrayView(&OutPriority, 1);

	// Test one view
	{
		PrioParams.View = MakeReplicationView(TestPosition, FVector(1, 0, 0), FMath::DegreesToRadians(60.0f));
		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);
		UE_NET_ASSERT_EQ(OutPriority, SphereNetObjectPrioritizerConfig->InnerPriority);
	}

	// Test two views
	{
		PrioParams.View = MakeReplicationView(TestPosition, FVector(1, 0, 0), FMath::DegreesToRadians(60.0f));
		// Add second view at origin.
		PrioParams.View.Views.Add(MakeReplicationView(FVector(0, 0, 0), FVector(1, 0, 0), FMath::DegreesToRadians(60.0f)).Views[0]);

		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);
		UE_NET_ASSERT_EQ(OutPriority, SphereNetObjectPrioritizerConfig->InnerPriority);

		// Reverse views. The result should be the same.
		Algo::Reverse(PrioParams.View.Views);
		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);
		UE_NET_ASSERT_EQ(OutPriority, SphereNetObjectPrioritizerConfig->InnerPriority);
	}

	// Test more than two views.
	{
		PrioParams.View = MakeReplicationView(TestPosition, FVector(1, 0, 0), FMath::DegreesToRadians(60.0f));
		// Add a couple of views.
		PrioParams.View.Views.Add(MakeReplicationView(FVector(0, 0, 0), FVector(1, 0, 0), FMath::DegreesToRadians(60.0f)).Views[0]);
		PrioParams.View.Views.Add(MakeReplicationView(-TestPosition, FVector(1, 0, 0), FMath::DegreesToRadians(60.0f)).Views[0]);

		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);
		UE_NET_ASSERT_EQ(OutPriority, SphereNetObjectPrioritizerConfig->InnerPriority);

		// Reverse views. The result should be the same.
		Algo::Reverse(PrioParams.View.Views);
		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);
		UE_NET_ASSERT_EQ(OutPriority, SphereNetObjectPrioritizerConfig->InnerPriority);
	}
}

UE_NET_TEST_FIXTURE(FTestSphereNetObjectPrioritizer, ViewPositionFarAwayFromObjectGivesLowPriority)
{
	USphereNetObjectPrioritizerForTest::FPrioritizeParams PrioParams;

	const FVector TestPosition(0, 0, 0);
	float OutPriority;

	PrioParams.InPositions = MakeArrayView(&TestPosition, 1);
	PrioParams.OutPriorities = MakeArrayView(&OutPriority, 1);

	// Test one view
	{
		PrioParams.View = MakeReplicationView(TestPosition + FVector(0, 0, SphereNetObjectPrioritizerConfig->OuterRadius), FVector(1, 0, 0), FMath::DegreesToRadians(60.0f));
		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);
		UE_NET_ASSERT_EQ(OutPriority, SphereNetObjectPrioritizerConfig->OuterPriority);
	}

	// Test two views
	{
		PrioParams.View = MakeReplicationView(TestPosition + FVector(0, 0, SphereNetObjectPrioritizerConfig->OuterRadius), FVector(1, 0, 0), FMath::DegreesToRadians(60.0f));
		// Add second view even further away
		PrioParams.View.Views.Add(MakeReplicationView(TestPosition + FVector(0, 0, 2.0f*SphereNetObjectPrioritizerConfig->OuterRadius), FVector(1, 0, 0), FMath::DegreesToRadians(60.0f)).Views[0]);

		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);
		UE_NET_ASSERT_EQ(OutPriority, SphereNetObjectPrioritizerConfig->OuterPriority);

		// Reverse views. The result should be the same.
		Algo::Reverse(PrioParams.View.Views);
		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);
		UE_NET_ASSERT_EQ(OutPriority, SphereNetObjectPrioritizerConfig->OuterPriority);
	}

	// Test more than two views.
	{
		PrioParams.View = MakeReplicationView(TestPosition + FVector(0, 0, SphereNetObjectPrioritizerConfig->OuterRadius), FVector(1, 0, 0), FMath::DegreesToRadians(60.0f));
		// Add a couple of views
		PrioParams.View.Views.Add(MakeReplicationView(TestPosition + FVector(0, 0, 2.0f*SphereNetObjectPrioritizerConfig->OuterRadius), FVector(1, 0, 0), FMath::DegreesToRadians(60.0f)).Views[0]);
		PrioParams.View.Views.Add(MakeReplicationView(TestPosition - FVector(0, 0, 4.0f*SphereNetObjectPrioritizerConfig->OuterRadius), FVector(1, 0, 0), FMath::DegreesToRadians(60.0f)).Views[0]);

		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);
		UE_NET_ASSERT_EQ(OutPriority, SphereNetObjectPrioritizerConfig->OuterPriority);

		// Reverse views. The result should be the same.
		Algo::Reverse(PrioParams.View.Views);
		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);
		UE_NET_ASSERT_EQ(OutPriority, SphereNetObjectPrioritizerConfig->OuterPriority);
	}
}

UE_NET_TEST_FIXTURE(FTestSphereNetObjectPrioritizer, ViewPositionVeryFarAwayFromObjectGivesLowestPriority)
{
	USphereNetObjectPrioritizerForTest::FPrioritizeParams PrioParams;

	const FVector TestPosition(0, 0, 0);
	float OutPriority;

	PrioParams.InPositions = MakeArrayView(&TestPosition, 1);
	PrioParams.OutPriorities = MakeArrayView(&OutPriority, 1);

	// Test one view
	{
		PrioParams.View = MakeReplicationView(TestPosition + SphereNetObjectPrioritizerConfig->OuterRadius, FVector(1, 0, 0), FMath::DegreesToRadians(60.0f));
		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);
		UE_NET_ASSERT_EQ(OutPriority, SphereNetObjectPrioritizerConfig->OutsidePriority);
	}

	// Test two views
	{
		PrioParams.View = MakeReplicationView(TestPosition + SphereNetObjectPrioritizerConfig->OuterRadius, FVector(1, 0, 0), FMath::DegreesToRadians(60.0f));
		// Add second view even further away
		PrioParams.View.Views.Add(MakeReplicationView(TestPosition + 2.0f*SphereNetObjectPrioritizerConfig->OuterRadius, FVector(1, 0, 0), FMath::DegreesToRadians(60.0f)).Views[0]);

		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);
		UE_NET_ASSERT_EQ(OutPriority, SphereNetObjectPrioritizerConfig->OutsidePriority);

		// Reverse views. The result should be the same.
		Algo::Reverse(PrioParams.View.Views);
		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);
		UE_NET_ASSERT_EQ(OutPriority, SphereNetObjectPrioritizerConfig->OutsidePriority);
	}

	// Test more than two views.
	{
		PrioParams.View = MakeReplicationView(TestPosition + SphereNetObjectPrioritizerConfig->OuterRadius, FVector(1, 0, 0), FMath::DegreesToRadians(60.0f));
		// Add a couple of views
		PrioParams.View.Views.Add(MakeReplicationView(TestPosition + 2.0f*SphereNetObjectPrioritizerConfig->OuterRadius, FVector(1, 0, 0), FMath::DegreesToRadians(60.0f)).Views[0]);
		PrioParams.View.Views.Add(MakeReplicationView(TestPosition - 4.0f*SphereNetObjectPrioritizerConfig->OuterRadius, FVector(1, 0, 0), FMath::DegreesToRadians(60.0f)).Views[0]);

		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);
		UE_NET_ASSERT_EQ(OutPriority, SphereNetObjectPrioritizerConfig->OutsidePriority);

		// Reverse views. The result should be the same.
		Algo::Reverse(PrioParams.View.Views);
		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);
		UE_NET_ASSERT_EQ(OutPriority, SphereNetObjectPrioritizerConfig->OutsidePriority);
	}
}

UE_NET_TEST_FIXTURE(FTestSphereNetObjectPrioritizer, ViewPositionInRadiusIntervalGivesReasonablePriority)
{
	USphereNetObjectPrioritizerForTest::FPrioritizeParams PrioParams;

	const FVector ViewPosition(0, 0, 0);
	const float InnerRadius = SphereNetObjectPrioritizerConfig->InnerRadius;
	const float SphereRadiusDiff = SphereNetObjectPrioritizerConfig->OuterRadius - InnerRadius;
	// Test positions in increasing distance from view.
	const FVector Positions[3] =
	{ 
		ViewPosition + FVector(InnerRadius + 0.13f*SphereRadiusDiff, 0, 0),
		ViewPosition + FVector(-(InnerRadius + 0.43*SphereRadiusDiff), 0, 0),
		ViewPosition + FVector(InnerRadius + 0.85f*SphereRadiusDiff, 0, 0),
	};

	float OutPriorities[UE_ARRAY_COUNT(Positions)];

	PrioParams.InPositions = MakeArrayView(Positions, UE_ARRAY_COUNT(Positions));
	PrioParams.OutPriorities = MakeArrayView(OutPriorities, UE_ARRAY_COUNT(OutPriorities));

	// Test single view
	{
		PrioParams.View = MakeReplicationView(ViewPosition, FVector(1, 0, 0), FMath::DegreesToRadians(60.0f));
		SphereNetObjectPrioritizer->PrioritizeWithParams(PrioParams);

		for (float Priority : OutPriorities)
		{
			UE_NET_ASSERT_LT(Priority, SphereNetObjectPrioritizerConfig->InnerPriority);
			UE_NET_ASSERT_GT(Priority, SphereNetObjectPrioritizerConfig->OuterPriority);
		}

		UE_NET_ASSERT_LT(OutPriorities[1], OutPriorities[0]);
		UE_NET_ASSERT_LT(OutPriorities[2], OutPriorities[1]);
	}
}

}
