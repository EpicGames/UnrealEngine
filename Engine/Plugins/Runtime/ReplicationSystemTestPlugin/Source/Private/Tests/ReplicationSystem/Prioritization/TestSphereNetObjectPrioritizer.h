// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/Prioritization/SphereNetObjectPrioritizer.h"
#include "TestSphereNetObjectPrioritizer.generated.h"

UCLASS()
class USphereNetObjectPrioritizerForTest : public USphereNetObjectPrioritizer
{
	GENERATED_BODY()

public:
	USphereNetObjectPrioritizerForTest();

	struct FPrioritizeParams
	{
		UE::Net::FReplicationView View;
		TArrayView<const FVector> InPositions;
		TArrayView<float> OutPriorities;
	};

	// Method for testing purposes.
	void PrioritizeWithParams(FPrioritizeParams& Params);

	const USphereNetObjectPrioritizerConfig* GetConfig() const
	{
		return Config.Get();
	}
};
