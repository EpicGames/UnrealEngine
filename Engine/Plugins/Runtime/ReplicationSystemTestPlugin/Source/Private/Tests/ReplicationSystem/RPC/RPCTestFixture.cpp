// Copyright Epic Games, Inc. All Rights Reserved.

#include "RPCTestFixture.h"

namespace UE::Net
{

void FRPCTestFixture::SetUp()
{
	UNetBlobHandlerDefinitions* BlobHandlerDefinitions = GetMutableDefault<UNetBlobHandlerDefinitions>();
	OriginalHandlerDefinitions = BlobHandlerDefinitions->ReadWriteHandlerDefinitions();

	HandlerDefinitions.Add({ TEXT("NetRPCHandler") });
	HandlerDefinitions.Add({ TEXT("PartialNetObjectAttachmentHandler") });
	HandlerDefinitions.Add({ TEXT("NetObjectBlobHandler") });

	BlobHandlerDefinitions->ReadWriteHandlerDefinitions() = HandlerDefinitions;

	Super::SetUp();
}

void FRPCTestFixture::TearDown()
{
	Super::TearDown();

	UNetBlobHandlerDefinitions* BlobHandlerDefinitions = GetMutableDefault<UNetBlobHandlerDefinitions>();
	BlobHandlerDefinitions->ReadWriteHandlerDefinitions() = OriginalHandlerDefinitions;
}

}
