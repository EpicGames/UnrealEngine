// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlobHandlerDefinitions.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"

namespace UE::Net
{

class FRPCTestFixture : public FReplicationSystemServerClientTestFixture
{
	typedef FReplicationSystemServerClientTestFixture Super;

public:
	FRPCTestFixture()
	{
	}

protected:
	virtual void SetUp() override;

	virtual void TearDown() override;

private:
	TArray<FNetBlobHandlerDefinition> OriginalHandlerDefinitions;
	TArray<FNetBlobHandlerDefinition> HandlerDefinitions;
};

}
