// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicatedTestObjectWithRPC.h"

#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"

#include "Net/UnrealNetwork.h"

//////////////////////////////////////////////////////////////////////////
// Implementation for UTestReplicatedObjectWithRPC
//////////////////////////////////////////////////////////////////////////
UTestReplicatedObjectWithRPC::UTestReplicatedObjectWithRPC()
	: UReplicatedTestObject()
{
}

void UTestReplicatedObjectWithRPC::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	this->ReplicationFragments.Reset();
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags, &this->ReplicationFragments);
}

void UTestReplicatedObjectWithRPC::Init(UReplicationSystem* InRepSystem)
{
	bIsServerObject = InRepSystem->IsServer();
	ReplicationSystem = InRepSystem;
}

void UTestReplicatedObjectWithRPC::SetRootObject(UTestReplicatedObjectWithRPC* InRootObject)
{
	check(InRootObject);
	RootObject = InRootObject;
}

int32 UTestReplicatedObjectWithRPC::GetFunctionCallspace(UFunction* Function, FFrame* Stack)
{
	check(!(Function->FunctionFlags & FUNC_Static));
	check(Function->FunctionFlags & FUNC_Net);

	const bool bIsOnServer = bIsServerObject;

	// get the top most function
	while (Function->GetSuperFunction() != nullptr)
	{
		Function = Function->GetSuperFunction();
	}

	// Multicast RPCs
	if ((Function->FunctionFlags & FUNC_NetMulticast))
	{
		if (bIsOnServer)
		{
			// Server should execute locally and call remotely
			return (FunctionCallspace::Local | FunctionCallspace::Remote);
		}
		else
		{
			return FunctionCallspace::Local;
		}
	}

	// if we are the authority
	if (bIsOnServer)
	{
		if (Function->FunctionFlags & FUNC_NetClient)
		{
			return FunctionCallspace::Remote;
		}
		else
		{
			return FunctionCallspace::Local;
		}

	}
	// if we are not the authority
	else
	{
		if (Function->FunctionFlags & FUNC_NetServer)
		{
			return FunctionCallspace::Remote;
		}
		else
		{
			// don't replicate
			return FunctionCallspace::Local;
		}
	}
}

bool UTestReplicatedObjectWithRPC::CallRemoteFunction(UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack)
{
	if (bIsSubObject)
	{
		return ReplicationSystem->SendRPC(RootObject, this, Function, Parameters);
	}
	else
	{
		return ReplicationSystem->SendRPC(this, nullptr, Function, Parameters);
	}
}

void UTestReplicatedObjectWithRPC::ClientRPC_Implementation()
{
	bClientRPCCalled = true;
	ClientRPCCallOrder = ++CallOrder;
}

void UTestReplicatedObjectWithRPC::ClientRPCWithParam_Implementation(int32 IntParam)
{
	ClientRPCWithParamCalled = IntParam;
	ClientRPCWithParamCallOrder = ++CallOrder;
}

void UTestReplicatedObjectWithRPC::ClientUnreliableRPC_Implementation()
{
	bClientUnreliableRPCCalled = true;
	ClientUnreliableRPCCallOrder = ++CallOrder;
}

void UTestReplicatedObjectWithRPC::ServerRPC_Implementation()
{
	bServerRPCCalled = true;
	ServerRPCCallOrder = ++CallOrder;
}

void UTestReplicatedObjectWithRPC::ServerRPCWithParam_Implementation(int32 IntParam)
{
	ServerRPCWithParamCalled = IntParam;
	ServerRPCWithParamCallOrder = ++CallOrder;
}

void UTestReplicatedObjectWithRPC::ServerRPCWithParamWithHiddenVirtualBase_Implementation(const FReplicatedStructWithHiddenVirtualBase& StructParam)
{
	LocalStructWithHiddenVirtualBaseParam = StructParam;
}

void UTestReplicatedObjectWithRPC::ServerUnreliableRPC_Implementation()
{
	bServerUnreliableRPCCalled = true;
	ServerUnreliableRPCCallOrder = ++CallOrder;
}

void UTestReplicatedObjectWithRPC::ServerUnreliableWithExportRPC_Implementation(const TArray<uint8>& Payload, UObject* Ref)
{
	bServerUnreliableWithExportRPCCalled = true;
	ServerUnreliableWithExportRPCCallOrder = ++CallOrder;
	++ServerUnreliableWithExportRPCCallCounter;
};

void UTestReplicatedObjectWithRPC::NetMulticast_MultiCastRPCSendImmediate_Implementation()
{
	NetMulticast_MultiCastRPCSendImmediateCallOrder = ++CallOrder;
}

void UTestReplicatedObjectWithRPC::NetMulticast_MultiCastRPC_Implementation()
{
	NetMulticast_MultiCastRPCCallOrder = ++CallOrder;
}

void UTestReplicatedObjectWithRPC::NetMulticast_ReliableMultiCastRPC_Implementation()
{
	NetMulticast_ReliableMultiCastRPCCallOrder = ++CallOrder;
}


//////////////////////////////////////////////////////////////////////////
// Implementation for UTestReplicatedObjectWithSingleRPC
//////////////////////////////////////////////////////////////////////////
UTestReplicatedObjectWithSingleRPC::UTestReplicatedObjectWithSingleRPC()
	: UReplicatedTestObject()
{
}

void UTestReplicatedObjectWithSingleRPC::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	this->ReplicationFragments.Reset();
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags, &this->ReplicationFragments);
}

void UTestReplicatedObjectWithSingleRPC::Init(UReplicationSystem* InRepSystem)
{
	bIsServerObject = InRepSystem->IsServer();
	ReplicationSystem = InRepSystem;
}

void UTestReplicatedObjectWithSingleRPC::SetRootObject(UTestReplicatedObjectWithSingleRPC* InRootObject)
{
	check(InRootObject);
	RootObject = InRootObject;
}

int32 UTestReplicatedObjectWithSingleRPC::GetFunctionCallspace(UFunction* Function, FFrame* Stack)
{
	check(!(Function->FunctionFlags & FUNC_Static));
	check(Function->FunctionFlags & FUNC_Net);

	const bool bIsOnServer = bIsServerObject;

	// get the top most function
	while (Function->GetSuperFunction() != nullptr)
	{
		Function = Function->GetSuperFunction();
	}

	// Multicast RPCs
	if ((Function->FunctionFlags & FUNC_NetMulticast))
	{
		if (bIsOnServer)
		{
			// Server should execute locally and call remotely
			return (FunctionCallspace::Local | FunctionCallspace::Remote);
		}
		else
		{
			return FunctionCallspace::Local;
		}
	}

	// if we are the authority
	if (bIsOnServer)
	{
		if (Function->FunctionFlags & FUNC_NetClient)
		{
			return FunctionCallspace::Remote;
		}
		else
		{
			return FunctionCallspace::Local;
		}

	}
	// if we are not the authority
	else
	{
		if (Function->FunctionFlags & FUNC_NetServer)
		{
			return FunctionCallspace::Remote;
		}
		else
		{
			// don't replicate
			return FunctionCallspace::Local;
		}
	}
}

bool UTestReplicatedObjectWithSingleRPC::CallRemoteFunction(UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack)
{
	if (bIsSubObject)
	{
		return ReplicationSystem->SendRPC(RootObject, this, Function, Parameters);
	}
	else
	{
		return ReplicationSystem->SendRPC(this, nullptr, Function, Parameters);
	}
}

void UTestReplicatedObjectWithSingleRPC::NetMulticast_ReliableMultiCastRPC_Implementation(int32 Int32Param)
{
	NetMulticast_ReliableMultiCastRPCCallOrder = ++CallOrder;
}


void UTestReplicatedObjectWithSingleRPC::NetMulticast_AnotherReliableMultiCastRPC_Implementation(double DoubleParam)
{
	NetMulticast_AnotherReliableMultiCastRPCCallOrder = ++CallOrder;
}

