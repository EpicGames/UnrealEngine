// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tests/ReplicationSystem/ReplicatedTestObject.h"

#include "ReplicatedTestObjectWithRPC.generated.h"

USTRUCT()
struct FReplicatedStructVirtualBase
{
	GENERATED_BODY()
	virtual ~FReplicatedStructVirtualBase() = default;

protected:
	int32 NotReplicatedVar;
};

// Exercise special case found in game code.
USTRUCT()
struct FReplicatedStructWithHiddenVirtualBase
#if CPP
: public FReplicatedStructVirtualBase
#endif
{
	GENERATED_BODY()

	UPROPERTY()
	FString TestString;

	UPROPERTY()
	bool bTestBool;
};

/**
 *  A test class for testing RPC replication
 */
UCLASS()
class UTestReplicatedObjectWithRPC : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UTestReplicatedObjectWithRPC();

	void Init(UReplicationSystem* InRepSystem);
	void SetRootObject(UTestReplicatedObjectWithRPC* InRootObject);

	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	virtual int32 GetFunctionCallspace(UFunction* Function, FFrame* Stack) override;
	virtual bool CallRemoteFunction(UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack) override;

	UReplicationSystem* ReplicationSystem = nullptr;

	// Network data only for test
	TArray<UE::Net::FReplicationFragment*> ReplicationFragments;

	// To determine if this object is located on the server or client
	bool bIsServerObject = false;

	// Our owner when the object is a subobject
	UTestReplicatedObjectWithRPC* RootObject = nullptr;

	// RPC test functions
	UFUNCTION(Reliable, Client)
	void ClientRPC();
	bool bClientRPCCalled = false;
	int32 ClientRPCCallOrder = 0;

	UFUNCTION(Reliable, Client)
	void ClientRPCWithParam(int32 IntParam);
	int32 ClientRPCWithParamCalled = 0;
	int32 ClientRPCWithParamCallOrder = 0;

	UFUNCTION(Unreliable, Client)
	void ClientUnreliableRPC();
	bool bClientUnreliableRPCCalled = false;
	int32 ClientUnreliableRPCCallOrder = 0;

	UFUNCTION(Reliable, Server)
	void ServerRPC();
	bool bServerRPCCalled = false;
	int32 ServerRPCCallOrder = 0;

	UFUNCTION(Reliable, Server)
	void ServerRPCWithParam(int32 IntParam);
	int32 ServerRPCWithParamCalled = 0;
	int32 ServerRPCWithParamCallOrder = 0;

	UFUNCTION(Reliable, Server)
	void ServerRPCWithParamWithHiddenVirtualBase(const FReplicatedStructWithHiddenVirtualBase& StructParam);
	FReplicatedStructWithHiddenVirtualBase LocalStructWithHiddenVirtualBaseParam;
	
	UFUNCTION(Unreliable, Server)
	void ServerUnreliableRPC();
	bool bServerUnreliableRPCCalled = false;
	int32 ServerUnreliableRPCCallOrder = 0;

	/** Server Unicast unreliable with reference */
	UFUNCTION(Unreliable, Server)
	void ServerUnreliableWithExportRPC(const TArray<uint8>& Payload, UObject* Ref);
	bool bServerUnreliableWithExportRPCCalled = false;
	int32 ServerUnreliableWithExportRPCCallOrder = 0;
	int32 ServerUnreliableWithExportRPCCallCounter = 0;

	UFUNCTION(NetMulticast, unreliable)
	void NetMulticast_MultiCastRPCSendImmediate();
	int32 NetMulticast_MultiCastRPCSendImmediateCallOrder = 0;

	UFUNCTION(NetMulticast, unreliable)
	void NetMulticast_MultiCastRPC();
	int32 NetMulticast_MultiCastRPCCallOrder = 0;

	UFUNCTION(NetMulticast, reliable)
	void NetMulticast_ReliableMultiCastRPC();
	int32 NetMulticast_ReliableMultiCastRPCCallOrder = 0;

	int32 CallOrder = 0;
};

/**
 *  A test class for testing RPC replication
 */
UCLASS()
class UTestReplicatedObjectWithSingleRPC : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UTestReplicatedObjectWithSingleRPC();

	void Init(UReplicationSystem* InRepSystem);
	void SetRootObject(UTestReplicatedObjectWithSingleRPC* InRootObject);

	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	virtual int32 GetFunctionCallspace(UFunction* Function, FFrame* Stack) override;
	virtual bool CallRemoteFunction(UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack) override;

	UReplicationSystem* ReplicationSystem = nullptr;

	// Network data only for test
	TArray<UE::Net::FReplicationFragment*> ReplicationFragments;

	// To determine if this object is located on the server or client
	bool bIsServerObject = false;

	// Our owner when the object is a subobject
	UTestReplicatedObjectWithSingleRPC* RootObject = nullptr;

	// RPC test functions
	UFUNCTION(NetMulticast, reliable)
	void NetMulticast_ReliableMultiCastRPC(int32 Int32Param);
	int32 NetMulticast_ReliableMultiCastRPCCallOrder = 0;

	UFUNCTION(NetMulticast, reliable)
	void NetMulticast_AnotherReliableMultiCastRPC(double DoubleParam);
	int32 NetMulticast_AnotherReliableMultiCastRPCCallOrder = 0;

	int32 CallOrder = 0;
};


