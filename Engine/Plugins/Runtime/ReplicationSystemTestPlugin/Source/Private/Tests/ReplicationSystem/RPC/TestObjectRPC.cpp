// Copyright Epic Games, Inc. All Rights Reserved.

#include "RPCTestFixture.h"
#include "ReplicatedTestObjectWithRPC.h"
#include "Containers/BitArray.h"
#include "HAL/IConsoleManager.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Tests/EnsureScope.h"
#include "Tests/CheckScope.h"
#include "AutoRTFM.h"

namespace UE::Net::Private
{

	UE_NET_TEST_FIXTURE(FRPCTestFixture, TestBasicObjectRPC)
	{
		// Add a client
		FReplicationSystemTestClient* Client = CreateClient();

		// Spawn object on server
		UTestReplicatedObjectWithRPC* ServerObject = Server->CreateObject<UTestReplicatedObjectWithRPC>();

		ServerObject->bIsServerObject = true;
		ServerObject->ReplicationSystem = Server->GetReplicationSystem();
		Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, 0x01);

		// Send and deliver packet
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, true);
		Server->PostSendUpdate();

		UTestReplicatedObjectWithRPC* ClientObject = Cast<UTestReplicatedObjectWithRPC>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		
		// Verify that created server handle now also exists on client
		UE_NET_ASSERT_TRUE(ClientObject != nullptr);

		ClientObject->ReplicationSystem = Client->GetReplicationSystem();

		// Call an RPC server->client
		ServerObject->ClientRPC();

		// Send and deliver packet
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, true);
		Server->PostSendUpdate();

		// Verify RPC reception
		UE_NET_ASSERT_TRUE(ClientObject->bClientRPCCalled);

		// Call an RPC server->client
		const int32 IntParam = 0xBABA;
		ServerObject->ClientRPCWithParam(0xBABA);

		// Send and deliver packet
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, true);
		Server->PostSendUpdate();

		// Verify RPC reception
		UE_NET_ASSERT_TRUE(ClientObject->ClientRPCWithParamCalled == IntParam);

		// Call an RPC client->server
		ClientObject->ServerRPC();

		// Send and deliver client packet
		Client->UpdateAndSend(Server);

		// Verify RPC reception
		UE_NET_ASSERT_TRUE(ServerObject->bServerRPCCalled);

		// Call an RPC client->server
		ClientObject->ServerRPCWithParam(IntParam);

		// Call an RPC client->server with hidden virtual base
		FReplicatedStructWithHiddenVirtualBase TestRpc;
		TestRpc.TestString = TEXT("TestString");
		ClientObject->ServerRPCWithParamWithHiddenVirtualBase(TestRpc);

		// Send and deliver client packet
		Client->UpdateAndSend(Server);

		// Verify RPC reception
		UE_NET_ASSERT_TRUE(ServerObject->ServerRPCWithParamCalled == IntParam);
	}

	UE_NET_TEST_FIXTURE(FRPCTestFixture, TestMultiCastSendImmediateRPC)
	{
		// Add a client
		FReplicationSystemTestClient* Client = CreateClient();

		// Spawn object on server
		UTestReplicatedObjectWithRPC* ServerObject = Server->CreateObject<UTestReplicatedObjectWithRPC>();

		ServerObject->bIsServerObject = true;
		ServerObject->ReplicationSystem = Server->GetReplicationSystem();
		Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, 0x01);

		// Setup NetMulticast_MultiCastRPCSendImmediateCallOrder to be sent immediately
		ServerObject->ReplicationSystem->SetRPCSendPolicyFlags(UTestReplicatedObjectWithRPC::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UTestReplicatedObjectWithRPC, NetMulticast_MultiCastRPCSendImmediate)), ENetObjectAttachmentSendPolicyFlags::SendImmediate);

		// Send and deliver packet
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, true);
		Server->PostSendUpdate();

		UTestReplicatedObjectWithRPC* ClientObject = Cast<UTestReplicatedObjectWithRPC>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		
		// Verify that created server handle now also exists on client
		UE_NET_ASSERT_TRUE(ClientObject != nullptr);

		ClientObject->ReplicationSystem = Client->GetReplicationSystem();

		// Send multicast RPCs Server->Client
		// This is a normal multicast rpc, it will be scheduled with replication of object
		ServerObject->NetMulticast_MultiCastRPC();

		// This is a send immediate multicast rpc, it should be scheduled using the OOB replication channel so it should be received before the normally flagged rpc.
		ServerObject->NetMulticast_MultiCastRPCSendImmediate();

		// Send and deliver packet, simulating a send from PostTickDispatch
		Server->TickPostReceive();
		Server->SendAndDeliverTo(Client, true);
		Server->PostSendUpdate();

		// Verify RPC reception, at this point we expect only the RPC flagged as ENetObjectAttachmentSendPolicyFlags::SendImmediate to have been received
		UE_NET_ASSERT_EQ(ClientObject->NetMulticast_MultiCastRPCCallOrder, 0);
		UE_NET_ASSERT_EQ(ClientObject->NetMulticast_MultiCastRPCSendImmediateCallOrder, 1);

		// Send and deliver packet
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, true);
		Server->PostSendUpdate();
		
		// Verify RPC reception, at this point we expect the other RPC to have been received as well.
		UE_NET_ASSERT_EQ(ClientObject->NetMulticast_MultiCastRPCSendImmediateCallOrder, 1);
		UE_NET_ASSERT_EQ(ClientObject->NetMulticast_MultiCastRPCCallOrder, 2);
	}

	UE_NET_TEST_FIXTURE(FRPCTestFixture, TestSubObjectRPC)
	{
		// Add a client
		FReplicationSystemTestClient* Client = CreateClient();

		// Spawn object on server
		UTestReplicatedObjectWithRPC* ServerRootObject = Server->CreateObject<UTestReplicatedObjectWithRPC>();
		ServerRootObject->Init(Server->GetReplicationSystem());

		const FNetRefHandle ServerRootObjectHandle = ServerRootObject->NetRefHandle;
		Server->ReplicationSystem->SetOwningNetConnection(ServerRootObjectHandle, 0x01);

		UTestReplicatedObjectWithRPC* ServerSubObject = Server->CreateSubObject<UTestReplicatedObjectWithRPC>(ServerRootObjectHandle);
		ServerSubObject->Init(Server->GetReplicationSystem());
		ServerSubObject->SetRootObject(ServerRootObject);

		const FNetRefHandle ServerSubObjectHandle = ServerSubObject->NetRefHandle;

		// Send and deliver packet
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, true);
		Server->PostSendUpdate();

		UTestReplicatedObjectWithRPC* ClientRootObject = Cast<UTestReplicatedObjectWithRPC>(Client->GetReplicationBridge()->GetReplicatedObject(ServerRootObjectHandle));

		// Verify that the root object exists on the client
		UE_NET_ASSERT_TRUE(ClientRootObject != nullptr);

		UTestReplicatedObjectWithRPC* ClientSubObject = Cast<UTestReplicatedObjectWithRPC>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObjectHandle));

		// Verify that the subobject also exists on the client
		UE_NET_ASSERT_TRUE(ClientSubObject != nullptr);

		ClientRootObject->Init(Client->GetReplicationSystem());
		ClientSubObject->Init(Client->GetReplicationSystem());
		ClientSubObject->SetRootObject(ClientRootObject);

		// Call an RPC server->client on the subobject
		ServerSubObject->ClientRPC();

		// Send and deliver packet
		Server->UpdateAndSend({ Client });

		// Verify RPC reception
		UE_NET_ASSERT_TRUE(ClientSubObject->bClientRPCCalled);

		// Call an RPC server->client on the subobject
		const int32 IntParam = 0xBABA;
		ServerSubObject->ClientRPCWithParam(0xBABA);

		// Send and deliver packet
		Server->UpdateAndSend({ Client });

		// Verify RPC reception
		UE_NET_ASSERT_TRUE(ClientSubObject->ClientRPCWithParamCalled == IntParam);

		// Call an RPC client->server
		ClientSubObject->ServerRPC();

		// Send and deliver client packet
		Client->UpdateAndSend(Server);

		// Verify RPC reception
		UE_NET_ASSERT_TRUE(ServerSubObject->bServerRPCCalled);

		// Call an RPC client->server
		ClientSubObject->ServerRPCWithParam(IntParam);

		// Send and deliver client packet
		Client->UpdateAndSend(Server);

		// Verify RPC reception
		UE_NET_ASSERT_TRUE(ServerSubObject->ServerRPCWithParamCalled == IntParam);
	}

	UE_NET_TEST_FIXTURE(FRPCTestFixture, TestUnreliableRPCIsOrderedWithReliableRPCToClient)
	{
		// Add a client
		FReplicationSystemTestClient* Client = CreateClient();

		// Spawn object on server
		UTestReplicatedObjectWithRPC* ServerObject = Server->CreateObject<UTestReplicatedObjectWithRPC>();

		ServerObject->bIsServerObject = true;
		ServerObject->ReplicationSystem = Server->GetReplicationSystem();
		Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

		// Send and deliver packet
		Server->UpdateAndSend({Client});

		UTestReplicatedObjectWithRPC* ClientObject = Cast<UTestReplicatedObjectWithRPC>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		
		// Verify that created server handle now also exists on client
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Call reliable + unreliable + reliable RPCs
		ServerObject->ClientRPC();
		ServerObject->ClientUnreliableRPC();
		ServerObject->ClientRPCWithParam(1);

		// Send and deliver packet
		Server->UpdateAndSend({Client});

		// Verify RPC reception
		UE_NET_ASSERT_TRUE(ClientObject->bClientRPCCalled);
		UE_NET_ASSERT_TRUE(ClientObject->bClientUnreliableRPCCalled);
		UE_NET_ASSERT_NE(ClientObject->ClientRPCWithParamCalled, 0);

		// Verify RPC call order
		UE_NET_ASSERT_LT(ClientObject->ClientRPCCallOrder, ClientObject->ClientUnreliableRPCCallOrder);
		UE_NET_ASSERT_LT(ClientObject->ClientUnreliableRPCCallOrder, ClientObject->ClientRPCWithParamCallOrder);
	}

	UE_NET_TEST_FIXTURE(FRPCTestFixture, TestUnreliableRPCIsOrderedWithReliableRPCToServer)
	{
		// Add a client
		FReplicationSystemTestClient* Client = CreateClient();

		// Spawn object on server
		UTestReplicatedObjectWithRPC* ServerObject = Server->CreateObject<UTestReplicatedObjectWithRPC>();

		ServerObject->bIsServerObject = true;
		ServerObject->ReplicationSystem = Server->GetReplicationSystem();
		Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

		// Send and deliver packet
		Server->UpdateAndSend({Client});

		UTestReplicatedObjectWithRPC* ClientObject = Cast<UTestReplicatedObjectWithRPC>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		
		// Verify that created server handle now also exists on client
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		ClientObject->ReplicationSystem = Client->GetReplicationSystem();

		// Call reliable + unreliable + reliable RPCs
		ClientObject->ServerRPC();
		ClientObject->ServerUnreliableRPC();
		ClientObject->ServerRPCWithParam(1);

		// Send and deliver packet
		Client->UpdateAndSend(Server);

		// Verify RPC reception
		UE_NET_ASSERT_TRUE(ServerObject->bServerRPCCalled);
		UE_NET_ASSERT_TRUE(ServerObject->bServerUnreliableRPCCalled);
		UE_NET_ASSERT_NE(ServerObject->ServerRPCWithParamCalled, 0);

		// Verify RPC call order
		UE_NET_ASSERT_LT(ServerObject->ServerRPCCallOrder, ServerObject->ServerUnreliableRPCCallOrder);
		UE_NET_ASSERT_LT(ServerObject->ServerUnreliableRPCCallOrder, ServerObject->ServerRPCWithParamCallOrder);
	}

	UE_NET_TEST_FIXTURE(FRPCTestFixture, TestUnreliableRPCIsNotResentAfterPacketLoss)
	{
		// Add a client
		FReplicationSystemTestClient* Client = CreateClient();

		// Spawn object on server
		UTestReplicatedObjectWithRPC* ServerObject = Server->CreateObject<UTestReplicatedObjectWithRPC>();

		ServerObject->bIsServerObject = true;
		ServerObject->ReplicationSystem = Server->GetReplicationSystem();
		Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

		// Send and deliver packet
		Server->UpdateAndSend({Client});

		UTestReplicatedObjectWithRPC* ClientObject = Cast<UTestReplicatedObjectWithRPC>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		
		// Verify that created server handle now also exists on client
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		// Call reliable + unreliable + reliable RPCs
		ServerObject->ClientRPC();
		ServerObject->ClientUnreliableRPC();
		ServerObject->ClientRPCWithParam(1);

		// Send and do not deliver packet
		Server->UpdateAndSend({Client}, DoNotDeliverPacket);

		// Verify no RPCs were received
		UE_NET_ASSERT_FALSE(ClientObject->bClientRPCCalled);
		UE_NET_ASSERT_FALSE(ClientObject->bClientUnreliableRPCCalled);
		UE_NET_ASSERT_EQ(ClientObject->ClientRPCWithParamCalled, 0);

		// Send and deliver packet
		Server->UpdateAndSend({Client}, DeliverPacket);

		// Verify unreliable RPC was dropped
		UE_NET_ASSERT_TRUE(ClientObject->bClientRPCCalled);
		UE_NET_ASSERT_FALSE(ClientObject->bClientUnreliableRPCCalled);
		UE_NET_ASSERT_NE(ClientObject->ClientRPCWithParamCalled, 0);

		// Verify RPC call order
		UE_NET_ASSERT_LT(ClientObject->ClientRPCCallOrder, ClientObject->ClientRPCWithParamCallOrder);
	}

	// This test is specifically written to exercise a path where we would cause bitstream errors by
	// posting rpc on an object only temporarily in scope, which would be assigned an internal index.
	// This same internal index would then be reused and would inherit previously posted attachments/rpcs possibly for a different type.
	// If rpc index would be valid for the new object and it did not have the same signature it would result in a bitstream error.
	UE_NET_TEST_FIXTURE(FRPCTestFixture, TestShortLivedSubObjectReliableRPC)
	{
		// Add a client
		FReplicationSystemTestClient* Client = CreateClient();

		// Spawn object on server
		UTestReplicatedObjectWithRPC* ServerRootObject = Server->CreateObject<UTestReplicatedObjectWithRPC>();
		ServerRootObject->Init(Server->GetReplicationSystem());

		const FNetRefHandle ServerRootObjectHandle = ServerRootObject->NetRefHandle;

		// Send and deliver packet
		Server->UpdateAndSend({Client});

		// Create subobject
		UTestReplicatedObjectWithRPC* ServerSubObject = Server->CreateSubObject<UTestReplicatedObjectWithRPC>(ServerRootObjectHandle);
		ServerSubObject->Init(Server->GetReplicationSystem());
		ServerSubObject->SetRootObject(ServerRootObject);

		const FNetRefHandle ServerSubObjectHandle = ServerSubObject->NetRefHandle;

		// Call rpc
		ServerSubObject->ClientRPC();

		// Destroy subobject
		Server->DestroyObject(ServerSubObject, EEndReplicationFlags::Destroy);

		// Send and deliver packet
		Server->UpdateAndSend({Client});

		// Spawn object on server
		UTestReplicatedObjectWithSingleRPC* ServerRootObject2 = Server->CreateObject<UTestReplicatedObjectWithSingleRPC>();
		ServerRootObject2->Init(Server->GetReplicationSystem());
		ServerRootObject2->NetMulticast_ReliableMultiCastRPC(3);

		// Send and deliver packet
		Server->UpdateAndSend({Client});
	}

	// This test is written to verify a bug with owner filtering and late join. There will be an ensure if this test isn't working.
	UE_NET_TEST_FIXTURE(FRPCTestFixture, TestSubObjectMulticastRPCIsNotReplicatedToNonOwningConnection)
	{
		constexpr SIZE_T OwningClientIndex = 0;
		constexpr SIZE_T NonOwningClientIndex = 1;

		// Add a client
		FReplicationSystemTestClient* ClientArray[2] = {};
		ClientArray[OwningClientIndex] = CreateClient();

		// Spawn owner filtered object with on server
		UTestReplicatedObjectWithRPC* ServerRootObject = Server->CreateObject<UTestReplicatedObjectWithRPC>();
		ServerRootObject->Init(Server->GetReplicationSystem());

		const FNetRefHandle ServerRootObjectHandle = ServerRootObject->NetRefHandle;
		
		// Turn on owner filter
		Server->ReplicationSystem->SetFilter(ServerRootObjectHandle, ToOwnerFilterHandle);
		Server->ReplicationSystem->SetOwningNetConnection(ServerRootObjectHandle, ClientArray[OwningClientIndex]->ConnectionIdOnServer);

		UTestReplicatedObjectWithRPC* ServerSubObject = Server->CreateSubObject<UTestReplicatedObjectWithRPC>(ServerRootObjectHandle);
		ServerSubObject->Init(Server->GetReplicationSystem());
		ServerSubObject->SetRootObject(ServerRootObject);

		// Send and deliver packet
		Server->UpdateAndSend({ClientArray[OwningClientIndex]});

		// Late join with second client
		ClientArray[NonOwningClientIndex] = CreateClient();

		// Call a multicast RPC server->client on the subobject
		ServerSubObject->NetMulticast_MultiCastRPC();

		// Send and deliver packet
		{
			FEnsureScope EnsureScope;
			FCheckScope CheckScope;

			Server->UpdateAndSend(ClientArray);

			// No ensures or checks should trigger
			UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 0);
			UE_NET_ASSERT_EQ(CheckScope.GetCount(), 0);
		}
	}

	// This test is written to test that we can control if ordered unreliable attachments are pruned or not after each tick.
	UE_NET_TEST_FIXTURE(FRPCTestFixture, TestOvercommitOrderedUnreliableRPC)
	{
		// This tests the behavior with and without DropUnsentOrderedUnreliableAttachmentAtEndOfTick enabled
		IConsoleVariable* CVarOverride = IConsoleManager::Get().FindConsoleVariable(TEXT("net.Iris.ReplicationWriter.MaxUnsentOrderedUnreliableAttachmentAtEndOfTick"));
		check(CVarOverride != nullptr && CVarOverride->IsVariableInt());
		const int32 OldCVarValue = CVarOverride->GetInt();

		// Force disable dropping at unsent unreliable attachments
		CVarOverride->Set(-1, ECVF_SetByCode);

		ON_SCOPE_EXIT
		{
			// Restore cvar
			CVarOverride->Set(OldCVarValue, ECVF_SetByCode);
		};

		// Add a client
		FReplicationSystemTestClient* Client = CreateClient();

		// Spawn object on server
		UTestReplicatedObjectWithRPC* ServerObject = Server->CreateObject<UTestReplicatedObjectWithRPC>();

		ServerObject->bIsServerObject = true;
		ServerObject->ReplicationSystem = Server->GetReplicationSystem();
		Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

		// Send and deliver packet
		Server->UpdateAndSend({Client});

		UTestReplicatedObjectWithRPC* ClientObject = Cast<UTestReplicatedObjectWithRPC>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		
		// Verify that created server handle now also exists on client
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		ClientObject->ReplicationSystem = Client->GetReplicationSystem();

		auto SendRpcs = [&ClientObject](uint32 RPCCount, uint32 PayloadSize, UObject* Reference)
		{
			for (uint32 i = 0; i<RPCCount; ++i)
			{
				TArray<uint8> Payload;
				Payload.SetNumZeroed(PayloadSize);
				ClientObject->ServerUnreliableWithExportRPC(Payload, Reference);
			}
		};

		const int32 PacketSize = 500;
		const int32 IssueRPCCount = 8;
		const int32 RPCPayloadSize = 128;
		const int32 ExpectedPerPacket = PacketSize / RPCPayloadSize;

		Client->SetMaxSendPacketSize(PacketSize);
	
		// Run test allowing backlog of ordered unreliable attachments
		{
			CVarOverride->Set(-1, ECVF_SetByCode);
	
			// Post a bunch of unreliable RPCs that will overcommit more unreliable attachments than we should be able to send.
			SendRpcs(IssueRPCCount, 128, nullptr);
	
			// We send a reliable RPC last so that we can query when all data has been transmitted (or not)
			ClientObject->ServerRPC();

			// Keep on sending until we have sent and acknowledged the reliable rpc.
			UDataStream* ClientDataStream = Client->ReplicationSystem->GetDataStream(Client->LocalConnectionId, "Replication");
			while (!ClientDataStream->HasAcknowledgedAllReliableData())
			{
				// Send and deliver packet
				Client->UpdateAndSend(Server);
				Server->UpdateAndSend({Client});
			}

			// Verify RPC reception
			UE_NET_ASSERT_TRUE(ServerObject->bServerRPCCalled);

			UE_NET_ASSERT_TRUE(ServerObject->bServerUnreliableWithExportRPCCalled);

			// We expect all unreliables to be be delivered
			UE_NET_ASSERT_EQ(ServerObject->ServerUnreliableWithExportRPCCallCounter, IssueRPCCount);
		}

		// Reset callcounters
		ServerObject->bServerUnreliableWithExportRPCCalled = false;
		ServerObject->ServerUnreliableWithExportRPCCallCounter = 0;

		// Run test where we discard unsent unreliable attachments after each tick/frame
		{
			CVarOverride->Set(0, ECVF_SetByCode);
	
			// Post a bunch of unreliable RPCs that will overcommit more unreliable attachments than we should be able to send.
			SendRpcs(IssueRPCCount, 128, nullptr);
	
			// We send a reliable RPC last so that we can query when all data has been transmitted (or not)
			ClientObject->ServerRPC();

			// Keep on sending until we have sent and acknowledged the reliable rpc.
			UDataStream* ClientDataStream = Client->ReplicationSystem->GetDataStream(Client->LocalConnectionId, "Replication");
			while (!ClientDataStream->HasAcknowledgedAllReliableData())
			{
				// Send and deliver packet
				Client->UpdateAndSend(Server);
				Server->UpdateAndSend({Client});
			}

			// Verify RPC reception
			UE_NET_ASSERT_TRUE(ServerObject->bServerRPCCalled);

			UE_NET_ASSERT_TRUE(ServerObject->bServerUnreliableWithExportRPCCalled);

			// We expect some unreliable attachments to be delivered but not all.
			UE_NET_ASSERT_EQ(ServerObject->ServerUnreliableWithExportRPCCallCounter, ExpectedPerPacket);
		}
	}

	UE_NET_TEST_FIXTURE(FRPCTestFixture, TestOverCommitReliableRPCDirtiesObjectWhenMoreAttachmentsCanBeSent)
	{
		// Add a client
		FReplicationSystemTestClient* Client = CreateClient();

		// Spawn object on server
		UTestReplicatedObjectWithRPC* ServerObject = Server->CreateObject<UTestReplicatedObjectWithRPC>();

		ServerObject->bIsServerObject = true;
		ServerObject->ReplicationSystem = Server->GetReplicationSystem();
		Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, 0x01);

		// Send and deliver packet
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, true);
		Server->PostSendUpdate();

		UTestReplicatedObjectWithRPC* ClientObject = Cast<UTestReplicatedObjectWithRPC>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		
		// Verify that created server handle now also exists on client
		UE_NET_ASSERT_TRUE(ClientObject != nullptr);

		ClientObject->ReplicationSystem = Client->GetReplicationSystem();

		// Explicitly set ReliableRPCQueueSize to avoid issues if it is changed as what we want to test is filling the reliable sequence window.
		TGuardConsoleVariable<int32> CVarOverride(TEXT("net.ReliableRPCQueueSize"), 10000);
		
		// Issue enough RPCs to overshoot reliable send window
		const int32 RPCCount = 10000;

		for (int32 It = 0; It < RPCCount; ++It)
		{
			ServerObject->ClientRPC();
		}

		int32 PrevCallOrder = ClientObject->ClientRPCCallOrder;

		// Send a bunch of packets to fill up Reliable send queue and still have rpcs in the PreQueue
		int32 PacketFlightCount = 6;
		for (int32 PacketIt = 0; PacketIt < PacketFlightCount; ++PacketIt)
		{
			Server->NetUpdate();
			Server->SendTo(Client);
			Server->PostSendUpdate();
		}

		for (int32 PacketIt = 0; PacketIt < PacketFlightCount; ++PacketIt)
		{
			// Deliver the packets in flight
			Server->DeliverTo(Client, true);
		}

		// Verify that we delivered RPCs
		UE_NET_ASSERT_GT(ClientObject->ClientRPCCallOrder, PrevCallOrder);
		PrevCallOrder = ClientObject->ClientRPCCallOrder;

		// Send and deliver more packets
		Server->UpdateAndSend({Client}, true);

		// Confirm that we can deliver more RPCs after getting packet notifications
		UE_NET_ASSERT_GT(ClientObject->ClientRPCCallOrder, PrevCallOrder);
		PrevCallOrder = ClientObject->ClientRPCCallOrder;
	}

#if UE_AUTORTFM
	UE_NET_TEST_FIXTURE(FRPCTestFixture, AbortingRPCWithStompDectionDoesNotCrash)
	{
		IConsoleVariable* CVarStompDetectionEnabled = IConsoleManager::Get().FindConsoleVariable(TEXT("net.Iris.RPC.ServerStompDetectionEnabled"));
		UE_NET_ASSERT_NE(CVarStompDetectionEnabled, nullptr);
		UE_NET_ASSERT_TRUE(CVarStompDetectionEnabled->IsVariableBool());
		const bool OldCVarValue = CVarStompDetectionEnabled->GetBool();

		// Enable RPC quantized state stomp detection
		CVarStompDetectionEnabled->Set(true, ECVF_SetByCode);

		ON_SCOPE_EXIT
		{
			// Restore cvar
			CVarStompDetectionEnabled->Set(OldCVarValue, ECVF_SetByCode);
		};

		// Add a client
		FReplicationSystemTestClient* Client = CreateClient();

		// Spawn object on server
		UTestReplicatedObjectWithRPC* ServerObject = Server->CreateObject<UTestReplicatedObjectWithRPC>();

		ServerObject->bIsServerObject = true;
		ServerObject->ReplicationSystem = Server->GetReplicationSystem();
		Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, 0x01);

		// Send and deliver packet
		Server->UpdateAndSend({Client});

		UTestReplicatedObjectWithRPC* ClientObject = Cast<UTestReplicatedObjectWithRPC>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		
		// Verify that created server handle now also exists on client
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		auto Transaction = AutoRTFM::Transact([&]()
			{
				// Call an RPC server->client
				ServerObject->ClientRPCWithParam(1);

				AutoRTFM::AbortTransaction();
			});

		// Send and deliver packet
		Server->UpdateAndSend({Client});

		// The RPC was aborted and should not have been called
		UE_NET_ASSERT_EQ(ClientObject->ClientRPCWithParamCalled, 0);
	}
#endif
}