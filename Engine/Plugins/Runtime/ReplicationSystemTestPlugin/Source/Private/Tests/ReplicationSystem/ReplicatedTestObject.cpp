// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicatedTestObject.h"
#include "ReplicatedTestObjectFactory.h"
#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Iris/ReplicationSystem/RepTag.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationState/InternalPropertyReplicationState.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorImplementationMacros.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Core/MemoryLayoutUtil.h"
#include "Templates/SharedPointer.h"
#include "Containers/Map.h"
#include "UObject/StrongObjectPtr.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Net/UnrealNetwork.h"
#include "Iris/Serialization/NetBitStreamUtil.h"

//////////////////////////////////////////////////////////////////////////
// Implementation for UReplicatedTestObjectBridge
//////////////////////////////////////////////////////////////////////////

UReplicatedTestObjectBridge::UReplicatedTestObjectBridge()
: UObjectReplicationBridge()
{
	
}

void UReplicatedTestObjectBridge::Initialize(UReplicationSystem* InReplicationSystem)
{
	Super::Initialize(InReplicationSystem);

	ReplicatedObjectFactoryId = UE::Net::FNetObjectFactoryRegistry::GetFactoryIdFromName(UReplicatedTestObjectFactory::GetFactoryName());
	check(ReplicatedObjectFactoryId != UE::Net::InvalidNetObjectFactoryId);

	TestObjectFactory = CastChecked<UReplicatedTestObjectFactory>(GetNetFactory(ReplicatedObjectFactoryId));
}

const UE::Net::FReplicationInstanceProtocol* UReplicatedTestObjectBridge::GetReplicationInstanceProtocol(FNetRefHandle Handle) const
{
	const UE::Net::Private::FNetRefHandleManager* LocalNetRefHandleManager = &GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager();

	if (uint32 InternalObjectIndex = LocalNetRefHandleManager->GetInternalIndex(Handle))
	{
		return LocalNetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalObjectIndex).InstanceProtocol;
	}

	return nullptr;
};

UE::Net::FNetRefHandle UReplicatedTestObjectBridge::BeginReplication(UReplicatedTestObject* Instance)
{
	// Create NetRefHandle for the registered fragments
	FNetRefHandle Handle = Super::StartReplicatingRootObject(Instance, ReplicatedObjectFactoryId);

	// This is optional but typically we want to cache at least the NetRefHandle in the game instance to avoid doing map lookups to find it
	if (Handle.IsValid())
	{
		Instance->NetRefHandle = Handle;
	}

	return Handle;
}

UE::Net::FNetRefHandle UReplicatedTestObjectBridge::BeginReplication(UReplicatedTestObject* Instance, const UObjectReplicationBridge::FRootObjectReplicationParams& Params)
{
	// Create NetRefHandle for the registered fragments
	TestObjectFactory->AddRootObjectReplicationParams(Instance, Params);
	FNetRefHandle Handle = Super::StartReplicatingRootObject(Instance, ReplicatedObjectFactoryId);

	// This is optional but typically we want to cache at least the NetRefHandle in the game instance to avoid doing map lookups to find it
	Instance->NetRefHandle = Handle;

	return Handle;
}

UE::Net::FNetRefHandle UReplicatedTestObjectBridge::BeginReplication(FNetRefHandle OwnerHandle, UReplicatedTestObject* SubObjectInstance)
{
	check(OwnerHandle.IsValid());

	// Create NetRefHandle for the registered fragments
	const FSubObjectReplicationParams Params { .RootObjectHandle = OwnerHandle };
	FNetRefHandle Handle = Super::StartReplicatingSubObject(SubObjectInstance, Params, ReplicatedObjectFactoryId);

	if (Handle.IsValid())
	{
		// This is optional but typically we want to cache at least the NetRefHandle in the game instance to avoid doing map lookups to find it
		SubObjectInstance->NetRefHandle = Handle;
		SubObjectInstance->bIsSubObject = true;
	}
	
	return Handle;
}

UE::Net::FNetRefHandle UReplicatedTestObjectBridge::BeginReplication(UReplicatedTestObject* SubObjectInstance, const FSubObjectReplicationParams& Params)
{
	check(Params.RootObjectHandle.IsValid());

	// Create NetRefHandle for the registered fragments
	FNetRefHandle Handle = Super::StartReplicatingSubObject(SubObjectInstance, Params, ReplicatedObjectFactoryId);

	if (Handle.IsValid())
	{
		// This is optional but typically we want to cache at least the NetRefHandle in the game instance to avoid doing map lookups to find it
		SubObjectInstance->NetRefHandle = Handle;
		SubObjectInstance->bIsSubObject = true;
	}
	
	return Handle;	
}

void UReplicatedTestObjectBridge::EndReplication(UReplicatedTestObject* Instance, EEndReplicationFlags Flags)
{
	StopReplicatingNetObject(Instance, Flags);
}

void UReplicatedTestObjectBridge::SetExternalWorldLocationUpdateFunctor(TFunction<void(FNetRefHandle NetHandle, const UObject* ReplicatedObject, FVector& OutLocation, float& OutCullDistance)> LocUpdateFunctor)
{
	UReplicatedTestObjectFactory* TestFactory = CastChecked<UReplicatedTestObjectFactory>(GetNetFactory(ReplicatedObjectFactoryId));
	TestFactory->SetWorldUpdateFunctor(LocUpdateFunctor);
}

void UReplicatedTestObjectBridge::SetExternalPreUpdateFunctor(TFunction<void(TArrayView<UObject*>, const UObjectReplicationBridge*)> PreUpdateFunctor)
{
	SetInstancePreUpdateFunction(PreUpdateFunctor);
}

void UReplicatedTestObjectBridge::SetCreatedObjectsOnNode(TArray<TStrongObjectPtr<UObject>>* InCreatedObjectsOnNode)
{
	using namespace UE::Net;
	UNetObjectFactory* Factory = GetNetFactory(ReplicatedObjectFactoryId);
	if (ensure(Factory))
	{
		CastChecked<UReplicatedTestObjectFactory>(Factory)->SetCreatedObjectsOnNode(InCreatedObjectsOnNode);
	}
}


//////////////////////////////////////////////////////////////////////////
// Implementation for UTestReplicatedIrisPropertyComponent
//////////////////////////////////////////////////////////////////////////
UTestReplicatedIrisPropertyComponent::UTestReplicatedIrisPropertyComponent() : UObject()
{
}

void UTestReplicatedIrisPropertyComponent::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	DOREPLIFETIME(UTestReplicatedIrisPropertyComponent, IntA);
	DOREPLIFETIME(UTestReplicatedIrisPropertyComponent, StructWithStructWithTag);
	DOREPLIFETIME_CONDITION(UTestReplicatedIrisPropertyComponent, IntB, COND_InitialOnly);
}

//////////////////////////////////////////////////////////////////////////
// Implementation for UTestReplicatedIrisPushModelComponentWithObjectReference
//////////////////////////////////////////////////////////////////////////
UTestReplicatedIrisPushModelComponentWithObjectReference::UTestReplicatedIrisPushModelComponentWithObjectReference() : UObject()
{
}

void UTestReplicatedIrisPushModelComponentWithObjectReference::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	FDoRepLifetimeParams LifetimeParams;
	LifetimeParams.bIsPushBased = true;

	DOREPLIFETIME_WITH_PARAMS(ThisClass, IntA, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS(ThisClass, RawObjectPtrRef, LifetimeParams);
	DOREPLIFETIME_WITH_PARAMS(ThisClass, WeakObjectPtrObjectRef, LifetimeParams);
}

void UTestReplicatedIrisPushModelComponentWithObjectReference::ModifyIntA()
{
	IntA += 1;
	MARK_PROPERTY_DIRTY_FROM_NAME(ThisClass, IntA, this);
}

//////////////////////////////////////////////////////////////////////////
// Implementation for UTestReplicatedIrisDynamicStatePropertyComponent
//////////////////////////////////////////////////////////////////////////
UTestReplicatedIrisDynamicStatePropertyComponent::UTestReplicatedIrisDynamicStatePropertyComponent() : UObject()
{
}

void UTestReplicatedIrisDynamicStatePropertyComponent::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	DOREPLIFETIME(UTestReplicatedIrisDynamicStatePropertyComponent, IntArray);
	DOREPLIFETIME(UTestReplicatedIrisDynamicStatePropertyComponent, IntStaticArray);
}

//////////////////////////////////////////////////////////////////////////
// Implementation for UTestReplicatedIrisLifetimeConditionalsPropertyState
//////////////////////////////////////////////////////////////////////////
UTestReplicatedIrisLifetimeConditionalsPropertyState::UTestReplicatedIrisLifetimeConditionalsPropertyState() : UObject()
{
}

void UTestReplicatedIrisLifetimeConditionalsPropertyState::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	DOREPLIFETIME_CONDITION(ThisClass, ToOwnerA, COND_OwnerOnly);
	DOREPLIFETIME_CONDITION(ThisClass, ToOwnerB, COND_OwnerOnly);
	DOREPLIFETIME_CONDITION(ThisClass, ReplayOrOwner, COND_ReplayOrOwner);

	DOREPLIFETIME_CONDITION(ThisClass, SkipOwnerA, COND_SkipOwner);
	DOREPLIFETIME_CONDITION(ThisClass, SkipOwnerB, COND_SkipOwner);

	DOREPLIFETIME_CONDITION(ThisClass, SimulatedOnlyInt, COND_SimulatedOnly);
	DOREPLIFETIME_CONDITION(ThisClass, AutonomousOnlyInt, COND_AutonomousOnly);
	DOREPLIFETIME_CONDITION(ThisClass, SimulatedOrPhysicsInt, COND_SimulatedOrPhysics);
	DOREPLIFETIME_CONDITION(ThisClass, SimulatedOnlyNoReplayInt, COND_SimulatedOnlyNoReplay);
	DOREPLIFETIME_CONDITION(ThisClass, SimulatedOrPhysicsNoReplayInt, COND_SimulatedOrPhysicsNoReplay);
	DOREPLIFETIME_CONDITION(ThisClass, NoneInt, COND_None);
	DOREPLIFETIME_CONDITION(ThisClass, NeverInt, COND_Never);
	DOREPLIFETIME_CONDITION(ThisClass, SkipReplayInt, COND_SkipReplay);
	DOREPLIFETIME_CONDITION(ThisClass, ReplayOnlyInt, COND_ReplayOnly);

	DOREPLIFETIME_CONDITION(ThisClass, SimulatedOnlyIntArray, COND_SimulatedOnly);
	DOREPLIFETIME_CONDITION(ThisClass, AutonomousOnlyIntArray, COND_AutonomousOnly);
	DOREPLIFETIME_CONDITION(ThisClass, SimulatedOrPhysicsIntArray, COND_SimulatedOrPhysics);
	DOREPLIFETIME_CONDITION(ThisClass, OwnerOnlyIntArray, COND_OwnerOnly);
}

//////////////////////////////////////////////////////////////////////////
// Implementation for UTestReplicationSystem_TestIrisComponent
//////////////////////////////////////////////////////////////////////////
FTestReplicatedIrisComponent::FTestReplicatedIrisComponent()
: ReplicationFragment(*this, ReplicationState)
{
}

void FTestReplicatedIrisComponent::ApplyReplicationState(const FFakeGeneratedReplicationState& State, UE::Net::FReplicationStateApplyContext& Context)
{
	ReplicationState = State;
}

//////////////////////////////////////////////////////////////////////////
// Implementation for UTestReplicatedIrisObject
//////////////////////////////////////////////////////////////////////////
UTestReplicatedIrisObject::UTestReplicatedIrisObject()
: UReplicatedTestObject()
{
}

void UTestReplicatedIrisObject::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = false;

	Params.Condition = COND_None;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntA, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntB, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntC, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, IntDWithOnRep, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, StructD, Params);
}

void UTestReplicatedIrisObject::OnRep_IntD()
{
	bIntDHitOnRep = true;
}

void UTestReplicatedIrisObject::AddComponents(const UTestReplicatedIrisObject::FComponents& InComponents)
{
	check(!NetRefHandle.IsValid())
	for (uint32 It = 0; It < InComponents.PropertyComponentCount; ++It)
	{
		Components.Emplace(NewObject<UTestReplicatedIrisPropertyComponent>());
	}

	for (uint32 It = 0; It < InComponents.IrisComponentCount; ++It)
	{
		IrisComponents.Emplace(TUniquePtr<FTestReplicatedIrisComponent>(new FTestReplicatedIrisComponent()));
	}

	for (uint32 It = 0; It < InComponents.DynamicStateComponentCount; ++It)
	{
		DynamicStateComponents.Emplace(NewObject<UTestReplicatedIrisDynamicStatePropertyComponent>());
	}

	for (uint32 It = 0; It < InComponents.ConnectionFilteredComponentCount; ++It)
	{
		ConnectionFilteredComponents.Emplace(NewObject<UTestReplicatedIrisLifetimeConditionalsPropertyState>());
	}

	for (uint32 It = 0; It < InComponents.ObjectReferenceComponentCount; ++It)
	{
		ObjectReferenceComponents.Emplace(NewObject<UTestReplicatedIrisPushModelComponentWithObjectReference>());
	}
}

void UTestReplicatedIrisObject::AddComponents(uint32 PropertyComponentCount, uint32 IrisComponentCount)
{
	check(!NetRefHandle.IsValid())
	// Setup a few components
	for (uint32 It = 0; It < PropertyComponentCount; ++It)
	{
		Components.Emplace(NewObject<UTestReplicatedIrisPropertyComponent>());
	}

	// Setup a few components
	for (uint32 It = 0; It < IrisComponentCount; ++It)
	{
		IrisComponents.Emplace(TUniquePtr<FTestReplicatedIrisComponent>(new FTestReplicatedIrisComponent()));
	}
}

void UTestReplicatedIrisObject::AddDynamicStateComponents(uint32 DynamicStateComponentCount)
{
	check(!NetRefHandle.IsValid())
	for (uint32 It = 0; It < DynamicStateComponentCount; ++It)
	{
		DynamicStateComponents.Emplace(NewObject<UTestReplicatedIrisDynamicStatePropertyComponent>());
	}
}

void UTestReplicatedIrisObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	using namespace UE::Net;

	// Base object owns the fragment in this case
	{
		this->ReplicationFragments.Reset();
		FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags, &this->ReplicationFragments);
	}

	// Register components from "Components" as well, in this case we let the replication system own the fragments
	for (const auto& Component : Components)
	{
		Component->ReplicationFragments.Reset();
		FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(Component.Get(), Context, RegistrationFlags, &Component->ReplicationFragments);
	}	

	// Register components from "IrisComponents" as well
	for (const auto& Component : IrisComponents)
	{
		Component->ReplicationFragment.Register(Context);
	}	

	// Register components from "DynamicStateComponents"
	for (const auto& Component : DynamicStateComponents)
	{
		Component->ReplicationFragments.Reset();
		FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(Component.Get(), Context, RegistrationFlags, &Component->ReplicationFragments);
	}	

	// Register components from "ConnectionFilteredComponents"
	for (const auto& Component : ConnectionFilteredComponents)
	{
		Component->ReplicationFragments.Reset();
		FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(Component.Get(), Context, RegistrationFlags, &Component->ReplicationFragments);
	}	

	// Register components from "ObjectReferenceComponents"
	for (const auto& Component : ObjectReferenceComponents)
	{
		Component->ReplicationFragments.Reset();
		FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(Component.Get(), Context, RegistrationFlags, &Component->ReplicationFragments);
	}	
}

//////////////////////////////////////////////////////////////////////////
// Implementation for UTestReplicatedIrisObjectWithObjectReference
//////////////////////////////////////////////////////////////////////////

void UTestReplicatedIrisObjectWithObjectReference::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	DOREPLIFETIME(ThisClass, IntA);
	DOREPLIFETIME(ThisClass, IntB);
	DOREPLIFETIME(ThisClass, IntC);
	DOREPLIFETIME(ThisClass, RawObjectPtrRef);
	DOREPLIFETIME(ThisClass, WeakObjectPtrObjectRef);
	DOREPLIFETIME(ThisClass, SoftObjectPtrRef);
	DOREPLIFETIME(ThisClass, TypedRawObjectPtrRef);
	DOREPLIFETIME(ThisClass, TypedWeakObjectPtrObjectRef);
	DOREPLIFETIME(ThisClass, TypedSoftObjectPtrRef);
}

UTestReplicatedIrisObjectWithObjectReference::UTestReplicatedIrisObjectWithObjectReference()
: UReplicatedTestObject()
{
}

void UTestReplicatedIrisObjectWithObjectReference::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	// Base object owns the fragment in this case
	{
		this->ReplicationFragments.Reset();
		UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags, &this->ReplicationFragments);
	}
}

uint32 UReplicatedSubObjectOrderObject::RepOrderCounter = 0U;

//////////////////////////////////////////////////////////////////////////
// Implementation for UTestReplicatedIrisObjectWithNoReplicatedMembers
//////////////////////////////////////////////////////////////////////////
UTestReplicatedIrisObjectWithNoReplicatedMembers::UTestReplicatedIrisObjectWithNoReplicatedMembers()
: UReplicatedTestObject()
{
}

void UTestReplicatedIrisObjectWithNoReplicatedMembers::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	Fragments.SetIsFragmentlessNetObject(true);
}

//////////////////////////////////////////////////////////////////////////
// Implementation for UTestReplicatedIrisObjectWithStableSubObject
//////////////////////////////////////////////////////////////////////////
UTestReplicatedIrisObjectWithStableSubObject::UTestReplicatedIrisObjectWithStableSubObject()
: UReplicatedTestObject()
{
	DefaultSubObject = CreateDefaultSubobject<UTestReplicatedIrisObjectWithObjectReference>(TEXT("DefaultSubObject"));
}

void UTestReplicatedIrisObjectWithStableSubObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	Fragments.SetIsFragmentlessNetObject(true);
}

//////////////////////////////////////////////////////////////////////////
// Implementation for UReplicatedSubObjectOrderObject
//////////////////////////////////////////////////////////////////////////
void UReplicatedSubObjectOrderObject::GetLifetimeReplicatedProps(TArray< class FLifetimeProperty >& OutLifetimeProps) const
{
	DOREPLIFETIME(UReplicatedSubObjectOrderObject, IntA);
	DOREPLIFETIME(UReplicatedSubObjectOrderObject, OtherSubObject);
}

UReplicatedSubObjectOrderObject::UReplicatedSubObjectOrderObject()
	: UReplicatedTestObject()
{
}

void UReplicatedSubObjectOrderObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	// Base object owns the fragment in this case
	{
		this->ReplicationFragments.Reset();
		UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags, &this->ReplicationFragments);
	}
}

void UReplicatedSubObjectDestroyOrderObject::SetObjectExpectedToBeDestroyed(UReplicatedSubObjectDestroyOrderObject* OtherObject)
{
	ObjectToWatch = MakeWeakObjectPtr(OtherObject);
}

void UReplicatedSubObjectDestroyOrderObject::BeginDestroy()
{
	Super::BeginDestroy();
}

void UReplicatedSubObjectDestroyOrderObject::PreNetReceive()
{
	Super::PreNetReceive();
	
	bObjectExistedInPreNetReceive = ObjectToWatch.IsSet() && ObjectToWatch.GetValue().IsValid();
}

void UReplicatedSubObjectDestroyOrderObject::PostNetReceive()
{
	Super::PostNetReceive();
	
	bObjectExistedInPostNetReceive = ObjectToWatch.IsSet() && ObjectToWatch.GetValue().IsValid();
}

//////////////////////////////////////////////////////////////////////////
// Implementation for UTestReplicatedObjectWithRepNotifies
//////////////////////////////////////////////////////////////////////////
void UTestReplicatedObjectWithRepNotifies::GetLifetimeReplicatedProps(TArray< class FLifetimeProperty >& OutLifetimeProps) const
{
	DOREPLIFETIME_CONDITION_NOTIFY(UTestReplicatedObjectWithRepNotifies, IntA, COND_None, REPNOTIFY_Always);
	DOREPLIFETIME_CONDITION_NOTIFY(UTestReplicatedObjectWithRepNotifies, IntB, COND_None, REPNOTIFY_Always);
	DOREPLIFETIME(UTestReplicatedObjectWithRepNotifies, IntC);
}

UTestReplicatedObjectWithRepNotifies::UTestReplicatedObjectWithRepNotifies()
	: UReplicatedTestObject()
{
}

void UTestReplicatedObjectWithRepNotifies::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	// Base object owns the fragment in this case
	{
		this->ReplicationFragments.Reset();
		UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags, &this->ReplicationFragments);
	}
}

void UTestReplicatedObjectWithRepNotifies::OnRep_IntA(int32 OldInt)
{
	PrevIntAStoredInOnRep = OldInt;
}

void UTestReplicatedObjectWithRepNotifies::OnRep_IntB(int32 OldInt)
{
	PrevIntBStoredInOnRep = OldInt;
}


//////////////////////////////////////////////////////////////////////////
// Implementation for UTestReplicatedIrisPushModelObject
//////////////////////////////////////////////////////////////////////////
void UTestReplicatedIrisPushModelObject::GetLifetimeReplicatedProps(TArray< class FLifetimeProperty >& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params{ .bIsPushBased = true };
	DOREPLIFETIME_WITH_PARAMS(UTestReplicatedIrisPushModelObject, IntA, Params);
	DOREPLIFETIME_WITH_PARAMS(UTestReplicatedIrisPushModelObject, IntB, Params);

	// Push based initial only
	Params.Condition = ELifetimeCondition::COND_InitialOnly;
	DOREPLIFETIME_WITH_PARAMS(UTestReplicatedIrisPushModelObject, InitialOnlyInt, Params);

	// Push based initial or owner
	Params.Condition = ELifetimeCondition::COND_InitialOrOwner;
	DOREPLIFETIME_WITH_PARAMS(UTestReplicatedIrisPushModelObject, InitialOrOwnerInt, Params);
}

void UTestReplicatedIrisPushModelObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	// Base object owns the fragment in this case
	{
		this->ReplicationFragments.Reset();
		UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags, &this->ReplicationFragments);
	}
}

void UTestReplicatedIrisPushModelObject::SetInitialOnlyInt(int32 InValue)
{
	InitialOnlyInt = InValue;
	MARK_PROPERTY_DIRTY_FROM_NAME(UTestReplicatedIrisPushModelObject, InitialOnlyInt, this);
}

int32 UTestReplicatedIrisPushModelObject::GetInitialOnlyInt() const
{
	return InitialOnlyInt;
}

void UTestReplicatedIrisPushModelObject::SetInitialOrOwnerInt(int32 InValue)
{
	InitialOrOwnerInt = InValue;
	MARK_PROPERTY_DIRTY_FROM_NAME(UTestReplicatedIrisPushModelObject, InitialOrOwnerInt, this);
}

int32 UTestReplicatedIrisPushModelObject::GetInitialOrOwnerInt() const
{
	return InitialOrOwnerInt;
}

void UTestReplicatedIrisPushModelObject::SetIntA(int32 InValue)
{
	IntA = InValue;
	MARK_PROPERTY_DIRTY_FROM_NAME(UTestReplicatedIrisPushModelObject, IntA, this);
}

int32 UTestReplicatedIrisPushModelObject::GetIntA() const
{
	return IntA;
}

void UTestReplicatedIrisPushModelObject::SetIntB(int32 InValue)
{
	IntB = InValue;
	MARK_PROPERTY_DIRTY_FROM_NAME(UTestReplicatedIrisPushModelObject, IntB, this);
}

int32 UTestReplicatedIrisPushModelObject::GetIntB() const
{
	return IntB;
}

//////////////////////////////////////////////////////////////////////////
// Implementation for UTestReplicatedIrisObjectWithDynamicCondition
//////////////////////////////////////////////////////////////////////////
UTestReplicatedIrisObjectWithDynamicCondition::UTestReplicatedIrisObjectWithDynamicCondition()
: UReplicatedTestObject()
{
}

void UTestReplicatedIrisObjectWithDynamicCondition::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	Super::RegisterReplicationFragments(Context, RegistrationFlags);

	// Base object owns the fragment in this case
	{
		this->ReplicationFragments.Reset();
		UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags, &this->ReplicationFragments);
	}
}

void UTestReplicatedIrisObjectWithDynamicCondition::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = false;

	Params.Condition = COND_Dynamic;
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, DynamicConditionInt, Params);
}

void UTestReplicatedIrisObjectWithDynamicCondition::SetDynamicCondition(ELifetimeCondition Condition)
{
	DOREPDYNAMICCONDITION_SETCONDITION_FAST(ThisClass, DynamicConditionInt, Condition);
}

void UTestReplicatedIrisObjectWithDynamicCondition::SetDynamicConditionCustomCondition(bool bActive)
{
	DOREPCUSTOMCONDITION_SETACTIVE_FAST(ThisClass, DynamicConditionInt, bActive);	
}

//////////////////////////////////////////////////////////////////////////
// THIS SECTION WILL BE GENERATED FROM UHT
//////////////////////////////////////////////////////////////////////////

// FFakeGeneratedReplicationState
const UE::Net::FRepTag RepTag_FakeGeneratedReplicationState_IntB = UE::Net::MakeRepTag("FakeGeneratedReplicationState_IntB");

constexpr UE::Net::FReplicationStateMemberChangeMaskDescriptor FFakeGeneratedReplicationState::sReplicationStateChangeMaskDescriptors[3];

IRIS_BEGIN_SERIALIZER_DESCRIPTOR(FFakeGeneratedReplicationState)
IRIS_SERIALIZER_DESCRIPTOR(UE::Net::FInt32NetSerializer, nullptr)
IRIS_SERIALIZER_DESCRIPTOR(UE::Net::FInt32NetSerializer, nullptr)
IRIS_SERIALIZER_DESCRIPTOR(UE::Net::FInt32NetSerializer, nullptr)
IRIS_END_SERIALIZER_DESCRIPTOR()

// Member traits
IRIS_BEGIN_TRAITS_DESCRIPTOR(FFakeGeneratedReplicationState)
IRIS_TRAITS_DESCRIPTOR(UE::Net::EReplicationStateMemberTraits::None)
IRIS_TRAITS_DESCRIPTOR(UE::Net::EReplicationStateMemberTraits::None)
IRIS_TRAITS_DESCRIPTOR(UE::Net::EReplicationStateMemberTraits::None)
IRIS_END_TRAITS_DESCRIPTOR()

// This is required to work around issues with static initialization order
IRIS_BEGIN_INTERNAL_TYPE_INFO(FFakeGeneratedReplicationState)
IRIS_INTERNAL_TYPE_INFO(UE::Net::FInt32NetSerializer)
IRIS_INTERNAL_TYPE_INFO(UE::Net::FInt32NetSerializer)
IRIS_INTERNAL_TYPE_INFO(UE::Net::FInt32NetSerializer)
IRIS_END_INTERNAL_TYPE_INFO()

IRIS_BEGIN_MEMBER_DESCRIPTOR(FFakeGeneratedReplicationState)
IRIS_MEMBER_DESCRIPTOR(FFakeGeneratedReplicationState, IntA, 0)
IRIS_MEMBER_DESCRIPTOR(FFakeGeneratedReplicationState, IntB, 1)
IRIS_MEMBER_DESCRIPTOR(FFakeGeneratedReplicationState, IntC, 2)
IRIS_END_MEMBER_DESCRIPTOR()

IRIS_BEGIN_MEMBER_DEBUG_DESCRIPTOR(FFakeGeneratedReplicationState)
IRIS_MEMBER_DEBUG_DESCRIPTOR(FFakeGeneratedReplicationState, IntA)
IRIS_MEMBER_DEBUG_DESCRIPTOR(FFakeGeneratedReplicationState, IntB)
IRIS_MEMBER_DEBUG_DESCRIPTOR(FFakeGeneratedReplicationState, IntC)
IRIS_END_MEMBER_DEBUG_DESCRIPTOR()

IRIS_BEGIN_TAG_DESCRIPTOR(FFakeGeneratedReplicationState)
IRIS_TAG_DESCRIPTOR(RepTag_FakeGeneratedReplicationState_IntB, 1)
IRIS_END_TAG_DESCRIPTOR()

IRIS_BEGIN_FUNCTION_DESCRIPTOR(FFakeGeneratedReplicationState)
IRIS_END_FUNCTION_DESCRIPTOR()

IRIS_BEGIN_REFERENCE_DESCRIPTOR(FFakeGeneratedReplicationState)
IRIS_END_REFERENCE_DESCRIPTOR()

IRIS_IMPLEMENT_CONSTRUCT_AND_DESTRUCT(FFakeGeneratedReplicationState)
IRIS_IMPLEMENT_REPLICATIONSTATEDESCRIPTOR(FFakeGeneratedReplicationState)
