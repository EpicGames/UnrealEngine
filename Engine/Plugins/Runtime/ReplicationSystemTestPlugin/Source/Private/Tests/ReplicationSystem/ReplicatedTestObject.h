// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkAutomationTest.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StrongObjectPtr.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"
#include "Iris/ReplicationSystem/PropertyReplicationFragment.h"
#include "Iris/ReplicationSystem/TypedReplicationFragment.h"
#include "Templates/UniquePtr.h"
#include "Net/Core/NetBitArray.h"
#include "Net/Core/PushModel/PushModelMacros.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorMacros.h"
#include "ReplicatedTestObject.generated.h"

class UReplicatedTestObjectFactory;

namespace UE::Net
{
	struct FReplicationInstanceProtocol;
}

/*
* Object used for test purposes
* It has a virtual methods for registering all replication fragments, NetHandle is cached in order to avoid having to do lookups into networking system 
*/
UCLASS()
class UReplicatedTestObject : public UObject
{
	GENERATED_BODY()
	REPLICATED_BASE_CLASS(UReplicatedTestObject)

public:
	// Register the fragments for this object
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override {}
	virtual bool IsSupportedForNetworking() const override { return true; }
	virtual bool IsNameStableForNetworking() const override { return Super::IsNameStableForNetworking() || bForceStablyNamed; }
	virtual bool IsFullNameStableForNetworking() const override { return Super::IsFullNameStableForNetworking() || bForceStablyNamed; }

	virtual void OnSubObjectCreated(UObject* SubObject) {}
	virtual void OnSubObjectDestroyed(UObject* SubObject) {}

	/** Cached NetRefHandle to simplify testing. DO NOT use this for multi-system tests. */
	UE::Net::FNetRefHandle NetRefHandle;

	/** If this is set to true, this instance will fail to instantiate on remote end. */
	bool bForceFailToInstantiateOnRemote = false;

	// To determine if this object is a root object or a subobject
	bool bIsSubObject = false;

	// Force the object to be treated as stably named
	bool bForceStablyNamed = false;
};

USTRUCT()
struct FTestReplicatedIrisPropertyComponentStructWithTag
{
	GENERATED_BODY()

public:
	UPROPERTY(transient)
	FVector NetTest_TestVector; // Expected to generate a NetTest_TestVector tag
};

USTRUCT()
struct FTestReplicatedIrisPropertyComponentStruct
{
	GENERATED_BODY()

public:
	UPROPERTY(transient)
	FTestReplicatedIrisPropertyComponentStructWithTag StructWithTag;
};

/**
 *  Property based test component
 */
UCLASS()
class UTestReplicatedIrisPropertyComponent : public UObject
{
	GENERATED_BODY()
public:
	UTestReplicatedIrisPropertyComponent();

	UPROPERTY(Transient, Replicated)
	int32 IntA;

	UPROPERTY(Transient, Replicated)
	FTestReplicatedIrisPropertyComponentStruct StructWithStructWithTag;

	UPROPERTY(Transient, ReplicatedUsing=OnRep_IntB)
	int32 IntB;

	virtual void PreNetReceive() override { ++CallCounts.PreNetReceiveCounter; }
	virtual void PostNetReceive() override { ++CallCounts.PostNetReceiveCounter; }

	struct FCallCounts
	{
		uint32 PreNetReceiveCounter;
		uint32 PostNetReceiveCounter;
		uint32 IntBRepNotifyCounter;
	};
	FCallCounts CallCounts = {0};

	// Network data only for test
	TArray<UE::Net::FReplicationFragment*> ReplicationFragments;

private:
	UFUNCTION()
	void OnRep_IntB() { ++CallCounts.IntBRepNotifyCounter; }
};

UCLASS()
class UTestReplicatedIrisPushModelComponentWithObjectReference : public UObject
{
	GENERATED_BODY()
public:
	UTestReplicatedIrisPushModelComponentWithObjectReference();

	void ModifyIntA();

	UPROPERTY(Transient, Replicated)
	int32 IntA;

	UPROPERTY(Transient, Replicated)
	TObjectPtr<UObject> RawObjectPtrRef;

	UPROPERTY(Transient, Replicated)
	TWeakObjectPtr<UObject> WeakObjectPtrObjectRef;

	// Network data only for test
	TArray<UE::Net::FReplicationFragment*> ReplicationFragments;
};

/**
 *  Property based test component with dynamic state
 */
UCLASS()
class UTestReplicatedIrisDynamicStatePropertyComponent : public UObject
{
	GENERATED_BODY()
public:
	UTestReplicatedIrisDynamicStatePropertyComponent();

	UPROPERTY(Transient, ReplicatedUsing=OnRep_IntArray)
	TArray<int32> IntArray;

	UPROPERTY(Transient, ReplicatedUsing=OnRep_IntStaticArray)
	int8 IntStaticArray[7];
	int8 Sentinel = 0x71;

	struct FCallCounts
	{
		uint32 IntArrayRepNotifyCounter;
		uint32 IntStaticArrayRepNotifyCounter;
	};
	FCallCounts CallCounts = {};

	// Network data only for test
	TArray<UE::Net::FReplicationFragment*> ReplicationFragments;

private:
	UFUNCTION()
	void OnRep_IntArray() { ++CallCounts.IntArrayRepNotifyCounter; }

	UFUNCTION()
	void OnRep_IntStaticArray() { ++CallCounts.IntStaticArrayRepNotifyCounter; }
};

/**
* Fake a generated ReplicationState
* This will be generated by either UHT or some other code generation solution
* A Declaration for the state below could look something like this?
*/
#if 0
ReplicationState FakeGeneratedReplicationState
{
	[NetSerializer(FInt32NetSerializer)]
	int32 IntA;

	[NetSerializer(FInt32NetSerializer)]
	int32 IntB;

	[NetSerializer(FInt32NetSerializer)]
	int32 IntC;
};
#endif

#define IRIS_GENERATED_SECTION_FOR_FFakeGeneratedReplicationState() \
private: \
	/* state mask bits definition must be in header so that we can inline the dirty checks methods */ \
	static constexpr UE::Net::FReplicationStateMemberChangeMaskDescriptor sReplicationStateChangeMaskDescriptors[3] = { {0,1}, {1,1}, {2,1} }; \
	/* $IRIS: Generate from UHT Private generated members */ \
	IRIS_DECLARE_COMMON(); \
\
/* Accessors */ \
public: \
	IRIS_ACCESS_BY_VALUE(IntA, int32, 0); \
	IRIS_ACCESS_BY_VALUE(IntB, int32, 1); \
	IRIS_ACCESS_BY_VALUE(IntC, int32, 2); 

class FFakeGeneratedReplicationState 
{
	IRIS_GENERATED_SECTION_FOR_FFakeGeneratedReplicationState();

private:	
	// Member List
	int32 IntA = 0;
	int32 IntB = 0;
	int32 IntC = 0;
};

/**
 *  Test component using Iris style replication
 */
class FTestReplicatedIrisComponent
{
public:
	FTestReplicatedIrisComponent();

	// This is the network interface
	void ApplyReplicationState(const FFakeGeneratedReplicationState& State, UE::Net::FReplicationStateApplyContext& Context);

	// The Replication state
	FFakeGeneratedReplicationState ReplicationState;
	UE::Net::TReplicationFragment<FTestReplicatedIrisComponent, FFakeGeneratedReplicationState> ReplicationFragment;
};

// LifetimeConditionals state
UCLASS()
class UTestReplicatedIrisLifetimeConditionalsPropertyState : public UObject
{
	GENERATED_BODY()
public:
	UTestReplicatedIrisLifetimeConditionalsPropertyState();

	// Member List
	UPROPERTY(Replicated, Transient)
	int32 ToOwnerA = 0;
	UPROPERTY(Replicated, Transient)
	int32 ToOwnerB = 0;
	UPROPERTY(Replicated, Transient)
	int32 ReplayOrOwner = 0;

	UPROPERTY(Replicated, Transient)
	int32 SkipOwnerA = 0;
	UPROPERTY(Replicated, Transient)
	int32 SkipOwnerB = 0;

	UPROPERTY(Replicated, Transient)
	int32 SimulatedOnlyInt = 0;
	UPROPERTY(Replicated, Transient)
	int32 AutonomousOnlyInt = 0;
	UPROPERTY(Replicated, Transient)
	int32 SimulatedOrPhysicsInt = 0;
	UPROPERTY(Replicated, Transient)
	int32 SimulatedOnlyNoReplayInt = 0;
	UPROPERTY(Replicated, Transient)
	int32 SimulatedOrPhysicsNoReplayInt = 0;
	UPROPERTY(Replicated, Transient)
	int32 NoneInt = 0;
	UPROPERTY(Replicated, Transient)
	int32 NeverInt = 0;
	UPROPERTY(Replicated, Transient)
	int32 SkipReplayInt = 0;
	UPROPERTY(Replicated, Transient)
	int32 ReplayOnlyInt = 0;

	// Arrays
	UPROPERTY(Replicated, Transient)
	TArray<int32> SimulatedOnlyIntArray;
	UPROPERTY(Replicated, Transient)
	TArray<int32> AutonomousOnlyIntArray;
	UPROPERTY(Replicated, Transient)
	TArray<int32> SimulatedOrPhysicsIntArray;
	UPROPERTY(Replicated, Transient)
	TArray<int32> OwnerOnlyIntArray;
 
	// Network data only for test
	TArray<UE::Net::FReplicationFragment*> ReplicationFragments;
};



/**
 *  A test class for Replication that itself uses Property based replication but also has "components" that uses a mix of property based replication and native ReplicationStates
 */
USTRUCT()
struct FTestReplicatedIrisObject_Struct
{
	GENERATED_BODY()

	UPROPERTY(NotReplicated)
	int32 NotReplicatedIntA = 0;

	UPROPERTY()
	int32 IntB = -1;
};

UCLASS()
class UTestReplicatedIrisObject : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	struct FComponents
	{
		uint32 PropertyComponentCount = 0;
		uint32 IrisComponentCount = 0;
		uint32 DynamicStateComponentCount = 0;
		uint32 ConnectionFilteredComponentCount = 0;
		uint32 ObjectReferenceComponentCount = 0;
	};

	UTestReplicatedIrisObject();

	// Prefer passing FComponents over the other component adding methods.
	void AddComponents(const FComponents& Components);

	// Network interface must be part of base.
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	// Deprecated
	void AddComponents(uint32 PropertyComponentCount, uint32 IrisComponentCount);
	void AddDynamicStateComponents(uint32 DynamicStateComponentCount);

public:
	UPROPERTY(Transient, Replicated)
	int32 IntA;

	UPROPERTY(Transient, Replicated)
	int32 IntB;

	UPROPERTY(Transient, Replicated)
	int8 IntC;

	UPROPERTY(Transient, ReplicatedUsing=OnRep_IntD)
	int32 IntDWithOnRep;
	bool bIntDHitOnRep = false;

	UPROPERTY(Transient, Replicated)
	FTestReplicatedIrisObject_Struct StructD;

protected:
	UFUNCTION()
	void OnRep_IntD();

public:
	TArray<TStrongObjectPtr<UTestReplicatedIrisPropertyComponent>> Components;
	TArray<TUniquePtr<FTestReplicatedIrisComponent>> IrisComponents;
	TArray<TStrongObjectPtr<UTestReplicatedIrisDynamicStatePropertyComponent>> DynamicStateComponents;
	TArray<TStrongObjectPtr<UTestReplicatedIrisLifetimeConditionalsPropertyState>> ConnectionFilteredComponents;
	TArray<TStrongObjectPtr<UTestReplicatedIrisPushModelComponentWithObjectReference>> ObjectReferenceComponents;

	// Network data only for test
	TArray<UE::Net::FReplicationFragment*> ReplicationFragments;
};

/**
 *  A test class for Replication that itself uses Property based replication but also has "components" that uses a mix of property based replication and native ReplicationStates
 */
UCLASS()
class UTestReplicatedIrisObjectWithObjectReference : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UTestReplicatedIrisObjectWithObjectReference();

	// Network interface must be part of base.
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

public:
	UPROPERTY(Transient, Replicated)
	int32 IntA;

	UPROPERTY(Transient, Replicated)
	int32 IntB;

	UPROPERTY(Transient, Replicated)
	int8 IntC;

	UPROPERTY(Transient, Replicated)
	TObjectPtr<UObject> RawObjectPtrRef;

	UPROPERTY(Transient, Replicated)
	TWeakObjectPtr<UObject> WeakObjectPtrObjectRef;

	UPROPERTY(Transient, Replicated)
	TSoftObjectPtr<UObject> SoftObjectPtrRef;

	UPROPERTY(Transient, Replicated)
	TObjectPtr<UTestReplicatedIrisObjectWithObjectReference> TypedRawObjectPtrRef;

	UPROPERTY(Transient, Replicated)
	TWeakObjectPtr<UTestReplicatedIrisObjectWithObjectReference> TypedWeakObjectPtrObjectRef;

	UPROPERTY(Transient, Replicated)
	TSoftObjectPtr<UTestReplicatedIrisObjectWithObjectReference> TypedSoftObjectPtrRef;

public:
	// Network data only for test
	TArray<UE::Net::FReplicationFragment*> ReplicationFragments;
};

/**
 *  A test class for Replication that itself uses Property based replication but also has "components" that uses a mix of property based replication and native ReplicationStates
 */
UCLASS()
class UTestReplicatedIrisObjectWithDynamicCondition : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UTestReplicatedIrisObjectWithDynamicCondition();

	// Sets the condition for the DynamicConditionInt member.
	void SetDynamicCondition(ELifetimeCondition Condition);

	// Sets custom condition on DynamicConditionInt member.
	void SetDynamicConditionCustomCondition(bool bActive);

public:
	UPROPERTY(Transient, Replicated)
	int32 DynamicConditionInt;

private:
	// Network interface must be part of base.
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

public:
	// Network data only for test
	TArray<UE::Net::FReplicationFragment*> ReplicationFragments;
};

UCLASS()
class UReplicatedSubObjectOrderObject : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UReplicatedSubObjectOrderObject();

	// Network interface must be part of base.
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

public:
	UPROPERTY(Transient, Replicated)
	int32 IntA;

	UPROPERTY(Transient, Replicated)
	TObjectPtr<UObject> OtherSubObject;

	uint32 LastRepOrderCounter = 0U;

	virtual void PostNetReceive() override 
	{ 
		LastRepOrderCounter = ++RepOrderCounter;
	}

	int32 CreationOrder = INDEX_NONE;

public:
	static uint32 RepOrderCounter;

	// Network data only for test
	TArray<UE::Net::FReplicationFragment*> ReplicationFragments;
};

UCLASS()
class UReplicatedObjectTestSubObjectCreationOrder : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UReplicatedObjectTestSubObjectCreationOrder() {}

	// Network interface must be part of base.
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) {};

public:
	virtual void OnSubObjectCreated(UObject* SubObject) override
	{
		if (UReplicatedSubObjectOrderObject* SubObjectOrder = Cast<UReplicatedSubObjectOrderObject>(SubObject))
		{
			SubObjectOrder->CreationOrder = ++CurrentCreationOrder;
		}
	}

	int32 CurrentCreationOrder = 0;
};

UCLASS()
class UReplicatedSubObjectDestroyOrderObject : public UReplicatedSubObjectOrderObject
{
	GENERATED_BODY()
public:

public:
	void SetObjectExpectedToBeDestroyed(UReplicatedSubObjectDestroyOrderObject* OtherObject);

	virtual void BeginDestroy() override;

	virtual void PreNetReceive() override;

	virtual void PostNetReceive() override;

	TOptional<TWeakObjectPtr<UReplicatedSubObjectDestroyOrderObject>> ObjectToWatch;

	bool bObjectExistedInPreNetReceive = false;
	bool bObjectExistedInPostNetReceive = false;
};


UCLASS()
class UTestReplicatedObjectWithRepNotifies : public UReplicatedTestObject
{
	GENERATED_BODY()

public:

	UTestReplicatedObjectWithRepNotifies();

	// Network interface must be part of base.
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

public:

	UPROPERTY(Transient, ReplicatedUsing=OnRep_IntA)
	int32 IntA = -1;
	int32 PrevIntAStoredInOnRep = -1;

	UPROPERTY(Transient, ReplicatedUsing=OnRep_IntB)
	int32 IntB = -1;
	int32 PrevIntBStoredInOnRep = -1;

	UPROPERTY(Transient, Replicated)
	int8 IntC = -1;

public:
	UFUNCTION()
	void OnRep_IntA(int32 OldInt);

	UFUNCTION()
	void OnRep_IntB(int32 OldInt);

	// Network data only for test
	TArray<UE::Net::FReplicationFragment*> ReplicationFragments;
};



/**
 *  A test class for Replication on an object with no replicated members
 */
UCLASS()
class UTestReplicatedIrisObjectWithNoReplicatedMembers : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UTestReplicatedIrisObjectWithNoReplicatedMembers();

	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;
};

/**
 *  A test class for replication of static subobjects
 */
UCLASS()
class UTestReplicatedIrisObjectWithStableSubObject : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UTestReplicatedIrisObjectWithStableSubObject();

	// Network interface must be part of base.
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	UPROPERTY(Transient)
	TObjectPtr<UReplicatedTestObject> DefaultSubObject;

	// Network data only for test
	TArray<UE::Net::FReplicationFragment*> ReplicationFragments;
};

/**
 * Replicated object with PushModel properties
 */
UCLASS()
class UTestReplicatedIrisPushModelObject : public UReplicatedTestObject
{
	GENERATED_BODY()

public:

	// Network interface must be part of base.
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	void SetIntA(int32 InValue);
	int32 GetIntA() const;

	void SetIntB(int32 InValue);
	int32 GetIntB() const;

	void SetInitialOnlyInt(int32 InValue);
	int32 GetInitialOnlyInt() const;

	void SetInitialOrOwnerInt(int32 InValue);
	int32 GetInitialOrOwnerInt() const;

private:
	UPROPERTY(Transient, Replicated)
	int32 InitialOnlyInt;

	UPROPERTY(Transient, Replicated)
	int32 InitialOrOwnerInt;
	
	UPROPERTY(Transient, Replicated)
	int32 IntA;

	UPROPERTY(Transient, Replicated)
	int32 IntB;

public:
	// Network data only for test
	TArray<UE::Net::FReplicationFragment*> ReplicationFragments;
};

/**
* Example of type specific ReplicationBridge, it currently only works with UTestReplicationSystem_TestClass
*/
UCLASS()
class UReplicatedTestObjectBridge : public UObjectReplicationBridge
{
	GENERATED_BODY()

public:
	UReplicatedTestObjectBridge();

	virtual void Initialize(UReplicationSystem* InReplicationSystem) override;
	
	void SetCreatedObjectsOnNode(TArray<TStrongObjectPtr<UObject>>* InCreatedObjectsOnNode);

	// This is the local Interface, it is up to each bridge implementation to define the interface for that type
	// In this example we have methods that directly uses UReplicatedTestObject;
	FNetRefHandle BeginReplication(UReplicatedTestObject* Instance);
	FNetRefHandle BeginReplication(UReplicatedTestObject* Instance, const UObjectReplicationBridge::FRootObjectReplicationParams& Params);
	FNetRefHandle BeginReplication(FNetRefHandle OwnerHandle, UReplicatedTestObject* Instance);
	FNetRefHandle BeginReplication(UReplicatedTestObject* SubObjectInstance, const FSubObjectReplicationParams& Params);

	void EndReplication(UReplicatedTestObject* Instance, EEndReplicationFlags Flags=EEndReplicationFlags::Destroy);

	// For testing we expose some things that normally are not accessible
	const UE::Net::FReplicationInstanceProtocol* GetReplicationInstanceProtocol(FNetRefHandle Handle) const;

	void SetExternalWorldLocationUpdateFunctor(TFunction<void(FNetRefHandle NetHandle, const UObject* ReplicatedObject, FVector& OutLocation, float& OutCullDistance)> LocUpdateFunctor);

	void SetExternalPreUpdateFunctor(TFunction<void(TArrayView<UObject*>, const UObjectReplicationBridge*)> PreUpdateFunctor);

	float GetMaxTickRate() const { return Super::GetMaxTickRate(); }

	// Scope to suppress ensure when testing instantiation failure on clients
	class FSuppressCreateInstanceFailedEnsureScope
	{
	public:
		explicit FSuppressCreateInstanceFailedEnsureScope(UReplicatedTestObjectBridge& BridgeIn) : Bridge(BridgeIn), bSuppressCreateInstanceFailedEnsure(Bridge.bSuppressCreateInstanceFailedEnsure)
		{
			Bridge.bSuppressCreateInstanceFailedEnsure = true;
		}
		~FSuppressCreateInstanceFailedEnsureScope()
		{
			// Restore
			Bridge.bSuppressCreateInstanceFailedEnsure = bSuppressCreateInstanceFailedEnsure;			
		}
			
	private:
		UReplicatedTestObjectBridge& Bridge;
		bool bSuppressCreateInstanceFailedEnsure;
	};

protected:

	friend FSuppressCreateInstanceFailedEnsureScope;

	TFunction<void(FNetRefHandle NetHandle, const UObject* ReplicatedObject, FVector& OutLocation, float& OutCullDistance)> WorldLocationUpdateFunc;
	bool bForceFailCreateRemoteInstance = false;

	UE::Net::FNetObjectFactoryId ReplicatedObjectFactoryId = UE::Net::InvalidNetObjectFactoryId;
	UReplicatedTestObjectFactory* TestObjectFactory = nullptr;
};

extern const UE::Net::FRepTag RepTag_FakeGeneratedReplicationState_IntB;
