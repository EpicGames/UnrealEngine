// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicatedTestObjectFactory.h"
#include "ReplicatedTestObject.h"

#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/ReplicationSystem/NetObjectFactory.h"
#include "Iris/Serialization/ObjectNetSerializer.h"

namespace UE::Net
{

//-------------------------------------------------------------------------
// FReplicationTestCreationHeader
//------------------------------------------------------------------------

FString FReplicationTestCreationHeader::ToString() const
{
	if (bIsStable)
	{
		return FString::Printf(TEXT("FReplicationTestCreationHeader (ProtocolId:0x%x)\n\t"
									"StableName=%s"
									"bForceFailCreationRemoteInstance=%u"),
									GetProtocolId(),
									*ArchetypeOrStableName,
									bForceFailCreateRemoteInstance);
	}
	else
	{
		return FString::Printf(TEXT("FReplicationTestCreationHeader (ProtocolId:0x%x)\n\t"
									"ArchetypeName=%s"
									"OuterReference=%s"
									"NumComponentsToSpawn=%u"
									"NumIrisComponentsToSpawn=%u"
									"NumDynamicComponentsToSpawn=%u"
									"NumConnectionFilteredComponentsToSpawn=%u"
									"NumObjectReferenceComponentsToSpawn=%u"
									"bForceFailCreationRemoteInstance=%u"),
									GetProtocolId(),
									*ArchetypeOrStableName,
									*DynamicObjectOuterReference.ToString(),
									NumComponentsToSpawn,
									NumIrisComponentsToSpawn,
									NumDynamicComponentsToSpawn,
									NumConnectionFilteredComponentsToSpawn,
									NumObjectReferenceComponentsToSpawn,
									bForceFailCreateRemoteInstance);
	}
}

bool FReplicationTestCreationHeader::Serialize(const FCreationHeaderContext& Context) const
{
	FNetBitStreamWriter& Writer = *Context.Serialization.GetBitStreamWriter();

	WriteString(&Writer, ArchetypeOrStableName);
	Writer.WriteBool(bForceFailCreateRemoteInstance);
	if (!Writer.WriteBool(bIsStable))
	{
		WriteFullNetObjectReference(Context.Serialization, DynamicObjectOuterReference);
		Writer.WriteBits(NumComponentsToSpawn, 16);
		Writer.WriteBits(NumIrisComponentsToSpawn, 16);
		Writer.WriteBits(NumDynamicComponentsToSpawn, 16);
		Writer.WriteBits(NumConnectionFilteredComponentsToSpawn, 16);
		Writer.WriteBits(NumObjectReferenceComponentsToSpawn, 16);
	}
	
	return !Writer.IsOverflown();
}

bool FReplicationTestCreationHeader::Deserialize(const FCreationHeaderContext& Context)
{
	FNetBitStreamReader& Reader = *Context.Serialization.GetBitStreamReader();

	ReadString(&Reader, this->ArchetypeOrStableName);
	this->bForceFailCreateRemoteInstance = Reader.ReadBool();
	this->bIsStable = Reader.ReadBool();
	if (!this->bIsStable)
	{
		ReadFullNetObjectReference(Context.Serialization, DynamicObjectOuterReference);
		this->NumComponentsToSpawn = Reader.ReadBits(16);
		this->NumIrisComponentsToSpawn = Reader.ReadBits(16);
		this->NumDynamicComponentsToSpawn = Reader.ReadBits(16);
		this->NumConnectionFilteredComponentsToSpawn = Reader.ReadBits(16);
		this->NumObjectReferenceComponentsToSpawn = Reader.ReadBits(16);
	}

	return !Reader.IsOverflown();
}

} // end namespace UE::Net

//------------------------------------------------------------------------
// UReplicatedTestObjectFactory
//------------------------------------------------------------------------

FName UReplicatedTestObjectFactory::GetFactoryName()
{
	return TEXT("TestObjectFactory");
}

TUniquePtr<UE::Net::FNetObjectCreationHeader> UReplicatedTestObjectFactory::CreateAndFillHeader(UE::Net::FNetRefHandle Handle)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const UObject* Object = Bridge->GetReplicatedObject(Handle);
	if (!ensure(Object))
	{
		return nullptr;
	}

	if (const bool bIsStable = Handle.IsStatic())
	{
		TUniquePtr<FReplicationTestCreationHeader> HeaderRef(new FReplicationTestCreationHeader);
		FReplicationTestCreationHeader* Header = HeaderRef.Get();

		Header->bIsStable = true;

		Header->ArchetypeOrStableName = Object->GetPathName(Bridge);

		if (const UReplicatedTestObject* ReplicatedTestObject = Cast<UReplicatedTestObject>(Object))
		{
			Header->bForceFailCreateRemoteInstance = ReplicatedTestObject->bForceFailToInstantiateOnRemote;
		}

		return HeaderRef;
	}

	UObject* Archetype = Object->GetArchetype();
	if (!Archetype)
	{
		check(Archetype);
		return nullptr;
	}

	TUniquePtr<FReplicationTestCreationHeader> HeaderRef(new FReplicationTestCreationHeader);
	FReplicationTestCreationHeader* Header = HeaderRef.Get();

	if (const UReplicatedTestObject* ReplicatedTestObject = Cast<UReplicatedTestObject>(Object))
	{
		Header->bForceFailCreateRemoteInstance = ReplicatedTestObject->bForceFailToInstantiateOnRemote;
	}

	if (const UTestReplicatedIrisObject* TestReplicatedIrisObject = Cast<UTestReplicatedIrisObject>(Object))
	{
		Header->NumComponentsToSpawn = IntCastChecked<uint16>(TestReplicatedIrisObject->Components.Num());
		Header->NumIrisComponentsToSpawn = IntCastChecked<uint16>(TestReplicatedIrisObject->IrisComponents.Num());
		Header->NumDynamicComponentsToSpawn = IntCastChecked<uint16>(TestReplicatedIrisObject->DynamicStateComponents.Num());
		Header->NumConnectionFilteredComponentsToSpawn = IntCastChecked<uint16>(TestReplicatedIrisObject->ConnectionFilteredComponents.Num());
		Header->NumObjectReferenceComponentsToSpawn = IntCastChecked<uint16>(TestReplicatedIrisObject->ObjectReferenceComponents.Num());
	}

	Header->ArchetypeOrStableName = Archetype->GetPathName();

	if (const UObject* ObjectOuter = Object->GetOuter())
	{
		Header->DynamicObjectOuterReference = Bridge->GetOrCreateObjectReference(ObjectOuter);
	}

	return HeaderRef;
}

bool UReplicatedTestObjectFactory::SerializeHeader(const UE::Net::FCreationHeaderContext& Context, const UE::Net::FNetObjectCreationHeader* Header)
{
	const UE::Net::FReplicationTestCreationHeader* TestHeader = static_cast<const UE::Net::FReplicationTestCreationHeader*>(Header);
	return TestHeader->Serialize(Context);
}

TUniquePtr<UE::Net::FNetObjectCreationHeader> UReplicatedTestObjectFactory::CreateAndDeserializeHeader(const UE::Net::FCreationHeaderContext& Context)
{
	TUniquePtr<UE::Net::FReplicationTestCreationHeader> Header(new UE::Net::FReplicationTestCreationHeader);
	Header->Deserialize(Context);
	return Header;
}

UNetObjectFactory::FInstantiateResult UReplicatedTestObjectFactory::InstantiateReplicatedObjectFromHeader(const FInstantiateContext& Context, const UE::Net::FNetObjectCreationHeader* InHeader)
{
	using namespace UE::Net;

	const FReplicationTestCreationHeader* Header = static_cast<const FReplicationTestCreationHeader*>(InHeader);

	// Force fail to create this remote instance
	if (Header->bForceFailCreateRemoteInstance)
	{
		return FInstantiateResult();
	}

	if (Header->bIsStable)
	{
		UObject* Object = FindObjectFast<UReplicatedTestObject>(Bridge, FName(Header->ArchetypeOrStableName));

		const bool bIsSubObject = Context.RootObjectOfSubObject.IsValid();
		if (UReplicatedTestObject* BaseTestObject = Cast<UReplicatedTestObject>(Object))
		{
			BaseTestObject->bIsSubObject = bIsSubObject;
		}

		FInstantiateResult InstantiateResult;
		InstantiateResult.Instance = Object;

		//TBD: allow for now
		InstantiateResult.Flags |= EReplicationBridgeCreateNetRefHandleResultFlags::AllowDestroyInstanceFromRemote;
		InstantiateResult.Flags |= EReplicationBridgeCreateNetRefHandleResultFlags::BindStaticObjectInReferenceCache;
		if (bIsSubObject)
		{
			InstantiateResult.Flags |= EReplicationBridgeCreateNetRefHandleResultFlags::ShouldCallSubObjectCreatedFromReplication;
		}
		return InstantiateResult;
	}
	else
	{
		UObject* ArcheType = StaticFindObject(UObject::StaticClass(), nullptr, *Header->ArchetypeOrStableName, EFindObjectFlags::None);
		check(ArcheType);

		FStaticConstructObjectParameters ConstructObjectParameters(ArcheType->GetClass());
		ConstructObjectParameters.Outer = Bridge->ResolveObjectReference(Header->DynamicObjectOuterReference, Context.ResolveContext);
		UObject* CreatedObject = StaticConstructObject_Internal(ConstructObjectParameters);

		const bool bIsSubObject = Context.RootObjectOfSubObject.IsValid();

		if (UReplicatedTestObject* BaseTestObject = Cast<UReplicatedTestObject>(CreatedObject))
		{
			BaseTestObject->bIsSubObject = bIsSubObject;
		}
	
		if (UTestReplicatedIrisObject* CreatedTestObject = Cast<UTestReplicatedIrisObject>(CreatedObject))
		{
			UTestReplicatedIrisObject::FComponents Components;
			Components.PropertyComponentCount = Header->NumComponentsToSpawn;
			Components.IrisComponentCount = Header->NumIrisComponentsToSpawn;
			Components.DynamicStateComponentCount = Header->NumDynamicComponentsToSpawn;
			Components.ConnectionFilteredComponentCount = Header->NumConnectionFilteredComponentsToSpawn;
			Components.ObjectReferenceComponentCount = Header->NumObjectReferenceComponentsToSpawn;

			CreatedTestObject->AddComponents(Components);
		}

		UReplicatedTestObjectBridge* TestBridge = Cast<UReplicatedTestObjectBridge>(Bridge);

		// Store the object so that we can find detached/torn off instances from tests
		if (CreatedObjectsOnNode)
		{
			CreatedObjectsOnNode->Add(TStrongObjectPtr<UObject>(CreatedObject));
		}
	
		FInstantiateResult InstantiateResult;
		InstantiateResult.Instance = CreatedObject;
		InstantiateResult.Flags |= EReplicationBridgeCreateNetRefHandleResultFlags::AllowDestroyInstanceFromRemote;

		if (bIsSubObject)
		{
			InstantiateResult.Flags |= EReplicationBridgeCreateNetRefHandleResultFlags::ShouldCallSubObjectCreatedFromReplication;
		}
		return InstantiateResult;
	}
}

void UReplicatedTestObjectFactory::PostInit(const FPostInitContext& Context)
{
	UReplicatedTestObject* Instance = CastChecked<UReplicatedTestObject>(Context.Instance);

	Instance->NetRefHandle = Context.Handle;
}

void UReplicatedTestObjectFactory::DetachedFromReplication(const FDetachContext& Context, const TOptional<FSubObjectDetachContext>& SubObjectContext)
{
	using namespace UE::Net;

	if (Context.DetachedInstance == nullptr)
	{
		return;
	}

	if (Context.Reason == EDetachReason::TornOff)
	{
		return;
	}

	// Always destroy dynamic objects and stable objects only when static destroyed
	const bool bDestroyInstance = Context.Reason == EDetachReason::StaticDestroyed || !Context.DetachedInstance->IsNameStableForNetworking();

	if (bDestroyInstance)
	{
		// Remove the object from the created objects on the node
		if (CreatedObjectsOnNode)
		{
			CreatedObjectsOnNode->Remove(TStrongObjectPtr<UObject>(Context.DetachedInstance));
		}

		Context.DetachedInstance->PreDestroyFromReplication();
		Context.DetachedInstance->MarkAsGarbage();
	}
}

TOptional<UNetObjectFactory::FWorldInfoData> UReplicatedTestObjectFactory::GetWorldInfo(const FWorldInfoContext& Context) const
{
	if (GetInstanceWorldObjectInfoFunction)
	{
		FWorldInfoData OutData;
		GetInstanceWorldObjectInfoFunction(Context.Handle, Context.Instance, OutData.WorldLocation, OutData.CullDistance);
		return OutData;
	}

	return NullOpt;
}

void UReplicatedTestObjectFactory::SubObjectCreatedFromReplication(UE::Net::FNetRefHandle RootObject, UE::Net::FNetRefHandle SubObjectCreated)
{
	UReplicatedTestObject* RootInstance = Cast<UReplicatedTestObject>(Bridge->GetReplicatedObject(RootObject));
	if (ensure(RootInstance))
	{
		UObject* SubObject = Bridge->GetReplicatedObject(SubObjectCreated);
		RootInstance->OnSubObjectCreated(SubObject);
	}
}

void UReplicatedTestObjectFactory::SubObjectDetachedFromReplication(const FDetachContext& Context, const FSubObjectDetachContext& SubObjectContext)
{
	UReplicatedTestObject* RootInstance = Cast<UReplicatedTestObject>(SubObjectContext.RootObject);
	if (RootInstance)
	{
		RootInstance->OnSubObjectDestroyed(Context.DetachedInstance);
	}
}

void UReplicatedTestObjectFactory::FillRootObjectReplicationParams(const UE::Net::FRootObjectReplicationParamsContext& Context, UE::Net::FRootObjectReplicationParams& OutParams)
{
	if (const UE::Net::FRootObjectReplicationParams* Params = InstanceReplicationParams.Find(FWeakObjectPtr(Context.Object)))
	{
		OutParams = *Params;
	}
	// else leave OutParams alone, presumably with default values
}

void UReplicatedTestObjectFactory::AddRootObjectReplicationParams(const UObject* Object, const UE::Net::FRootObjectReplicationParams& Params)
{
	InstanceReplicationParams.Add(FWeakObjectPtr(Object), Params);
}
