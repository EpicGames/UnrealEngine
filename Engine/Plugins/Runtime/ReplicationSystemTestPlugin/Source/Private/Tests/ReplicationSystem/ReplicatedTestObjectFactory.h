// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/NetObjectFactory.h"
#include "Iris/Core/NetObjectReference.h"
#include "Containers/Map.h"

#include "ReplicatedTestObjectFactory.generated.h"

namespace UE::Net
{

class FReplicationTestCreationHeader : public FNetObjectCreationHeader
{
public:
	FString ArchetypeOrStableName;
	FNetObjectReference DynamicObjectOuterReference;
	uint32 NumComponentsToSpawn = 0;
	uint32 NumIrisComponentsToSpawn = 0;
	uint32 NumDynamicComponentsToSpawn = 0;
	uint32 NumConnectionFilteredComponentsToSpawn = 0;
	uint32 NumObjectReferenceComponentsToSpawn = 0;

	bool bForceFailCreateRemoteInstance = false;
	bool bIsStable = false;

	virtual FString ToString() const override;

	bool Serialize(const FCreationHeaderContext& Context) const;
	bool Deserialize(const FCreationHeaderContext& Context);
};

} // end namespace UE::Net

UCLASS()
class UReplicatedTestObjectFactory : public UNetObjectFactory
{
	GENERATED_BODY()

public:

	static FName GetFactoryName();

	virtual FInstantiateResult InstantiateReplicatedObjectFromHeader(const FInstantiateContext& Context, const UE::Net::FNetObjectCreationHeader* InHeader) override;

	// UNetObjectFactory interface
	virtual void PostInit(const FPostInitContext& Context) override;

	virtual void DetachedFromReplication(const FDetachContext& Context, const TOptional<FSubObjectDetachContext>& SubObjectContext) override;

	virtual TOptional<FWorldInfoData> GetWorldInfo(const FWorldInfoContext& Context) const override;

	virtual void SubObjectCreatedFromReplication(UE::Net::FNetRefHandle RootObject, UE::Net::FNetRefHandle SubObjectCreated) override;

	virtual void SubObjectDetachedFromReplication(const FDetachContext& Context, const FSubObjectDetachContext& SubObjectContext) override;

	virtual void FillRootObjectReplicationParams(const UE::Net::FRootObjectReplicationParamsContext& Context, UE::Net::FRootObjectReplicationParams& OutParams);

	//------------------------------------------------------------------------

	void SetCreatedObjectsOnNode(TArray<TStrongObjectPtr<UObject>>* InCreatedObjectsOnNode)
	{
		CreatedObjectsOnNode = InCreatedObjectsOnNode;
	}

	//------------------------------------------------------------------------

	void SetWorldUpdateFunctor(TFunction<void(UE::Net::FNetRefHandle, const UObject*, FVector&, float&)> LocUpdateFunctor)
	{
		GetInstanceWorldObjectInfoFunction = LocUpdateFunctor;
	}

	/** Store instance specific replication params. This factory doesn't support any other interface for retrieving the info through other means. */
	void AddRootObjectReplicationParams(const UObject*, const UE::Net::FRootObjectReplicationParams& Params);

public:

	TArray<TStrongObjectPtr<UObject>>* CreatedObjectsOnNode = nullptr;

protected:

	virtual TUniquePtr<UE::Net::FNetObjectCreationHeader> CreateAndFillHeader(UE::Net::FNetRefHandle Handle) override;
	
	virtual bool SerializeHeader(const UE::Net::FCreationHeaderContext& Context, const UE::Net::FNetObjectCreationHeader* Header) override;

	virtual TUniquePtr<UE::Net::FNetObjectCreationHeader> CreateAndDeserializeHeader(const UE::Net::FCreationHeaderContext& Context) override;

private:
	
	TFunction<void(UE::Net::FNetRefHandle, const UObject*, FVector&, float&)> GetInstanceWorldObjectInfoFunction;

private:
	// Helper for the cases where we actually provide instance specific replication parameters. This is far from ideal but good enough for testing purposes.
	TSparseMap<FWeakObjectPtr, UE::Net::FRootObjectReplicationParams, FDefaultSparseSetAllocator, TWeakObjectPtrMapKeyFuncs<FWeakObjectPtr, UE::Net::FRootObjectReplicationParams>> InstanceReplicationParams;
};
