// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/ReplicationSystem/ReplicationSystemConfigOverrideTestFixture.h"
#include "Iris/Core/IrisLog.h"

namespace UE::Net
{

void FReplicationSystemConfigOverrideRPCTestFixture::SetUp()
{
	UNetBlobHandlerDefinitions* BlobHandlerDefinitions = GetMutableDefault<UNetBlobHandlerDefinitions>();
	OriginalHandlerDefinitions = BlobHandlerDefinitions->ReadWriteHandlerDefinitions();

	HandlerDefinitions.Add({ TEXT("NetRPCHandler") });
	HandlerDefinitions.Add({ TEXT("PartialNetObjectAttachmentHandler") });
	HandlerDefinitions.Add({ TEXT("NetObjectBlobHandler") });

	BlobHandlerDefinitions->ReadWriteHandlerDefinitions() = HandlerDefinitions;

	FNetworkAutomationTestSuiteFixture::SetUp();

	// Fake what we normally get from config
	DataStreamUtil.SetUp();
	DataStreamUtil.AddDataStreamDefinition(TEXT("NetToken"), TEXT("/Script/IrisCore.NetTokenDataStream"));
	DataStreamUtil.AddDataStreamDefinition(TEXT("Replication"), TEXT("/Script/IrisCore.ReplicationDataStream"));
	DataStreamUtil.FixupDefinitions();

	NetTokenDataStoreUtil.SetUp();
}

void FReplicationSystemConfigOverrideRPCTestFixture::TearDown()
{
	for (TArray<FReplicationSystemTestClient*> TempClients = Clients; FReplicationSystemTestClient * Client : TempClients)
	{
		delete Client;
	}
	delete Server;
	DataStreamUtil.TearDown();
	NetTokenDataStoreUtil.TearDown();

	FNetworkAutomationTestSuiteFixture::TearDown();

	UNetBlobHandlerDefinitions* BlobHandlerDefinitions = GetMutableDefault<UNetBlobHandlerDefinitions>();
	BlobHandlerDefinitions->ReadWriteHandlerDefinitions() = OriginalHandlerDefinitions;
}

void FReplicationSystemConfigOverrideRPCTestFixture::CreateServer(const FReplicationSystemTestNode::FReplicationSystemParamsOverride& ParamsOverride)
{
	checkf(Server == nullptr, TEXT("Server already created"));
	Server = new FReplicationSystemTestServer(FReplicationSystemTestNode::DelaySetup);
	Server->Setup(true, GetName(), &ParamsOverride);
}

FReplicationSystemTestClient* FReplicationSystemConfigOverrideRPCTestFixture::CreateClient(const FReplicationSystemTestNode::FReplicationSystemParamsOverride& ParamsOverride)
{
	if (!ensureMsgf(Server, TEXT("Must create a server before a client.")))
	{
		return nullptr;
	}

	FReplicationSystemTestClient* Client = new FReplicationSystemTestClient(FReplicationSystemTestNode::DelaySetup);
	Client->Setup(false, GetName(), &ParamsOverride);

	Clients.Add(Client);

	// The client needs a connection
	Client->LocalConnectionId = Client->AddConnection();

	// Auto connect to server
	Client->ConnectionIdOnServer = Server->AddConnection();

	return Client;
}

void FReplicationSystemConfigOverrideRPCTestFixture::DestroyClient(FReplicationSystemTestClient* Client)
{
	if (!Clients.Remove(Client))
	{
		UE_LOGF(LogIris, Warning, "Unable to find FReplicationSystemTestClient %p for destroy. NOT destroying.", Client);
		return;
	}

	Server->RemoveConnection(Client->ConnectionIdOnServer);

	delete Client;
}

}
