// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlobHandlerDefinitions.h"

namespace UE::Net
{

/**
 * A test fixture that allows overriding the replication system params and supports RPCs.
 * Unlike FReplicationSystemServerClientTestFixture, does not create a server automatically in SetUp()
 * to give test cases a chance to set up the override params. Call CreateServer() to create a server.
 */
class FReplicationSystemConfigOverrideRPCTestFixture : public FNetworkAutomationTestSuiteFixture
{
public:
	FReplicationSystemConfigOverrideRPCTestFixture() : FNetworkAutomationTestSuiteFixture() {}

protected:
	enum : bool
	{
		DoNotDeliverPacket = false,
		DeliverPacket = true,
	};

	virtual void SetUp() override;
	virtual void TearDown() override;

	void CreateServer(const FReplicationSystemTestNode::FReplicationSystemParamsOverride& ParamsOverride);
	FReplicationSystemTestClient* CreateClient(const FReplicationSystemTestNode::FReplicationSystemParamsOverride& ParamsOverride);
	void DestroyClient(FReplicationSystemTestClient* Client);

	FDataStreamTestUtil DataStreamUtil;
	FNetTokenDataStoreTestUtil NetTokenDataStoreUtil;
	FReplicationSystemTestServer* Server = nullptr;
	TArray<FReplicationSystemTestClient*> Clients;

	FReplicationSystemTestNode::FReplicationSystemParamsOverride OverrideServerConfig;
	FReplicationSystemTestNode::FReplicationSystemParamsOverride OverrideClientConfig;

private:
	TArray<FNetBlobHandlerDefinition> OriginalHandlerDefinitions;
	TArray<FNetBlobHandlerDefinition> HandlerDefinitions;
};

}
