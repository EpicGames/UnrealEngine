// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSystemProxyTestFixture.h"

namespace UE::Net
{

FReplicationSystemTestProxy::FReplicationSystemTestProxy(const TCHAR* Name)
{
	FReplicationSystemTestNode::FReplicationSystemParamsOverride ProxyFrontendParams;
	ProxyFrontendParams.ProxyType = EProxyType::Frontend;

	FrontendServer.Reset(new FReplicationSystemTestServer(Name, &ProxyFrontendParams));
}

FReplicationSystemTestClient* FReplicationSystemTestProxy::CreateAndConnectBackendClient(FReplicationSystemTestServer* BackendServer)
{
	FReplicationSystemTestNode::FReplicationSystemParamsOverride ProxyBackendParams;
	ProxyBackendParams.ProxyType = EProxyType::Backend;

	FReplicationSystemTestClient* BackendClient = new FReplicationSystemTestClient(TEXT("ProxyBackendClient"), &ProxyBackendParams);
	BackendClients.Emplace(BackendClient);

	// The client needs a connection
	BackendClient->LocalConnectionId = BackendClient->AddConnection();

	// Auto connect to server
	BackendClient->ConnectionIdOnServer = BackendServer->AddConnection();

	return BackendClient;
}

void FReplicationSystemProxyTestFixture::SetUp()
{
	FNetworkAutomationTestSuiteFixture::SetUp();

	// Fake what we normally get from config
	DataStreamUtil.SetUp();
	DataStreamUtil.AddDataStreamDefinition(TEXT("NetToken"), TEXT("/Script/IrisCore.NetTokenDataStream"));
	DataStreamUtil.AddDataStreamDefinition(TEXT("Replication"), TEXT("/Script/IrisCore.ReplicationDataStream"));
	DataStreamUtil.FixupDefinitions();

	NetTokenDataStoreUtil.SetUp();
}

void FReplicationSystemProxyTestFixture::TearDown()
{
	Clients.Reset();
	ProxyServers.Reset();
	BackendServers.Reset();

	ServerToProxyClients.Reset();
	ProxyFrontendToClients.Reset();

	DataStreamUtil.TearDown();
	NetTokenDataStoreUtil.TearDown();

	FNetworkAutomationTestSuiteFixture::TearDown();
}

FReplicationSystemTestServer* FReplicationSystemProxyTestFixture::CreateServer(uint32 InitialNetRefHandleIndex)
{
	FReplicationSystemTestNode::FReplicationSystemParamsOverride BackendServerParams;
	BackendServerParams.InitialNetRefHandleIndex = InitialNetRefHandleIndex;

	return BackendServers.Emplace_GetRef(new FReplicationSystemTestServer(GetName(), &BackendServerParams)).Get();
}

FReplicationSystemTestProxy* FReplicationSystemProxyTestFixture::CreateProxy()
{
	return ProxyServers.Emplace_GetRef(new FReplicationSystemTestProxy(GetName())).Get();;
}

FReplicationSystemTestClient* FReplicationSystemProxyTestFixture::CreateClient()
{
	return Clients.Emplace_GetRef(new FReplicationSystemTestClient(GetName())).Get();
}

void FReplicationSystemProxyTestFixture::ConnectProxyToBackendServer(FReplicationSystemTestProxy* Proxy, FReplicationSystemTestServer* BackendServer)
{
	FReplicationSystemTestClient* BackendClient = Proxy->CreateAndConnectBackendClient(BackendServer);
	if (BackendClient == nullptr)
	{
		return;
	}

	TArray<FReplicationSystemTestClient*>& ServerClients = ServerToProxyClients.FindOrAdd(BackendServer);
	ServerClients.Add(BackendClient);

	UObjectReplicationBridge* BackendBridge = Cast<UObjectReplicationBridge>(GetProxyClientForBackendServer(BackendServer, Proxy)->GetReplicationBridge());
	UObjectReplicationBridge* FrontendBridge = Cast<UObjectReplicationBridge>(Proxy->GetFrontendServer()->GetReplicationBridge());
	
	if (ensure(BackendBridge) && ensure(FrontendBridge))
	{
		BindProxyFrontendAndBackend(FrontendBridge, BackendBridge);
	}
}

void FReplicationSystemProxyTestFixture::ConnectClientToProxy(FReplicationSystemTestClient* Client, FReplicationSystemTestProxy* Proxy)
{
	FReplicationSystemTestServer* ProxyFrontendServer = Proxy->GetFrontendServer();

	// The client needs a connection
	Client->LocalConnectionId = Client->AddConnection();

	// Auto connect to server
	Client->ConnectionIdOnServer = ProxyFrontendServer->AddConnection();

	TArray<FReplicationSystemTestClient*>& ProxyFrontendClients = ProxyFrontendToClients.FindOrAdd(ProxyFrontendServer);
	ProxyFrontendClients.Add(Client);
}

bool FReplicationSystemProxyTestFixture::UpdateAndSendServerToAllProxies(FReplicationSystemTestServer* Server)
{
	bool bSuccess = true;

	Server->NetUpdate();

	TArray<FReplicationSystemTestClient*>* ProxyClients = ServerToProxyClients.Find(Server);

	if (ProxyClients)
	{
		for (FReplicationSystemTestClient* Client : *ProxyClients)
		{
			bSuccess &= Server->SendAndDeliverTo(Client, true);
		}
	}
	else
	{
		bSuccess = false;
	}

	Server->PostSendUpdate();

	return bSuccess;
}

bool FReplicationSystemProxyTestFixture::UpdateAndSendProxyToAllClients(FReplicationSystemTestProxy* Proxy)
{
	bool bSuccess = true;

	FReplicationSystemTestServer* const ProxyFrontendServer = Proxy->GetFrontendServer();
	ProxyFrontendServer->NetUpdate();

	TArray<FReplicationSystemTestClient*>* ProxyClients = ProxyFrontendToClients.Find(ProxyFrontendServer);

	if (ProxyClients)
	{
		for (FReplicationSystemTestClient* Client : *ProxyClients)
		{
			bSuccess &= ProxyFrontendServer->SendAndDeliverTo(Client, true);
		}
	}
	else
	{
		bSuccess = false;
	}

	ProxyFrontendServer->PostSendUpdate();

	return bSuccess;
}

FReplicationSystemTestClient* FReplicationSystemProxyTestFixture::GetProxyClientForBackendServer(FReplicationSystemTestServer* Server, FReplicationSystemTestProxy* Proxy)
{
	TArray<FReplicationSystemTestClient*>* ProxyClientsForServer = ServerToProxyClients.Find(Server);
		
	if (ProxyClientsForServer)
	{
		for (const TUniquePtr<FReplicationSystemTestClient>& ProxyClient : Proxy->BackendClients)
		{
			int32 FoundIndex = ProxyClientsForServer->Find(ProxyClient.Get());
			if (FoundIndex != INDEX_NONE)
			{
				return (*ProxyClientsForServer)[FoundIndex];
			}
		}
	}

	return nullptr;
}

FOnRootObjectPostInit::RegistrationType& FReplicationSystemProxyTestFixture::GetRootObjectPostInitDelegate(UObjectReplicationBridge* Bridge)
{
	return UE::Net::Private::FObjectReplicationBridgeDelegates::GetRootObjectPostInitDelegate(Bridge);
}

FOnRootObjectDetached::RegistrationType& FReplicationSystemProxyTestFixture::GetRootObjectDetachedDelegate(UObjectReplicationBridge* Bridge)
{
	return UE::Net::Private::FObjectReplicationBridgeDelegates::GetRootObjectDetachedDelegate(Bridge);
}

FOnSubObjectPostInit::RegistrationType& FReplicationSystemProxyTestFixture::GetSubObjectPostInitDelegate(UObjectReplicationBridge* Bridge)
{
	return UE::Net::Private::FObjectReplicationBridgeDelegates::GetSubObjectPostInitDelegate(Bridge);
}

FOnSubObjectDetached::RegistrationType& FReplicationSystemProxyTestFixture::GetSubObjectDetachedDelegate(UObjectReplicationBridge* Bridge)
{
	return UE::Net::Private::FObjectReplicationBridgeDelegates::GetSubObjectDetachedDelegate(Bridge);
}

void FReplicationSystemProxyTestFixture::BindProxyFrontendAndBackend(UObjectReplicationBridge* FrontendBridge, UObjectReplicationBridge* BackendBridge)
{
	GetRootObjectPostInitDelegate(BackendBridge).AddLambda(
		[FrontendBridge](UE::Net::FNetRefHandle RootHandle, UObject* RootObject, UE::Net::FNetObjectFactoryId FactoryId)
		{
			FrontendBridge->StartReplicatingRootObject(RootObject, FactoryId);
		});

	GetRootObjectDetachedDelegate(BackendBridge).AddLambda(
		[FrontendBridge](UE::Net::FNetRefHandle RootHandle, UObject* RootObject, UE::Net::EDetachReason Reason)
		{
			const EEndReplicationFlags Flags = (Reason == UE::Net::EDetachReason::TornOff) ? EEndReplicationFlags::TearOff : EEndReplicationFlags::Destroy;
			FrontendBridge->StopReplicatingNetObject(RootObject, Flags);
		});

	GetSubObjectPostInitDelegate(BackendBridge).AddLambda(
		[FrontendBridge](UE::Net::FNetRefHandle SubObjectHandle, UObject* SubObject, UE::Net::FNetRefHandle RootObjectHandle, UObject* RootObject, UE::Net::FNetObjectFactoryId FactoryId)
		{
			UE::Net::FNetRefHandle FrontendRootObjectHandle = FrontendBridge->GetReplicatedRefHandle(RootObject);
			if (ensure(FrontendRootObjectHandle.IsValid()))
			{
				UObjectReplicationBridge::FSubObjectReplicationParams Params;
				Params.RootObjectHandle = FrontendRootObjectHandle;
				FrontendBridge->StartReplicatingSubObject(SubObject, Params, FactoryId);
			}
		});

	GetSubObjectDetachedDelegate(BackendBridge).AddLambda(
		[FrontendBridge](UE::Net::FNetRefHandle SubObjectHandle, UObject* SubObject, UE::Net::FNetRefHandle RootObjectHandle, UObject* RootObject, UE::Net::EDetachReason Reason)
		{
			const EEndReplicationFlags Flags = (Reason == UE::Net::EDetachReason::TornOff) ? EEndReplicationFlags::TearOff : EEndReplicationFlags::Destroy;
			FrontendBridge->StopReplicatingNetObject(SubObject, Flags);
		});
}

}
