// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "NetworkAutomationTest.h"
#include "ReplicationSystemServerClientTestFixture.h"
#include "Templates/UniquePtr.h"

namespace UE::Net
{

class FReplicationSystemTestProxy
{
public:
	// Creates the frontend server
	explicit FReplicationSystemTestProxy(const TCHAR* Name);

	// Returns non-owning pointer, do not delete it!
	FReplicationSystemTestClient* CreateAndConnectBackendClient(FReplicationSystemTestServer* BackendServer);

	// Returns non-owning pointer, do not delete it!
	FReplicationSystemTestServer* GetFrontendServer()
	{
		return FrontendServer.Get();
	}

	TArray<TUniquePtr<FReplicationSystemTestClient>> BackendClients;
	TUniquePtr<FReplicationSystemTestServer> FrontendServer;
};

class FReplicationSystemProxyTestFixture : public FNetworkAutomationTestSuiteFixture
{
public:
	FReplicationSystemProxyTestFixture() : FNetworkAutomationTestSuiteFixture() {}

protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	// Returns non-owning pointer, do not delete it!
	FReplicationSystemTestServer* CreateServer(uint32 InitialNetRefHandleIndex = 1);

	// Returns non-owning pointer, do not delete it!
	FReplicationSystemTestProxy* CreateProxy();

	// Returns non-owning pointer, do not delete it!
	FReplicationSystemTestClient* CreateClient();

	void ConnectProxyToBackendServer(FReplicationSystemTestProxy* Proxy, FReplicationSystemTestServer* BackendServer);
	void ConnectClientToProxy(FReplicationSystemTestClient* Client, FReplicationSystemTestProxy* Proxy);

	bool UpdateAndSendServerToAllProxies(FReplicationSystemTestServer* Server);
	bool UpdateAndSendProxyToAllClients(FReplicationSystemTestProxy* Proxy);

	FReplicationSystemTestClient* GetProxyClientForBackendServer(FReplicationSystemTestServer* Server, FReplicationSystemTestProxy* Proxy);

	FDataStreamTestUtil DataStreamUtil;
	FNetTokenDataStoreTestUtil NetTokenDataStoreUtil;

	TArray<TUniquePtr<FReplicationSystemTestClient>> Clients;
	TArray<TUniquePtr<FReplicationSystemTestProxy>> ProxyServers;
	TArray<TUniquePtr<FReplicationSystemTestServer>> BackendServers;

	// Map of backend server nodes to their corresponding client nodes on proxies.
	// Stores non-owning pointers, do not delete them!
	TMap<FReplicationSystemTestServer*, TArray<FReplicationSystemTestClient*>> ServerToProxyClients;

	// Map of frontend proxy server nodes to their corresponding client nodes
	// Stores non-owning pointers, do not delete them!
	TMap<FReplicationSystemTestServer*, TArray<FReplicationSystemTestClient*>> ProxyFrontendToClients;
	
	static FOnRootObjectPostInit::RegistrationType& GetRootObjectPostInitDelegate(UObjectReplicationBridge* Bridge);
	static FOnRootObjectDetached::RegistrationType& GetRootObjectDetachedDelegate(UObjectReplicationBridge* Bridge);
	static FOnSubObjectPostInit::RegistrationType& GetSubObjectPostInitDelegate(UObjectReplicationBridge* Bridge);
	static FOnSubObjectDetached::RegistrationType& GetSubObjectDetachedDelegate(UObjectReplicationBridge* Bridge);

private:

	// Setup delegates to start and stop replicating objects between the backend and frontend replication systems.
	void BindProxyFrontendAndBackend(UObjectReplicationBridge* FrontendBridge, UObjectReplicationBridge* BackendBridge);
};

}
