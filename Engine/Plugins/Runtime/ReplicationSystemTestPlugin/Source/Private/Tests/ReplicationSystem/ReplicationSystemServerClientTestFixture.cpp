// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSystemServerClientTestFixture.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/ReplicationConnections.h"
#include "Iris/ReplicationSystem/ReplicationWriter.h"
#include "Iris/ReplicationSystem/ReplicationReader.h"
#include "Iris/PacketControl/PacketNotification.h"
#include "Iris/ReplicationSystem/NetTokenStore.h"
#include "Net/Core/Trace/NetTrace.h"

namespace UE::Net
{

void FNetTokenDataStoreTestUtil::SetUp()
{
	StoreNetTokenStoreConfig();
	AddNetTokenStoreTypeIdPair(TEXT("StringTokenStore"), 0);
	AddNetTokenStoreTypeIdPair(TEXT("NameTokenStore"), 1);
}

void FNetTokenDataStoreTestUtil::TearDown()
{
	RestoreNetTokenStoreConfig();
}

void FNetTokenDataStoreTestUtil::AddNetTokenStoreTypeIdPair(FString StoreTypeName, uint32 TypeID)
{
	bool bFound = false;
	for (const auto& Elem :  NetTokenTypeIdConfig->ReservedTypeIds)
	{
		if (Elem.StoreTypeName == StoreTypeName)
		{
			bFound = true;
			break;
		}
	}
	if (!bFound)
	{
		FNetTokenStoreTypeIdPair TypeIdPair = {.StoreTypeName = StoreTypeName, .TypeID = TypeID};
		NetTokenTypeIdConfig->ReservedTypeIds.Add(TypeIdPair);
	}
}

void FNetTokenDataStoreTestUtil::StoreNetTokenStoreConfig()
{
	GetDefault<UNetTokenTypeIdConfig>();
	NetTokenTypeIdConfig = static_cast<UNetTokenTypeIdConfig*>(GetMutableDefault<UNetTokenTypeIdConfig>());
	check(NetTokenTypeIdConfig != nullptr);
	OriginalReservedTypeIds = NetTokenTypeIdConfig->ReservedTypeIds;
}

void FNetTokenDataStoreTestUtil::RestoreNetTokenStoreConfig()
{
	Swap(NetTokenTypeIdConfig->ReservedTypeIds, OriginalReservedTypeIds);
}

// FDataStreamTestUtil implementation
FDataStreamTestUtil::FDataStreamTestUtil()
: DataStreamDefinitions(nullptr)
, CurrentDataStreamDefinitions(nullptr)
{
}

void FDataStreamTestUtil::SetUp()
{
	StoreDataStreamDefinitions();
}

void FDataStreamTestUtil::TearDown()
{
	RestoreDataStreamDefinitions();
}

void FDataStreamTestUtil::StoreDataStreamDefinitions()
{
	DataStreamDefinitions = static_cast<UMyDataStreamDefinitions*>(GetMutableDefault<UDataStreamDefinitions>());
	check(DataStreamDefinitions != nullptr);
	CurrentDataStreamDefinitions = &DataStreamDefinitions->ReadWriteDataStreamDefinitions();
	PointerToFixupComplete = &DataStreamDefinitions->ReadWriteFixupComplete();

	PreviousDataStreamDefinitions.Empty();
	Swap(*CurrentDataStreamDefinitions, PreviousDataStreamDefinitions);
	*PointerToFixupComplete = false;
}

void FDataStreamTestUtil::RestoreDataStreamDefinitions()
{
	Swap(*CurrentDataStreamDefinitions, PreviousDataStreamDefinitions);
	*PointerToFixupComplete = false;
}

void FDataStreamTestUtil::AddDataStreamDefinition(const TCHAR* StreamName, const TCHAR* ClassPath, const FAddDataStreamDefinitionParams& Params)
{
	FDataStreamDefinition Definition = {};

	Definition.DataStreamName = FName(StreamName);
	Definition.ClassName = Params.bValid ? FName(ClassPath): FName();
	Definition.Class = nullptr;
	Definition.DefaultSendStatus = EDataStreamSendStatus::Send;
	Definition.bAutoCreate = Params.bAutoCreate;
	Definition.bDynamicCreate = Params.bDynamicCreate;

	CurrentDataStreamDefinitions->Add(Definition);
}

// FReplicationSystemTestNode implementation
FReplicationSystemTestNode::FReplicationSystemTestNode(bool bIsServer, const TCHAR* Name, const FReplicationSystemTestNode::FReplicationSystemParamsOverride* ParamsOverride)
{
	Setup(bIsServer, Name, ParamsOverride);
}

FReplicationSystemTestNode::FReplicationSystemTestNode(FReplicationSystemTestNode::EDelaySetup)
{
	// Do nothing until asked to start up
}

void FReplicationSystemTestNode::Setup(bool bIsServer, const TCHAR* Name, const FReplicationSystemTestNode::FReplicationSystemParamsOverride* ParamsOverride)
{
	// Init NetTokenStore
	{
		using namespace UE::Net;

		NetTokenDataStoreUtil.SetUp();
		
		NetTokenStore = MakeUnique<FNetTokenStore>();

		FNetTokenStore::FInitParams NetTokenStoreInitParams;
		NetTokenStoreInitParams.Authority = bIsServer ? FNetToken::ENetTokenAuthority::Authority : FNetToken::ENetTokenAuthority::None;
		NetTokenStore->Init(NetTokenStoreInitParams);

		// Register data stores for supported types, $TODO: make this configurable.
		NetTokenStore->CreateAndRegisterDataStore<FStringTokenStore>();
		NetTokenStore->CreateAndRegisterDataStore<FNameTokenStore>();
	}

	ReplicationBridge = NewObject<UReplicatedTestObjectBridge>();
	check(ReplicationBridge != nullptr);

	CreatedObjects.Add(TStrongObjectPtr<UObject>(ReplicationBridge));

	UReplicationSystem::FReplicationSystemParams Params;
	Params.ReplicationBridge = ReplicationBridge;
	Params.bIsServer = bIsServer;
	Params.bAllowObjectReplication = bIsServer;
	Params.NetTokenStore = NetTokenStore.Get();

	if (ParamsOverride)
	{
		Params.MaxReplicatedObjectCount = ParamsOverride->MaxReplicatedObjectCount > 0 ? ParamsOverride->MaxReplicatedObjectCount : Params.MaxReplicatedObjectCount;
		Params.InitialNetObjectListCount = ParamsOverride->InitialNetObjectListCount > 0 ? ParamsOverride->InitialNetObjectListCount : Params.InitialNetObjectListCount;
		Params.NetObjectListGrowCount = ParamsOverride->NetObjectListGrowCount > 0 ? ParamsOverride->NetObjectListGrowCount : Params.NetObjectListGrowCount;
		Params.bUseRemoteObjectReferences = ParamsOverride->bUseRemoteObjectReferences;
		Params.bAllowParallelTasks = ParamsOverride->bAllowParallelTasks;
		if (ParamsOverride->bAllowMinimalUpdateIfNoConnections.IsSet())
		{
			Params.bAllowMinimalUpdateIfNoConnections = ParamsOverride->bAllowMinimalUpdateIfNoConnections.GetValue();
		}
		Params.InitialNetRefHandleIndex = ParamsOverride->InitialNetRefHandleIndex;
		Params.ProxyType = ParamsOverride->ProxyType;
	}

	LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Error);
	ReplicationSystem = FReplicationSystemFactory::CreateReplicationSystem(Params);	
	if (!bIsServer)
	{
		ReplicationBridge->SetCreatedObjectsOnNode(&CreatedObjects);
	}

	UE_NET_TRACE_UPDATE_INSTANCE(GetNetTraceId(), bIsServer, Name);
}

FReplicationSystemTestNode::~FReplicationSystemTestNode()
{
	const uint32 NetTraceId = ReplicationSystem->GetId();

	LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Error);
	FReplicationSystemFactory::DestroyReplicationSystem(ReplicationSystem);
	CreatedObjects.Empty();
	NetTokenDataStoreUtil.TearDown();

	// End NetTrace session for this instance
	UE_NET_TRACE_END_SESSION(NetTraceId);
}

uint32 FReplicationSystemTestNode::GetNetTraceId() const
{ 
	return ReplicationSystem ? ReplicationSystem->GetId() : ~0U;
}

UTestReplicatedIrisObject* FReplicationSystemTestNode::CreateObject(const UObjectReplicationBridge::FRootObjectReplicationParams& Params, UTestReplicatedIrisObject::FComponents* ComponentsToCreate)
{
	UTestReplicatedIrisObject* CreatedObject = NewObject<UTestReplicatedIrisObject>();

	if (ComponentsToCreate)
	{
		CreatedObject->AddComponents(*ComponentsToCreate);
	}

	CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedObject));

	// Add it to the bridge for replication
	ReplicationBridge->BeginReplication(CreatedObject, Params);

	return CreatedObject;
}

UTestReplicatedIrisObject* FReplicationSystemTestNode::CreateObject(uint32 NumComponents, uint32 NumIrisComponents)
{
	UTestReplicatedIrisObject* CreatedObject = NewObject<UTestReplicatedIrisObject>();
	CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedObject));
	CreatedObject->AddComponents(NumComponents, NumIrisComponents);

	// Add it to the bridge for replication
	ReplicationBridge->BeginReplication(CreatedObject);
	
	return CreatedObject;
}

UTestReplicatedIrisObject* FReplicationSystemTestNode::CreateObject(const UTestReplicatedIrisObject::FComponents& Components)
{
	UTestReplicatedIrisObject* CreatedObject = NewObject<UTestReplicatedIrisObject>();
	CreatedObject->AddComponents(Components);

	CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedObject));

	// Add it to the bridge for replication
	ReplicationBridge->BeginReplication(CreatedObject);
	
	return CreatedObject;
}

UTestReplicatedIrisObject* FReplicationSystemTestNode::CreateSubObject(FNetRefHandle Owner, const UTestReplicatedIrisObject::FComponents& Components)
{
	UTestReplicatedIrisObject* CreatedObject = NewObject<UTestReplicatedIrisObject>();
	CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedObject));
	CreatedObject->AddComponents(Components);

	// Add it to the bridge for replication
	ReplicationBridge->BeginReplication(Owner, CreatedObject);
	
	return CreatedObject;
}

UTestReplicatedIrisObject* FReplicationSystemTestNode::CreateSubObject(FNetRefHandle Owner, uint32 NumComponents, uint32 NumIrisComponents)
{
	UTestReplicatedIrisObject* CreatedObject = NewObject<UTestReplicatedIrisObject>();
	CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedObject));
	CreatedObject->AddComponents(NumComponents, NumIrisComponents);

	// Add it to the bridge for replication
	ReplicationBridge->BeginReplication(Owner, CreatedObject);
	
	return CreatedObject;
}

UTestReplicatedIrisObject* FReplicationSystemTestNode::CreateObjectWithDynamicState(uint32 NumComponents, uint32 NumIrisComponents, uint32 NumDynamicStateComponents)
{
	UTestReplicatedIrisObject* CreatedObject = NewObject<UTestReplicatedIrisObject>();
	CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedObject));

	CreatedObject->AddComponents(NumComponents, NumIrisComponents);
	CreatedObject->AddDynamicStateComponents(NumDynamicStateComponents);
	
	return CreatedObject;
}

void FReplicationSystemTestNode::DestroyObject(UReplicatedTestObject* Object, EEndReplicationFlags EndReplicationFlags)
{
	// Destroy handle
	check(Object && Object->NetRefHandle.IsValid());

	// End replication for the handle
	ReplicationBridge->EndReplication(Object, EndReplicationFlags);

	// Release ref
	CreatedObjects.Remove(TStrongObjectPtr<UObject>(Object));

	// Mark as garbage
	Object->MarkAsGarbage();
}

bool FReplicationSystemTestNode::IsResolvableNetRefHandle(FNetRefHandle RefHandle)
{
	return ReplicationBridge->GetReplicatedObject(RefHandle) != nullptr;
}

bool FReplicationSystemTestNode::IsValidNetRefHandle(FNetRefHandle RefHandle)
{
	return ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager().IsNetRefHandleAssigned(RefHandle);
}

uint32 FReplicationSystemTestNode::AddConnection()
{
	check(ReplicationSystem);

	FConnectionInfo Connection;
	Connection.ConnectionId = Connections.Num() + 1U;

	UE_NET_TRACE_CONNECTION_CREATED(GetNetTraceId(), Connection.ConnectionId);
	UE_NET_TRACE_CONNECTION_STATE_UPDATED(GetNetTraceId(), Connection.ConnectionId, static_cast<uint8>(3));

	// Init and store RemoteNetTokenStoreState
	ReplicationSystem->GetNetTokenStore()->InitRemoteNetTokenStoreState(Connection.ConnectionId);
	//Connection.RemoteNetTokenStoreState = ReplicationSystem->GetNetTokenStore()->GetRemoteNetTokenStoreState(Connection.ConnectionId);
	
	// Add a connection
	ReplicationSystem->AddConnection(Connection.ConnectionId);

	Connection.DataStreamManager = NewObject<UDataStreamManager>();

	// Make ReplicationSystem aware of new DataStreamManager associated with connection.
	ReplicationSystem->InitDataStreamManager(Connection.ConnectionId, Connection.DataStreamManager);

	CreatedObjects.Add(TStrongObjectPtr<UObject>(Connection.DataStreamManager));

	// Streams created based on config
	Connection.DataStreamManager->CreateStream("NetToken");
	Connection.DataStreamManager->CreateStream("Replication");

	// Enable replication
	ReplicationSystem->SetReplicationEnabledForConnection(Connection.ConnectionId, true);

	// Add view
	FReplicationView View;
	View.Views.AddDefaulted();

	ReplicationSystem->SetReplicationView(Connection.ConnectionId, View);

	Connections.Add(Connection);

	return Connection.ConnectionId;
}

void FReplicationSystemTestNode::RemoveConnection(uint32 ConnectionId)
{
	for (TArray<FConnectionInfo>::TIterator It = Connections.CreateIterator(); It; ++It)
	{
		FConnectionInfo& ConnectionInfo = *It;
		if (ConnectionInfo.ConnectionId != ConnectionId)
		{
			continue;
		}

		ReplicationSystem->RemoveConnection(ConnectionInfo.ConnectionId);

		if (IsValid(ConnectionInfo.DataStreamManager))
		{
			ConnectionInfo.DataStreamManager->Deinit();
			ConnectionInfo.DataStreamManager->MarkAsGarbage();
		}

		UE_NET_TRACE_CONNECTION_CLOSED(GetNetTraceId(), ConnectionId);

		It.RemoveCurrent();
		break;
	}
}

void FReplicationSystemTestNode::NetUpdate()
{
	ReplicationSystem->NetUpdate(1.f);
}

void FReplicationSystemTestNode::TickPostReceive()
{
	ReplicationSystem->TickPostReceive();
}

bool FReplicationSystemTestNode::SendUpdate(uint32 ConnectionId, const TCHAR* Desc)
{
	FPacketData Packet;

	FNetBitStreamWriter Writer;
	Writer.InitBytes(Packet.PacketBuffer, FMath::Min<uint32>(FPacketData::MaxPacketSize, MaxSendPacketSize));

	FNetSerializationContext Context(&Writer);

	Context.SetTraceCollector(UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace));

	FConnectionInfo& Connection = GetConnectionInfo(ConnectionId);

	const FDataStreamRecord* Record = nullptr;
	UDataStream::FBeginWriteParameters BeginWriteParameters;

	if (CurrentSendPass == EReplicationSystemSendPass::PostTickDispatch)
	{
		BeginWriteParameters.WriteMode = EDataStreamWriteMode::PostTickDispatch;
	}

	bool bWroteData = false;
	if (Connection.DataStreamManager->BeginWrite(BeginWriteParameters) != UDataStream::EWriteResult::NoData)
	{
		Connection.DataStreamManager->WriteData(Context, Record);

		if (Writer.GetPosBits() > 0)
		{
			Writer.CommitWrites();
			Packet.BitCount = Writer.GetPosBits();
			Packet.PacketId = PacketId++;
			if (Desc)
			{
				Packet.Desc = FString(Desc);
			}

			Connection.WriteRecords.Enqueue(Record);
			Connection.WrittenPackets.Enqueue(Packet);

			bWroteData = true;
		}

		Connection.DataStreamManager->EndWrite();
	}

	if (bWroteData)
	{
		UE_NET_TRACE_FLUSH_COLLECTOR(Context.GetTraceCollector(), GetNetTraceId(), Connection.ConnectionId, ENetTracePacketType::Outgoing);
		UE_NET_TRACE_PACKET_SEND(GetNetTraceId(), Connection.ConnectionId, Packet.PacketId, Packet.BitCount);
		UE_LOGF(LogIris, Verbose, "ReplicationSystemTestFixture: Conn: %u Send PacketId: %u %ls", ConnectionId, Packet.PacketId, *Packet.Desc);
	}

	UE_NET_TRACE_DESTROY_COLLECTOR(Context.GetTraceCollector());

	return bWroteData;
}

void FReplicationSystemTestNode::PostSendUpdate()
{
	ReplicationSystem->PostSendUpdate();
	CurrentSendPass =	EReplicationSystemSendPass::Invalid;
}

void FReplicationSystemTestNode::DeliverTo(FReplicationSystemTestNode& Dest, uint32 LocalConnectionId, uint32 RemoteConnectionId, bool bDeliver)
{
	FConnectionInfo& Connection = GetConnectionInfo(LocalConnectionId);
	if (Connection.WrittenPackets.IsEmpty())
	{
		UE_LOGF(LogIris, Verbose, "ReplicationSystemTestFixture: Conn: %u Unable to %s packet as there are no packets.", LocalConnectionId, (bDeliver ? "deliver" : "drop"));
		return;
	}

	const FPacketData& Packet = Connection.WrittenPackets.Peek();

	if (bDeliver)
	{
		FNetBitStreamReader Reader;
		Reader.InitBits(Packet.PacketBuffer, Packet.BitCount);
		FNetSerializationContext Context(&Reader);

		Context.SetTraceCollector(UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace));

		UE_LOGF(LogIris, Verbose, "ReplicationSystemTestFixture: Conn: %u Deliver PacketId: %u %ls", LocalConnectionId, Packet.PacketId, *Packet.Desc);
		Dest.RecvUpdate(RemoteConnectionId, Context);

		UE_NET_TRACE_FLUSH_COLLECTOR(Context.GetTraceCollector(), Dest.GetNetTraceId(), RemoteConnectionId, ENetTracePacketType::Incoming);
		UE_NET_TRACE_DESTROY_COLLECTOR(Context.GetTraceCollector());
		UE_NET_TRACE_PACKET_RECV(Dest.GetNetTraceId(), RemoteConnectionId, Packet.PacketId, Packet.BitCount);
	}
	else
	{
		UE_LOGF(LogIris, Verbose, "ReplicationSystemTestFixture: Conn: %u Dropped PacketId: %u %ls", LocalConnectionId, Packet.PacketId, *Packet.Desc);
		UE_NET_TRACE_PACKET_DROPPED(Dest.GetNetTraceId(), RemoteConnectionId, Packet.PacketId, ENetTracePacketType::Incoming);
		UE_NET_TRACE_PACKET_DROPPED(GetNetTraceId(), LocalConnectionId, Packet.PacketId, ENetTracePacketType::Outgoing);
	}

	// If this triggers an assert, ensure that SendTo() actually wrote any packets before.
	Connection.DataStreamManager->ProcessPacketDeliveryStatus(bDeliver ? EPacketDeliveryStatus::Delivered : EPacketDeliveryStatus::Lost, Connection.WriteRecords.Peek());
	Connection.WriteRecords.Pop();
	Connection.WrittenPackets.Pop();
}

void FReplicationSystemTestNode::RecvUpdate(uint32 ConnectionId, FNetSerializationContext& Context)
{
	FConnectionInfo& Connection = GetConnectionInfo(ConnectionId);

	Connection.DataStreamManager->ReadData(Context);

	ensure(!Context.HasErrorOrOverflow());
	ensure(Context.GetBitStreamReader()->GetBitsLeft() == 0U);
}

uint32 FReplicationSystemTestNode::GetReplicationSystemId() const
{
	return ReplicationSystem ? ReplicationSystem->GetId() : uint32(~0U);
}

float FReplicationSystemTestNode::ConvertPollPeriodIntoFrequency(uint32 PollPeriod) const
{
	const float PollFrequency = ReplicationBridge->GetMaxTickRate() / (float)(PollPeriod + 1);
	return PollFrequency;
}

//*****************************************************************************
// Class FReplicationSystemTestClient
//*****************************************************************************

FReplicationSystemTestClient::FReplicationSystemTestClient(const TCHAR* Name, const FReplicationSystemTestNode::FReplicationSystemParamsOverride* ParamsOverride)
: FReplicationSystemTestNode(false, Name, ParamsOverride)
, ConnectionIdOnServer(~0U)
{
}

bool FReplicationSystemTestClient::UpdateAndSend(FReplicationSystemTestServer* Server, bool bDeliver, const TCHAR* Desc)
{
	bool bSuccess = false;

	NetUpdate();
	const bool bSentPacket = SendUpdate(Desc);
	PostSendUpdate();

	if (bSentPacket)
	{
		constexpr uint32 ServerRemoteConnectionId = 0x01;
		DeliverTo(*Server, LocalConnectionId, ServerRemoteConnectionId, bDeliver);
		bSuccess = true;
	}

	return bSuccess;
}

//*****************************************************************************
// Class FReplicationSystemTestServer
//*****************************************************************************

FReplicationSystemTestServer::FReplicationSystemTestServer(const TCHAR* Name, const FReplicationSystemTestNode::FReplicationSystemParamsOverride* ParamsOverride)
	: FReplicationSystemTestNode(true, Name, ParamsOverride)
{
}

bool FReplicationSystemTestServer::SendAndDeliverTo(FReplicationSystemTestClient* Client, bool bDeliver, const TCHAR* Desc)
{
	if (SendUpdate(Client->ConnectionIdOnServer, Desc))
	{
		DeliverTo(Client, bDeliver);

		return true;
	}

	return false;
}

// Send data, return true if data was written
bool FReplicationSystemTestServer::SendTo(FReplicationSystemTestClient* Client, const TCHAR* Desc)
{
	return SendUpdate(Client->ConnectionIdOnServer, Desc);
}

// If bDeliver is true deliver data to client and report packet as delivered
// if bDeliver is false, do not deliver packet and report a dropped packet
void FReplicationSystemTestServer::DeliverTo(FReplicationSystemTestClient* Client, bool bDeliver)
{
	FReplicationSystemTestNode::DeliverTo(*Client, Client->ConnectionIdOnServer, Client->LocalConnectionId, bDeliver);
	if (bUpdateClientOnDelivery)
	{
		Client->NetUpdate();
		Client->PostSendUpdate();
	}
}

bool FReplicationSystemTestServer::UpdateAndSend(const TArrayView<FReplicationSystemTestClient*const>& Clients, bool bDeliver /*= true*/, const TCHAR* Desc)
{
	bool bSentPackets = false;

	NetUpdate();

	TArray<FReplicationSystemTestClient*> ClientsWithData;
	ClientsWithData.Reserve(Clients.Num());

	FNetBitArray SentPackets;
	SentPackets.Init(Clients.Num());

	for (FReplicationSystemTestClient* Client : Clients)
	{
		if (SendTo(Client, Desc))
		{
			ClientsWithData.Add(Client);
		}
	}
	
	PostSendUpdate();

	for (FReplicationSystemTestClient* Client : ClientsWithData)
	{
		DeliverTo(Client, bDeliver);
		bSentPackets = true;
	}

	return bSentPackets;
}

//*****************************************************************************
// Class FReplicationSystemServerClientTestFixture
//*****************************************************************************

void FReplicationSystemServerClientTestFixture::SetUp()
{
	FNetworkAutomationTestSuiteFixture::SetUp();

	// Fake what we normally get from config
	DataStreamUtil.SetUp();
	DataStreamUtil.AddDataStreamDefinition(TEXT("NetToken"), TEXT("/Script/IrisCore.NetTokenDataStream"));
	DataStreamUtil.AddDataStreamDefinition(TEXT("Replication"), TEXT("/Script/IrisCore.ReplicationDataStream"));
	DataStreamUtil.FixupDefinitions();

	NetTokenDataStoreUtil.SetUp();

	Server = new FReplicationSystemTestServer(GetName(), &ServerParamsOverride);
}

void FReplicationSystemServerClientTestFixture::TearDown()
{
	for (TArray<FReplicationSystemTestClient*> TempClients = Clients; FReplicationSystemTestClient* Client : TempClients)
	{
		delete Client;
	}
	delete Server;
	DataStreamUtil.TearDown();
	NetTokenDataStoreUtil.TearDown();

	FNetworkAutomationTestSuiteFixture::TearDown();
}

FReplicationSystemTestClient* FReplicationSystemServerClientTestFixture::CreateClient()
{
	FReplicationSystemTestClient* Client = new FReplicationSystemTestClient(GetName());
	Clients.Add(Client);

	// The client needs a connection
	Client->LocalConnectionId = Client->AddConnection();

	// Auto connect to server
	Client->ConnectionIdOnServer = Server->AddConnection();

	return Client;
}

void FReplicationSystemServerClientTestFixture::DestroyClient(FReplicationSystemTestClient* Client)
{
	if (!Clients.Remove(Client))
	{
		UE_LOGF(LogIris, Warning, "Unable to find FReplicationSystemTestClient %p for destroy. NOT destroying.", Client);
		return;
	}

	Server->RemoveConnection(Client->ConnectionIdOnServer);

	delete Client;
}

}
