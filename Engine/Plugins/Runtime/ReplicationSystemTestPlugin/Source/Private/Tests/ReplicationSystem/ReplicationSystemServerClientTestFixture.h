// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "ReplicatedTestObject.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "HAL/PlatformMath.h"

#include "Iris/DataStream/DataStreamDefinitions.h"
#include "Iris/DataStream/DataStreamManager.h"
#include "Iris/ReplicationSystem/NetTokenStore.h"
#include "Iris/ReplicationSystem/NetTokenDataStream.h"
#include "Iris/ReplicationSystem/ReplicationDataStream.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"

#include "Net/Core/Misc/ResizableCircularQueue.h"
#include "Misc/Optional.h"

namespace UE::Net
{

// Utility class to use DataStreams in test code
class FDataStreamTestUtil
{
public:
	FDataStreamTestUtil();

	void SetUp();
	void TearDown();

	class UMyDataStreamDefinitions : public UDataStreamDefinitions
	{
	public:
		void FixupDefinitions() { return UDataStreamDefinitions::FixupDefinitions(); }
	};

	void StoreDataStreamDefinitions();
	void RestoreDataStreamDefinitions();
	struct FAddDataStreamDefinitionParams
	{
		bool bAutoCreate = false;
		bool bDynamicCreate = false;
		bool bValid = true;
	};

	void AddDataStreamDefinition(const TCHAR* StreamName, const TCHAR* ClassPath, const FAddDataStreamDefinitionParams& Params);
	void AddDataStreamDefinition(const TCHAR* StreamName, const TCHAR* ClassPath) { AddDataStreamDefinition(StreamName, ClassPath, FAddDataStreamDefinitionParams()); }
	void FixupDefinitions() { DataStreamDefinitions->FixupDefinitions(); }

protected:
	UMyDataStreamDefinitions* DataStreamDefinitions;
	TArray<FDataStreamDefinition>* CurrentDataStreamDefinitions;
	TArray<FDataStreamDefinition> PreviousDataStreamDefinitions;
	bool* PointerToFixupComplete;
};

class FNetTokenDataStoreTestUtil
{
public:
	void SetUp();
	void TearDown();
	void StoreNetTokenStoreConfig();
	void RestoreNetTokenStoreConfig();
	void AddNetTokenStoreTypeIdPair(FString StoreTypeName, uint32 TypeID);

private:
	UNetTokenTypeIdConfig* NetTokenTypeIdConfig = nullptr;
	TArray<FNetTokenStoreTypeIdPair> OriginalReservedTypeIds;
};

// Implements everything we need to drive the replication system to test the system
class FReplicationSystemTestNode
{
public:
	uint32 MaxSendPacketSize = 2048U;

	struct FPacketData
	{
		enum { MaxPacketSize = 2048U };
		
		alignas(16) uint8 PacketBuffer[MaxPacketSize];
		uint32 BitCount;
		uint32 PacketId;
		FString Desc;
	};

	struct FConnectionInfo
	{
		UDataStreamManager* DataStreamManager = nullptr;
		TResizableCircularQueue<const FDataStreamRecord*> WriteRecords;
		TResizableCircularQueue<FPacketData> WrittenPackets;
		uint32 ConnectionId = 0;
	};

	struct FReplicationSystemParamsOverride
	{
		uint32 MaxReplicatedObjectCount = 0;
		uint32 InitialNetObjectListCount = 0;
		uint32 NetObjectListGrowCount = 0;
		bool bUseRemoteObjectReferences = false;
		bool bAllowParallelTasks = false;
		TOptional<bool> bAllowMinimalUpdateIfNoConnections;
		uint32 InitialNetRefHandleIndex = 1;
		UE::Net::EProxyType ProxyType = UE::Net::EProxyType::None;
	};

	enum EDelaySetup { DelaySetup=0 };

public:
	FReplicationSystemTestNode(bool bIsServer, const TCHAR* Name, const FReplicationSystemTestNode::FReplicationSystemParamsOverride* ParamsOverride = nullptr);
	explicit FReplicationSystemTestNode(FReplicationSystemTestNode::EDelaySetup);
	virtual ~FReplicationSystemTestNode();

	void Setup(bool bIsServer, const TCHAR* Name, const FReplicationSystemTestNode::FReplicationSystemParamsOverride* ParamsOverride=nullptr);

	template<typename T>
	T* CreateObject()
	{
		T* CreatedObject = NewObject<T>();
		if (Cast<UReplicatedTestObject>(CreatedObject))
		{
			CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedObject));

			// Add it to the bridge for replication
			ReplicationBridge->BeginReplication(CreatedObject);
	
			return CreatedObject;
		}
		return nullptr;
	}

	template<typename T>
	T* CreateStablyNamedObject(const TCHAR* ObjectName)
	{
		check(ReplicationSystem);
		if (!ReplicationSystem)
		{
			return nullptr;
		}

		T* CreatedObject = NewObject<T>(ReplicationSystem->GetReplicationBridge(), *FString::Printf(TEXT("%s"), ObjectName));

		if (UReplicatedTestObject* ReplicatedTestObject = Cast<UReplicatedTestObject>(CreatedObject))
		{
			ReplicatedTestObject->bForceStablyNamed = true;
			CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedObject));

			if (ReplicationSystem && ReplicationSystem->IsServer())
			{
				// Add it to the bridge for replication
				ReplicationBridge->BeginReplication(CreatedObject);
			}
	
			return CreatedObject;
		}
		return nullptr;
	}

	template<typename T>
	T* CreateObjectWithParams(const UObjectReplicationBridge::FRootObjectReplicationParams& Params)
	{
		T* CreatedObject = NewObject<T>();
		if (Cast<UReplicatedTestObject>(CreatedObject))
		{
			CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedObject));

			// Add it to the bridge for replication
			ReplicationBridge->BeginReplication(CreatedObject, Params);
	
			return CreatedObject;
		}
		return nullptr;
	}

	template<typename T>
	T* CreateSubObject(FNetRefHandle OwnerHandle)
	{
		T* CreatedSubObject = NewObject<T>();
		if (Cast<UReplicatedTestObject>(CreatedSubObject))
		{
			CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedSubObject));

			// Add it to the bridge for replication
			ReplicationBridge->BeginReplication(OwnerHandle, CreatedSubObject);
	
			return CreatedSubObject;
		}
		return nullptr;
	}

	template<typename T>
	T* CreateSubObjectWithParams(const UObjectReplicationBridge::FSubObjectReplicationParams& SubObjectParams)
	{
		T* CreatedSubObject = NewObject<T>();
		if (Cast<UReplicatedTestObject>(CreatedSubObject))
		{
			CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedSubObject));

			// Add it to the bridge for replication
			ReplicationBridge->BeginReplication(CreatedSubObject, SubObjectParams);
	
			return CreatedSubObject;
		}
		return nullptr;
	}

	template <typename T>
	T* GetObjectAs(FNetRefHandle Handle)
	{
		return static_cast<T*>(ReplicationBridge->GetReplicatedObject(Handle));
	}

	UTestReplicatedIrisObject* CreateObject(const UObjectReplicationBridge::FRootObjectReplicationParams& Params, UTestReplicatedIrisObject::FComponents* ComponentsToCreate=nullptr);
	UTestReplicatedIrisObject* CreateObject(uint32 NumComponents, uint32 NumIrisComponents);
	UTestReplicatedIrisObject* CreateSubObject(FNetRefHandle Owner, uint32 NumComponents, uint32 NumIrisComponents);
	UTestReplicatedIrisObject* CreateObject(const UTestReplicatedIrisObject::FComponents& Components = UTestReplicatedIrisObject::FComponents());
	UTestReplicatedIrisObject* CreateSubObject(FNetRefHandle Owner, const UTestReplicatedIrisObject::FComponents& Components = UTestReplicatedIrisObject::FComponents());
	UTestReplicatedIrisObject* CreateObjectWithDynamicState(uint32 NumComponents, uint32 NumIrisComponents, uint32 NumDynamicStateComponents);

	void DestroyObject(UReplicatedTestObject*, EEndReplicationFlags EndReplicationFlags = EEndReplicationFlags::Destroy);

	// Returns true if the NetRefHandle can still be resolved to an object.
	bool IsResolvableNetRefHandle(FNetRefHandle RefHandle);

	// Returns true if the NetRefHandleManager considers the NetRefHandle valid.
	bool IsValidNetRefHandle(FNetRefHandle RefHandle);

	// Connection
	uint32 AddConnection();
	void RemoveConnection(uint32 ConnectionId);

	// System Update
	void NetUpdate();
	void TickPostReceive();

	// Update methods for server connections
	bool SendUpdate(uint32 ConnectionId, const TCHAR* Desc = nullptr);
	bool SendUpdate(const TCHAR* Desc = nullptr) { return SendUpdate(1, Desc); }

	void PostSendUpdate();

	void DeliverTo(FReplicationSystemTestNode& Dest, uint32 LocalConnectionId, uint32 RemoteConnectionId, bool bDeliver);
	void RecvUpdate(uint32 ConnectionId, FNetSerializationContext& Context);
	void RecvUpdate(FNetSerializationContext& Context) { RecvUpdate(1, Context); }

	UReplicatedTestObjectBridge* GetReplicationBridge() { return ReplicationBridge; }
	UReplicationSystem* GetReplicationSystem() { return ReplicationSystem; }
	uint32 GetReplicationSystemId() const;

	void SetMaxSendPacketSize(uint32 InMaxSendPacketSize) { MaxSendPacketSize = InMaxSendPacketSize; }

	FConnectionInfo& GetConnectionInfo(uint32 ConnectionId) { return Connections[ConnectionId - 1]; }

	uint32 GetNetTraceId() const;

	float ConvertPollPeriodIntoFrequency(uint32 PollPeriod) const;

public:
	TUniquePtr<UE::Net::FNetTokenStore> NetTokenStore;
	UReplicationSystem* ReplicationSystem = nullptr;
	UReplicatedTestObjectBridge* ReplicationBridge = nullptr;
	TArray<TStrongObjectPtr<UObject>> CreatedObjects;
	EReplicationSystemSendPass CurrentSendPass = EReplicationSystemSendPass::Invalid;

private:
	FNetTokenDataStoreTestUtil NetTokenDataStoreUtil;
	TArray<FConnectionInfo> Connections;
	uint32 PacketId = 0U;
};

class FReplicationSystemTestClient : public FReplicationSystemTestNode
{
public:
	explicit FReplicationSystemTestClient(const TCHAR* Name, const FReplicationSystemTestNode::FReplicationSystemParamsOverride* ParamsOverride = nullptr);
	explicit FReplicationSystemTestClient(FReplicationSystemTestNode::EDelaySetup Delay) : FReplicationSystemTestNode(Delay) {}

	// Tick and send packets to the server
	bool UpdateAndSend(class FReplicationSystemTestServer* Server, bool bDeliver = true, const TCHAR* Desc = nullptr);

	uint32 ConnectionIdOnServer;
	uint32 LocalConnectionId;
};

class FReplicationSystemTestServer : public FReplicationSystemTestNode
{
public:
	explicit FReplicationSystemTestServer(const TCHAR* Name, const FReplicationSystemTestNode::FReplicationSystemParamsOverride* ParamsOverride = nullptr);
	explicit FReplicationSystemTestServer(FReplicationSystemTestNode::EDelaySetup Delay) : FReplicationSystemTestNode(Delay) {}

	// Send data and deliver to the client if bDeliver is true
	bool SendAndDeliverTo(FReplicationSystemTestClient* Client, bool bDeliver, const TCHAR* Desc = nullptr);

	// Send data, return true if data was written
	bool SendTo(FReplicationSystemTestClient* Client, const TCHAR* Desc = nullptr);

	// Explicitly set delivery status
	void DeliverTo(FReplicationSystemTestClient* Client, bool bDeliver);

	// Tick and send packets to one or all clients
	bool UpdateAndSend(const TArrayView<FReplicationSystemTestClient*const>& Clients, bool bDeliver=true, const TCHAR* Desc = nullptr);

	// If set, automatically call NetUpdate and PostSendUpdate after each deliverd packet to simulate a client tick.
	bool bUpdateClientOnDelivery = true;
};

class FReplicationSystemServerClientTestFixture : public FNetworkAutomationTestSuiteFixture
{
public:
	FReplicationSystemServerClientTestFixture() : FNetworkAutomationTestSuiteFixture() {}

protected:
	enum : bool
	{
		DoNotDeliverPacket = false,
		DeliverPacket = true,
	};

	virtual void SetUp() override;
	virtual void TearDown() override;

	FReplicationSystemTestClient* CreateClient();
	void DestroyClient(FReplicationSystemTestClient* Client);

	FDataStreamTestUtil DataStreamUtil;
	FNetTokenDataStoreTestUtil NetTokenDataStoreUtil;
	FReplicationSystemTestServer* Server;
	TArray<FReplicationSystemTestClient*> Clients;
	FReplicationSystemTestNode::FReplicationSystemParamsOverride ServerParamsOverride;
};

}
