// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "ReplicatedTestObject.h"
#include "ReplicationSystemServerClientTestFixture.h"
#include "Containers/Array.h"
#include "Templates/UniquePtr.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationSystem/ReplicationProtocolManager.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/StringTokenStore.h"
#include "Iris/ReplicationSystem/NameTokenStore.h"
#include "Net/Core/NetToken/NetToken.h"

namespace UE::Net
{

/** Simple fixture that spins up a ReplicationSystem and manages creation of UTestReplicatedIrisObjects */
class FReplicationSystemTestFixture : public FNetworkAutomationTestSuiteFixture
{
public:
	FReplicationSystemTestFixture()
	: FNetworkAutomationTestSuiteFixture()
	, ReplicationBridge(nullptr)
	{
	}

protected:
	virtual void SetUp() override
	{
		const bool bIsServer = true;
		// Init NetTokenStore
		{
			using namespace UE::Net;

			NetTokenDataStoreUtil.SetUp();
			
			NetTokenStore = MakeUnique<FNetTokenStore>();

			FNetTokenStore::FInitParams NetTokenStoreInitParams;
			NetTokenStoreInitParams.Authority = bIsServer ? FNetToken::ENetTokenAuthority::Authority : FNetToken::ENetTokenAuthority::None;
			NetTokenStore->Init(NetTokenStoreInitParams);

			// Register data stores for supported types, $TODO: make this configurable.
			NetTokenStore->CreateAndRegisterDataStore<FStringTokenStore>();
			NetTokenStore->CreateAndRegisterDataStore<FNameTokenStore>();
		}

		ReplicationBridge = NewObject<UReplicatedTestObjectBridge>();
		CreatedObjects.Add(TStrongObjectPtr<UObject>(ReplicationBridge));

		UReplicationSystem::FReplicationSystemParams Params;
		Params.ReplicationBridge = ReplicationBridge;
		Params.bIsServer = bIsServer;
		Params.bAllowObjectReplication = true;
		Params.NetTokenStore =  NetTokenStore.Get();

		// The types of tests using this type of fixture expects a full update
		Params.bAllowMinimalUpdateIfNoConnections = false;

		// In a testing environment without configs the creation of the ReplicationSystem can be quite spammy
		ELogVerbosity::Type IrisLogVerbosity = UE_GET_LOG_VERBOSITY(LogIris);
		LogIris.SetVerbosity(ELogVerbosity::Error);
		ReplicationSystem = FReplicationSystemFactory::CreateReplicationSystem(Params);
		LogIris.SetVerbosity(IrisLogVerbosity);

		UE_NET_ASSERT_NE(ReplicationBridge, nullptr);
	}

	virtual void TearDown() override
	{
		const ELogVerbosity::Type IrisLogVerbosity = UE_GET_LOG_VERBOSITY(LogIris);
		LogIris.SetVerbosity(ELogVerbosity::Error);
		FReplicationSystemFactory::DestroyReplicationSystem(ReplicationSystem);
		LogIris.SetVerbosity(IrisLogVerbosity);
		CreatedObjects.Empty();
		NetTokenDataStoreUtil.TearDown();
	}

	// Creates a test object without components
	UTestReplicatedIrisObject* CreateObject()
	{
		UTestReplicatedIrisObject* CreatedObject = NewObject<UTestReplicatedIrisObject>();
		CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedObject));
		return CreatedObject;
	}

	void DestroyObject(UObject* Object)
	{
		CreatedObjects.Remove(TStrongObjectPtr<UObject>(Object));
		Object->MarkAsGarbage();
	}

	// Creates an object of a specific type. Only ReplicatedTestObject derived classes are supported.
	template<typename T>
	T* CreateObject()
	{
		T* CreatedObject = NewObject<T>();
		if (Cast<UReplicatedTestObject>(CreatedObject))
		{
			CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedObject));
			return CreatedObject;
		}

		return nullptr;
	}

	// Creates a test object without the specified number of property and native Iris components
	UTestReplicatedIrisObject* CreateObject(uint32 NumPropertyComponents, uint32 NumIrisComponents)
	{
		UTestReplicatedIrisObject* CreatedObject = NewObject<UTestReplicatedIrisObject>();
		CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedObject));

		CreatedObject->AddComponents(NumPropertyComponents, NumIrisComponents);
	
		return CreatedObject;
	}

	// Creates a test object without the specified number of property, native Iris and dynamic state components
	UTestReplicatedIrisObject* CreateObjectWithDynamicState(uint32 NumPropertyComponents, uint32 NumIrisComponents, uint32 NumDynamicStateComponents)
	{
		UTestReplicatedIrisObject* CreatedObject = NewObject<UTestReplicatedIrisObject>();
		CreatedObjects.Add(TStrongObjectPtr<UObject>(CreatedObject));

		CreatedObject->AddComponents(NumPropertyComponents, NumIrisComponents);
		CreatedObject->AddDynamicStateComponents(NumDynamicStateComponents);
	
		return CreatedObject;
	}

	FNetTokenDataStoreTestUtil NetTokenDataStoreUtil;
	TUniquePtr<UE::Net::FNetTokenStore> NetTokenStore;
	UReplicationSystem* ReplicationSystem;
	UReplicatedTestObjectBridge* ReplicationBridge;
	
	TArray<TStrongObjectPtr<UObject>> CreatedObjects;
};

}
