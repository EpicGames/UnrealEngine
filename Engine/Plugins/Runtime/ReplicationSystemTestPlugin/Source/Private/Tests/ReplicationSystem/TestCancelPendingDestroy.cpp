// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSystemServerClientTestFixture.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Tests/EnsureScope.h"

namespace UE::Net::Private
{

class FTestCancelPendingDestroyFixture : public FReplicationSystemServerClientTestFixture
{
};

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelPendingDestroyDuringWaitOnCreateConfirmationWithoutPacketLoss)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());

	// Write packet
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Filter out object to cause a PendingDestroy
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Remove object from filter to cause object to end up in CancelPendingDestroy
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Deliver original packet
	Server->DeliverTo(Client, DeliverPacket);

	// Verify that the object now exists on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));

	// Modify a property on the object and make sure it's replicated as the object should now be confirmed created
	ServerObject->IntA ^= 1;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelPendingDestroyDuringWaitOnCreateConfirmationWithPacketLoss)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());

	// Write packet
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Filter out object to cause a PendingDestroy
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Remove object from filter to cause object to end up in CancelPendingDestroy
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Cause packet loss on object creation
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// Write and send packet and verify object is created
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelPendingDestroyDuringWaitOnCreateConfirmationWithPacketLossNotifyBeforeCancel)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());

	// Write packet
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Filter out object to cause a PendingDestroy
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Notify packet loss, which should set state to Invalid/not replicated
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// Remove object from filter to cause object to end up in PendingCreate
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Write and send packet and verify object is created
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelPendingDestroyDuringCreated)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the object now also exists on the client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));

	// Filter out object to cause a PendingDestroy
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Remove object from filter to cause object to end up in CancelPendingDestroy
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Modify a property on the object and make sure it's replicated as the object should still be created
	ServerObject->IntA ^= 1;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelPendingDestroyDuringWaitOnDestroyConfirmationWithPacketLoss)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Filter out object and write packet to cause object to end up in WaitOnDestroyConfirmation
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Remove object from filter to cause object to end up in CancelPendingDestroy
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Cause packet loss on the object destroy packet
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// Verify that object still exists on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));

	// Modify a property on the object and make sure it's replicated as the object should still be created
	ServerObject->IntA ^= 1;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelPendingDestroyDuringWaitOnDestroyConfirmationWithPacketLossNotifyBeforeCancel)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Filter out object and write packet to cause object to end up in WaitOnDestroyConfirmation
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Cause packet loss on the object destroy packet
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// Remove object from filter to cause object to end up in CancelPendingDestroy
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Verify that object still exists on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));

	// Modify a property on the object and make sure it's replicated as the object should still be created
	ServerObject->IntA ^= 1;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
}


UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelPendingDestroyDuringWaitOnDestroyConfirmationWithInitialPacketLoss)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());

	// Introduce latency by not immediately delivering packets.
	Server->NetUpdate();
	Server->SendTo(Client, TEXT("Create object"));
	Server->PostSendUpdate();

	// Filter out object to cause it to end up being PendingDestroy
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);

	Server->NetUpdate();
	Server->SendTo(Client, TEXT("Destroy object"));
	Server->PostSendUpdate();

	// Remove object from filter to cause object to not destroy it. 
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);

	// Modify object.
	ServerObject->IntA += 1;

	Server->NetUpdate();
	Server->SendTo(Client, TEXT("Update object"));
	Server->PostSendUpdate();

	// Drop the initial packet.
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// Deliver remaining packets, if any.
	{
		SIZE_T PacketCount = 0;
		const auto& ConnectionInfo = Server->GetConnectionInfo(Client->ConnectionIdOnServer);
		PacketCount = ConnectionInfo.WrittenPackets.Count();
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, DeliverPacket);
		}
	}

	Server->UpdateAndSend({ Client });

	// The object should end up being created with the latest state.
	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelPendingDestroyDuringWaitOnDestroyConfirmationWithoutPacketLoss)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Filter out object and write packet to cause object to end up in WaitOnDestroyConfirmation
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Remove object from filter to cause object to end up in CancelPendingDestroy, also modify a property 
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	ServerObject->IntA ^= 1;
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Deliver object destroy packet
	Server->DeliverTo(Client, DeliverPacket);

	// The destroy was delivered so the client object should not exist
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerObject->NetRefHandle));

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
}

// Tests for canceling a canceled pending destroy
UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelingCancelPendingDestroyDuringWaitOnCreateConfirmationWithoutPacketLoss)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());

	// Write packet
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Filter out object to cause a PendingDestroy
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Remove object from filter to cause object to end up in CancelPendingDestroy...
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// ... and cancel that thought! We want the object to be destroyed after all.
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	Server->DeliverTo(Client, DeliverPacket);

	// Verify that the object now exists on client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Write and send a packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the object was destroyed on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelingCancelPendingDestroyDuringWaitOnCreateConfirmationWithPacketLoss)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());

	// Write packet
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Filter out object to cause a PendingDestroy
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Remove object from filter to cause object to end up in CancelPendingDestroy...
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// ... and cancel that thought! We want the object to be destroyed after all.
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Cause packet loss on object creation
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// Verify that the object doesn't exist on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Write and send a packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the object still doesn't exist on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelingCancelPendingDestroyDuringCreated)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the object now also exists on the client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Filter out object to cause a PendingDestroy
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Remove object from filter to cause object to end up in CancelPendingDestroy...
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// ... and cancel that thought! We want the object to be destroyed after all.
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Write and send a packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the object was destroyed on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelingCancelPendingDestroyDuringWaitOnDestroyConfirmationWithPacketLoss)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Filter out object and write packet to cause object to end up in WaitOnDestroyConfirmation
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Remove object from filter to cause object to end up in CancelPendingDestroy...
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// ... and cancel that thought! We want the object to be destroyed after all.
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Cause packet loss on the object destroy packet
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// Verify that the object still exists on client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Write and send a packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the object was destroyed on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelingCancelPendingDestroyDuringWaitOnDestroyConfirmationWithoutPacketLoss)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Filter out object and write packet to cause object to end up in WaitOnDestroyConfirmation
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Remove object from filter to cause object to end up in CancelPendingDestroy...
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// ... and cancel that thought! We want the object to be destroyed after all.
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Deliver destroy packet
	Server->DeliverTo(Client, DeliverPacket);

	// Verify that the object doesn't exist on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Write and send a packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the object still doesn't exist on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}


// Subobject tests
UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelPendingSubObjectDestroyDuringCreated)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with subobject on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the subobject exists on the client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);

	// Filter out subobject to cause a SubObjectPendingDestroy
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Remove subobject from filter to cause object to end up in CancelPendingDestroy
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Modify a property on the subobject and make sure it's replicated as the subobject should still exist
	ServerSubObject->IntA ^= 1;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientSubObject, nullptr);
	UE_NET_ASSERT_EQ(ClientSubObject->IntA, ServerSubObject->IntA);
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestDestroySubObjectWhenRootGoesOutOfAndInToScope)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with subobject on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());
	const FNetRefHandle ObjectRefHandle = ServerObject->NetRefHandle;
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents());
	const FNetRefHandle SubObjectRefHandle = ServerSubObject->NetRefHandle;

	// Write and send packet
	Server->UpdateAndSend({Client});

	// Verify that the subobject exists on the client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(SubObjectRefHandle));

	// Destroy subobject to cause a SubObjectPendingDestroy
	Server->DestroyObject(ServerSubObject);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Filter out root object and write packet. This causes subobject to go from SubObjectPendingDestroy to PendingDestroy.
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Lose packet
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// Remove root object from NotReplicated group to have it end up in CancelPendingDestroy
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);

	// Update and deliver packet
	Server->UpdateAndSend({Client});

	// The subobject should not exist
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(SubObjectRefHandle));
	// The root object should still exist
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ObjectRefHandle));
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelPendingSubObjectDestroyDuringWaitOnDestroyConfirmationWithPacketLoss)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Filter out object and write packet to cause subobject to end up in WaitOnDestroyConfirmation
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Remove subobject from filter to cause object to end up in CancelPendingDestroy
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Cause packet loss on the subobject destroy packet
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// Verify that object still exists on client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);

	// Modify a property on the subobject and make sure it's replicated as the subobject should still exist
	ServerSubObject->IntA ^= 1;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientSubObject, nullptr);
	UE_NET_ASSERT_EQ(ClientSubObject->IntA, ServerSubObject->IntA);
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelPendingSubObjectDestroyDuringWaitOnDestroyConfirmationWithPacketLossNotifyBeforeCancel)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Filter out object and write packet to cause subobject to end up in WaitOnDestroyConfirmation
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Cause packet loss on the subobject destroy packet
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// Remove subobject from filter to cause object to end up in CancelPendingDestroy
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Verify that object still exists on client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);

	// Modify a property on the subobject and make sure it's replicated as the subobject should still exist
	ServerSubObject->IntA ^= 1;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientSubObject, nullptr);
	UE_NET_ASSERT_EQ(ClientSubObject->IntA, ServerSubObject->IntA);
}


UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelPendingSubObjectDestroyDuringWaitOnDestroyConfirmationWithoutPacketLoss)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Filter out object and write packet to cause subobject to end up in WaitOnDestroyConfirmation
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Remove object from filter to cause object to end up in CancelPendingDestroy, also modify a property 
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	ServerSubObject->IntA ^= 1;
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Deliver subobject destroy packet
	Server->DeliverTo(Client, DeliverPacket);

	// The destroy was delivered so the client subobject should not exist
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);

	// Write and send a packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Client subobject should have been created with the latest state
	UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientSubObject, nullptr);
	UE_NET_ASSERT_EQ(ClientSubObject->IntA, ServerSubObject->IntA);
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelingCancelPendingSubObjectDestroyDuringCreated)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the subobject now also exists on the client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);

	// Filter out object to cause a SubObjectPendingDestroy
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Remove object from filter to cause object to end up in CancelPendingDestroy...
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// ... and cancel that thought! We want the subobject to be destroyed after all.
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Write and send a packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the subobject was destroyed on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelPendingDestroyFromCreatedDoesNotMissChanges)
{
	// Add a client
	FReplicationSystemTestClient* Client0 = CreateClient();

	// Add second client, with not filtering to keep object in scope
	FReplicationSystemTestClient* Client1 = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client0, DeliverPacket);
	Server->SendAndDeliverTo(Client1, DeliverPacket);
	Server->PostSendUpdate();

	UTestReplicatedIrisObject* Client0Object = Cast<UTestReplicatedIrisObject>(Client0->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UTestReplicatedIrisObject* Client1Object = Cast<UTestReplicatedIrisObject>(Client1->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Verify that the object was replicated to all clients
	UE_NET_ASSERT_NE(Client0Object, nullptr);
	UE_NET_ASSERT_EQ(Client0Object->IntA, ServerObject->IntA);
	UE_NET_ASSERT_NE(Client1Object, nullptr);
	UE_NET_ASSERT_EQ(Client1Object->IntA, ServerObject->IntA);

	// Filter out object to cause a PendingDestroy for Client0
	FNetObjectGroupHandle ExclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(ExclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(ExclusionGroupHandle);

	Server->ReplicationSystem->SetGroupFilterStatus(ExclusionGroupHandle, Client0->ConnectionIdOnServer, ENetFilterStatus::Disallow);
	Server->ReplicationSystem->SetGroupFilterStatus(ExclusionGroupHandle, Client1->ConnectionIdOnServer, ENetFilterStatus::Allow);
	
	Server->NetUpdate();
	Server->SendTo(Client0);
	Server->PostSendUpdate();

	// Mark dirty
	ServerObject->IntA = 3;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client1, DeliverPacket);
	Server->PostSendUpdate();

	// Allow replication again, to trigger cancel
	Server->ReplicationSystem->SetGroupFilterStatus(ExclusionGroupHandle, Client0->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Drop the packet with the destroy
	Server->DeliverTo(Client0, false);

	// Write and send a packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client0, DeliverPacket);
	Server->SendAndDeliverTo(Client1, DeliverPacket);
	Server->PostSendUpdate();

	Client0Object = Cast<UTestReplicatedIrisObject>(Client0->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	Client1Object = Cast<UTestReplicatedIrisObject>(Client1->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Verify that the cancel worked and did not miss changed state
	UE_NET_ASSERT_NE(Client0Object, nullptr);
	UE_NET_ASSERT_EQ(Client0Object->IntA, ServerObject->IntA);
	UE_NET_ASSERT_NE(Client1Object, nullptr);
	UE_NET_ASSERT_EQ(Client1Object->IntA, ServerObject->IntA);
}


UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelingCancelPendingSubObjectDestroyDuringWaitOnDestroyConfirmationWithPacketLoss)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Filter out object and write packet to cause subobject to end up in WaitOnDestroyConfirmation
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Remove object from filter to cause subobject to end up in CancelPendingDestroy...
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// ... and cancel that thought! We want the subobject to be destroyed after all.
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Cause packet loss on the subobject destroy packet
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// Verify that the object still exists on client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);

	// Write and send a packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the subobject was destroyed on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelingCancelPendingSubObjectDestroyDuringWaitOnDestroyConfirmationWithoutPacketLoss)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Filter out subobject and write packet to cause subobject to end up in WaitOnDestroyConfirmation
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Remove subobject from filter to cause subobject to end up in CancelPendingDestroy...
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// ... and cancel that thought! We want the subobject to be destroyed after all.
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Deliver destroy packet
	Server->DeliverTo(Client, DeliverPacket);

	// Verify that the subobject doesn't exist on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);

	// Write and send a packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the subobject still doesn't exist on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelRootObjectDestroyAfterSubObjectDestroy)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents());

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the subobject now also exists on the client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);

	// Filter out subobject to cause a SubObjectPendingDestroy
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerSubObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Filter out root object to cause subobject to move to PendingDestroy instead
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// ... and cancel that thought! We don't want the object to be destroyed after all.
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Write and send a packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the subobject was destroyed on the client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestCancelPendingDestroyFixture, TestCancelPendingDestroyDuringWaitOnDestroyConfirmationDoesNotTriggerEnsure)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn temp object on server to reserver low internal index
	UTestReplicatedIrisObject* ServerTempObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());
	
	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents());
	
	// Destroy temp object and perfom an update tick to allow the internal index to be reused
	{
		Server->DestroyObject(ServerTempObject);
		ServerTempObject = nullptr;
		Server->UpdateAndSend({});
	}

	// Write and send packet
	Server->UpdateAndSend({Client});

	// Filter out object and write packet to cause object to end up in WaitOnDestroyConfirmation
	FNetObjectGroupHandle ExclusionGroupHandle = Server->GetReplicationSystem()->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(ExclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(ExclusionGroupHandle);
	Server->ReplicationSystem->SetGroupFilterStatus(ExclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Disallow);

	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Create subobject on server
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, UTestReplicatedIrisObject::FComponents());

	// Allow replication again to have the object end up in CancelPendingDestroy. Assert that no ensure was triggered.
	{
		FEnsureScope EnsureScope;

		Server->ReplicationSystem->SetGroupFilterStatus(ExclusionGroupHandle, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);
		Server->NetUpdate();
		Server->PostSendUpdate();

		UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 0);
	}
}

// See TestObjectSplitting.cpp for cancel pending destroy on huge object

}
