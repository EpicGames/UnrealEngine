// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSystemServerClientTestFixture.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilterDefinitions.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Tests/ReplicationSystem/Filtering/MockNetObjectFilter.h"
#include "NetBlob/PartialNetBlobTestFixture.h"

namespace UE::Net::Private
{
	
// Use this fixture for all tests in this file regardless of whether the test requires it or not. That makes it easy to only run or never run these tests.
class FTestFilteredDependentObjectFixture : public FReplicationSystemServerClientTestFixture
{
protected:
	virtual void SetUp() override
	{
		InitFilterDefinitions();
		FReplicationSystemServerClientTestFixture::SetUp();
		InitFilterHandles();
	}

	virtual void TearDown() override
	{
		FReplicationSystemServerClientTestFixture::TearDown();
		RestoreFilterDefinitions();
	}

	void SetMockFilterStatus(UE::Net::ENetFilterStatus FilterStatus)
	{
		UMockNetObjectFilter::FFunctionCallSetup CallSetup;
		CallSetup.AddObject.bReturnValue = true;
		CallSetup.Filter.bFilterOutByDefault = FilterStatus == UE::Net::ENetFilterStatus::Disallow;
		MockFilter->SetFunctionCallSetup(CallSetup);
	}

	FNetObjectFilterHandle NotRoutedFilterHandle = InvalidNetObjectFilterHandle;
	FNetObjectFilterHandle MockFilterHandle = InvalidNetObjectFilterHandle;
	TObjectPtr<UMockNetObjectFilter> MockFilter = nullptr;

private:
	void InitFilterDefinitions()
	{
		const UClass* NetObjectFilterDefinitionsClass = UNetObjectFilterDefinitions::StaticClass();
		const FProperty* DefinitionsProperty = NetObjectFilterDefinitionsClass->FindPropertyByName("NetObjectFilterDefinitions");
		check(DefinitionsProperty != nullptr);

		// Save CDO state.
		UNetObjectFilterDefinitions* FilterDefinitions = GetMutableDefault<UNetObjectFilterDefinitions>();
		DefinitionsProperty->CopyCompleteValue(&OriginalFilterDefinitions, (void*)(UPTRINT(FilterDefinitions) + DefinitionsProperty->GetOffset_ForInternal()));

		// Modify definitions to only include our filters. 
		TArray<FNetObjectFilterDefinition> NewFilterDefinitions;
		{
			FNetObjectFilterDefinition& NotRoutedDefinition = NewFilterDefinitions.Emplace_GetRef();
			NotRoutedDefinition.FilterName = "NotRouted";
			NotRoutedDefinition.ClassName = "/Script/IrisCore.FilterOutNetObjectFilter";
			NotRoutedDefinition.ConfigClassName = "/Script/IrisCore.NetObjectGridFilterConfig";
		}

		{
			FNetObjectFilterDefinition& MockDefinition = NewFilterDefinitions.Emplace_GetRef();
			MockDefinition.FilterName = "MockFilter";
			MockDefinition.ClassName = "/Script/ReplicationSystemTestPlugin.MockNetObjectFilter";
			MockDefinition.ConfigClassName = "/Script/ReplicationSystemTestPlugin.MockNetObjectFilterConfig";
		}

		DefinitionsProperty->CopyCompleteValue((void*)(UPTRINT(FilterDefinitions) + DefinitionsProperty->GetOffset_ForInternal()), &NewFilterDefinitions);
	}

	void RestoreFilterDefinitions()
	{
		// Restore CDO state from the saved state.
		const UClass* NetObjectFilterDefinitionsClass = UNetObjectFilterDefinitions::StaticClass();
		const FProperty* DefinitionsProperty = NetObjectFilterDefinitionsClass->FindPropertyByName("NetObjectFilterDefinitions");
		UNetObjectFilterDefinitions* FilterDefinitions = GetMutableDefault<UNetObjectFilterDefinitions>();
		DefinitionsProperty->CopyCompleteValue((void*)(UPTRINT(FilterDefinitions) + DefinitionsProperty->GetOffset_ForInternal()), &OriginalFilterDefinitions);
		OriginalFilterDefinitions.Empty();

		NotRoutedFilterHandle = InvalidNetObjectFilterHandle;
		MockFilterHandle = InvalidNetObjectFilterHandle;
		MockFilter = nullptr;
	}

	void InitFilterHandles()
	{
		NotRoutedFilterHandle = Server->GetReplicationSystem()->GetFilterHandle("NotRouted");
		MockFilterHandle = Server->GetReplicationSystem()->GetFilterHandle("MockFilter");
		MockFilter = Cast<UMockNetObjectFilter>(Server->GetReplicationSystem()->GetFilter("MockFilter"));
	}

private:
	TArray<FNetObjectFilterDefinition> OriginalFilterDefinitions;
};

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestDependentObjectDroppedDataIsRetransmitted)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);

	// Spawn second object on server add as a dependent object
	UTestReplicatedIrisObject* ServerDependentObject = Server->CreateObject(0, 0);
	
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Trigger replication
	ServerObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	Bridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);
	ServerDependentObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Store Pointer to objects
	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UTestReplicatedIrisObject* ClientDependentObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));

	UE_NET_ASSERT_NE(ClientDependentObject, nullptr);
	UE_NET_ASSERT_EQ(ServerDependentObject->IntA, ClientDependentObject->IntA);

	// Modify the value of dependent object only
	ServerDependentObject->IntA = 2;

	// Send and do not deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, false);
	Server->PostSendUpdate();

	// Verify that the final state was applied to dependent object 
	UE_NET_ASSERT_NE(ServerDependentObject->IntA, ClientDependentObject->IntA);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that the final state was applied to dependent object 
	UE_NET_ASSERT_EQ(ServerDependentObject->IntA, ClientDependentObject->IntA);
}

// Dependent objects
UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestDependentObjectWithZeroPrioOnlyReplicatesWithParent)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);

	// Filter out Server object to start with
	FNetObjectGroupHandle FilterGroup = ReplicationSystem->CreateGroup(NAME_None);
	ReplicationSystem->AddExclusionFilterGroup(FilterGroup);
	ReplicationSystem->AddToGroup(FilterGroup, ServerObject->NetRefHandle);

	// Setup dependent object to only replicate with ServerObject
	UTestReplicatedIrisObject* ServerDependentObject = Server->CreateObject(0, 0);
	ReplicationSystem->SetStaticPriority(ServerDependentObject->NetRefHandle, 0.f);
	Bridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Objects should not be created on client
	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UTestReplicatedIrisObject* ClientDependentObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));
	UE_NET_ASSERT_EQ(ClientObject, nullptr);
	UE_NET_ASSERT_EQ(ClientDependentObject, nullptr);

	// Enable 
	ReplicationSystem->SetGroupFilterStatus(FilterGroup, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Objects should now exist on client
	ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	ClientDependentObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_NE(ClientDependentObject, nullptr);
}

// Chained dependent objects
UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestChainedDependentObjectWithZeroPrio)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);

	// Filter out Server object to start with
	FNetObjectGroupHandle FilterGroup = ReplicationSystem->CreateGroup(NAME_None);
	ReplicationSystem->AddExclusionFilterGroup(FilterGroup);
	ReplicationSystem->AddToGroup(FilterGroup, ServerObject->NetRefHandle);

	// Setup dependent object to only replicate with ServerObject
	UTestReplicatedIrisObject* ServerDependentObject0 = Server->CreateObject(0, 0);
	UTestReplicatedIrisObject* ServerDependentObject1 = Server->CreateObject(0, 0);
	
	ReplicationSystem->SetStaticPriority(ServerDependentObject0->NetRefHandle, 0.f);
	ReplicationSystem->SetStaticPriority(ServerDependentObject1->NetRefHandle, 0.f);
	
	// Setup dependency chain
	Bridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject0->NetRefHandle);
	Bridge->AddDependentObject(ServerDependentObject0->NetRefHandle, ServerDependentObject1->NetRefHandle);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Objects should not be created on client
	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UTestReplicatedIrisObject* ClientDependentObject0 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject0->NetRefHandle));
	UTestReplicatedIrisObject* ClientDependentObject1 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject1->NetRefHandle));

	UE_NET_ASSERT_EQ(ClientObject, nullptr);
	UE_NET_ASSERT_EQ(ClientDependentObject0, nullptr);
	UE_NET_ASSERT_EQ(ClientDependentObject1, nullptr);

	// Enable the parent
	ReplicationSystem->SetGroupFilterStatus(FilterGroup, ENetFilterStatus::Allow);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that dependent object now is created
	ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	ClientDependentObject0 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject0->NetRefHandle));
	ClientDependentObject1 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject1->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_NE(ClientDependentObject0, nullptr);
	UE_NET_ASSERT_NE(ClientDependentObject1, nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestDependentObjectPollFrequency)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);

	// Spawn second object on server that later will be added as a dependent object
	// With high PollFramePeriod so that it will not replicate in a while unless it is a dependent
	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	Params.PollFrequency = Server->ConvertPollPeriodIntoFrequency(255U);
	UTestReplicatedIrisObject* ServerDependentObject = Server->CreateObject(Params);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Store Pointer to objects and verify state after initial replication
	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UTestReplicatedIrisObject* ClientDependentObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));
	
	UE_NET_ASSERT_NE(ClientDependentObject, nullptr);
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
	UE_NET_ASSERT_EQ(ClientDependentObject->IntA, ServerDependentObject->IntA);

	// Trigger replication

	constexpr uint32 MaxIterationCount = 256;
	uint32 It = 0;
	for (const uint32 EndIt = MaxIterationCount; It < EndIt; ++It)
	{
		ServerObject->IntA += 1;
		ServerDependentObject->IntA += 1;

		// Send and deliver packet
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, true);
		Server->PostSendUpdate();

		UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObject->IntA);

		// Verify that only the server object has been updated
		if (ClientDependentObject->IntA != ServerDependentObject->IntA)
		{
			break;
		}
	}
	// At some point the object is expected not to be polled
	UE_NET_ASSERT_LT(It, MaxIterationCount);

	// Add dependency
	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	// Change a value on owner
	ServerObject->IntA += 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// We now expect both objects to be in sync
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
	UE_NET_ASSERT_EQ(ClientDependentObject->IntA, ServerDependentObject->IntA);
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestDependentObjectPollWithDirtyParent)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UObjectReplicationBridge::FRootObjectReplicationParams Params;

	// Setup different poll frequencies for the objects
	Params.PollFrequency = Server->ConvertPollPeriodIntoFrequency(10U);
	UTestReplicatedIrisObject* ServerObject0 = Server->CreateObject(Params);

	Params.PollFrequency = Server->ConvertPollPeriodIntoFrequency(20U);
	UTestReplicatedIrisObject* ServerObject1 = Server->CreateObject(Params);

	// Spawn second object on server that later will be added as a dependent object
	Params.PollFrequency = Server->ConvertPollPeriodIntoFrequency(40U);
	UTestReplicatedIrisObject* ServerDependentObject = Server->CreateObject(Params);



	// Add dependent object to both server objects
	Server->ReplicationBridge->AddDependentObject(ServerObject0->NetRefHandle, ServerDependentObject->NetRefHandle);
	Server->ReplicationBridge->AddDependentObject(ServerObject1->NetRefHandle, ServerDependentObject->NetRefHandle);

	// Send and deliver packet, All objects are polled and replicated
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify state after initial replication
	UTestReplicatedIrisObject* ClientDependentObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));
	UTestReplicatedIrisObject* ClientObject0 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject0->NetRefHandle));
	UTestReplicatedIrisObject* ClientObject1 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject1->NetRefHandle));

	UE_NET_ASSERT_NE(ClientDependentObject, nullptr);
	UE_NET_ASSERT_NE(ClientObject0, nullptr);
	UE_NET_ASSERT_NE(ClientObject1, nullptr);
	UE_NET_ASSERT_EQ(ClientDependentObject->IntA, 0);

	const uint32 TickCount = 256;
	int PrevRcvdValue = 0;
	uint32 RepCount0 = 0U;
	uint32 RepCount1 = 0U;
	uint32 RepCountDependent = 0U;

	for (uint32 CurrentFrame = 1U; CurrentFrame < TickCount; ++CurrentFrame)
	{
		ServerObject0->IntA = CurrentFrame;
		ServerObject1->IntA = CurrentFrame;
		ServerDependentObject->IntA = CurrentFrame;

		// Send and deliver packet
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, true);
		Server->PostSendUpdate();

		const int RcvdValue = ClientDependentObject->IntA;
		if (RcvdValue != PrevRcvdValue)
		{
			if (RcvdValue == ClientObject0->IntA)
			{
				++RepCount0;
			}
			if (RcvdValue == ClientObject1->IntA)
			{
				++RepCount1;
			}
			++RepCountDependent;
		}
		PrevRcvdValue = RcvdValue;		
	}
	// We expect the dependent object to have replicated more often than the parents
	UE_NET_ASSERT_GE(RepCountDependent, RepCount0);
	UE_NET_ASSERT_GE(RepCountDependent, RepCount1);
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestDependentObjectIsPolledIfParentIsMarkedDirty)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	Params.PollFrequency = Server->ConvertPollPeriodIntoFrequency(14U);
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(Params);

	// Spawn second object add it as a dependency and bump poll period
	Params.PollFrequency = Server->ConvertPollPeriodIntoFrequency(30U);
	UTestReplicatedIrisObject* ServerDependentObject = Server->CreateObject(Params);
	Server->ReplicationBridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Store Pointer to objects and verify state after initial replication
	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UTestReplicatedIrisObject* ClientDependentObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));

	// Modify data until none of the objects are replicated, due to the set polling frequencies.
	constexpr uint32 MaxIterationCount = 32;
	uint32 It = 0;
	for (const uint32 EndIt = MaxIterationCount; It < EndIt; ++It)
	{
		ServerObject->IntA += 1;
		ServerDependentObject->IntA += 1;

		// Send and deliver packet
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, true);
		Server->PostSendUpdate();

		// Verify that nothing replicated
		if (ClientObject->IntA != ServerObject->IntA && ClientDependentObject->IntA != ServerDependentObject->IntA)
		{
			break;
		}
	}
	// If we hit this assert then it's likely the poll frequency limiter is broken.
	UE_NET_ASSERT_LT(It, MaxIterationCount);

	// Mark dependent dirty
	ReplicationSystem->ForceNetUpdate(ServerDependentObject->NetRefHandle);

	// Send and deliver packet, we expect dependent object to have replicated 
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	UE_NET_EXPECT_NE(ClientObject->IntA, ServerObject->IntA);
	// Verify that dependent object has replicated
	UE_NET_ASSERT_EQ(ClientDependentObject->IntA, ServerDependentObject->IntA);

	// Modify data
	ServerObject->IntA += 2;
	ServerDependentObject->IntA += 2;

	// Mark parent dirty
	ReplicationSystem->ForceNetUpdate(ServerObject->NetRefHandle);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that both objects have replicated
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
	UE_NET_ASSERT_EQ(ClientDependentObject->IntA, ServerDependentObject->IntA);
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestDependentObjectScheduledAfterParent)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedSubObjectOrderObject* ServerObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();

	// Spawn dependent object
	UReplicatedSubObjectOrderObject* ServerDependentObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();
	Bridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	// Set prio to only replicate with parent
	ReplicationSystem->SetStaticPriority(ServerDependentObject->NetRefHandle, 0.f);

	// Reset RepOrderCounter
	UReplicatedSubObjectOrderObject::RepOrderCounter = 0U;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that objects have replicated
	UReplicatedSubObjectOrderObject* ClientObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientDependentObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));

	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_NE(ClientDependentObject, nullptr);

	// Verify that they have replicated in expected order for initial objects
	UE_NET_ASSERT_GT(ClientDependentObject->LastRepOrderCounter, ClientObject->LastRepOrderCounter);

	// Modify both parent and dependent
	ServerObject->IntA = 1;
	ServerDependentObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that they have replicated in expected order 
	UE_NET_ASSERT_GT(ClientDependentObject->LastRepOrderCounter, ClientObject->LastRepOrderCounter);
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestDependentObjectScheduledBeforeParent)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedSubObjectOrderObject* ServerObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();

	// Spawn dependent object
	UReplicatedSubObjectOrderObject* ServerDependentObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();
	Bridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle, EDependentObjectSchedulingHint::ScheduleBeforeParent);

	// Set prio to only replicate with parent
	ReplicationSystem->SetStaticPriority(ServerDependentObject->NetRefHandle, 0.f);

	// Reset RepOrderCounter
	UReplicatedSubObjectOrderObject::RepOrderCounter = 0U;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that objects have replicated
	UReplicatedSubObjectOrderObject* ClientObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientDependentObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));

	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_NE(ClientDependentObject, nullptr);

	// Verify that they have replicated in expected order for initial objects
	UE_NET_ASSERT_LT(ClientDependentObject->LastRepOrderCounter, ClientObject->LastRepOrderCounter);

	// Modify both parent and dependent
	ServerObject->IntA = 1;
	ServerDependentObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that they have replicated in expected order
	UE_NET_ASSERT_LT(ClientDependentObject->LastRepOrderCounter, ClientObject->LastRepOrderCounter);
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestDependentObjectScheduledBeforeParentIfInitialState)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedSubObjectOrderObject* ServerObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();

	// Spawn dependent object
	UReplicatedSubObjectOrderObject* ServerDependentObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();
	Bridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle, EDependentObjectSchedulingHint::ScheduleBeforeParentIfInitialState);

	// Set prio to only replicate with parent
	ReplicationSystem->SetStaticPriority(ServerDependentObject->NetRefHandle, 0.f);

	// Reset RepOrderCounter
	UReplicatedSubObjectOrderObject::RepOrderCounter = 0U;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that objects have replicated
	UReplicatedSubObjectOrderObject* ClientObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientDependentObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));

	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_NE(ClientDependentObject, nullptr);

	// Verify that they have replicated in expected order for initial objects
	UE_NET_ASSERT_LT(ClientDependentObject->LastRepOrderCounter, ClientObject->LastRepOrderCounter);

	// Modify both parent and dependent
	ServerObject->IntA = 1;
	ServerDependentObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that they have replicated in expected order
	UE_NET_ASSERT_GT(ClientDependentObject->LastRepOrderCounter, ClientObject->LastRepOrderCounter);
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestNestedDependentObjectScheduledBeforeParent)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server, abusing some assumptions about scheduling order based on assigned internal indices to try out ordering of dependent objects
	UReplicatedSubObjectOrderObject* ServerNestedDependentObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();
	UReplicatedSubObjectOrderObject* ServerDependentObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();
	UReplicatedSubObjectOrderObject* ServerObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();

	// Setup dependent object and nested dependent object to both replicate before the parent
	Bridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle, EDependentObjectSchedulingHint::ScheduleBeforeParent);
	Bridge->AddDependentObject(ServerDependentObject->NetRefHandle, ServerNestedDependentObject->NetRefHandle, EDependentObjectSchedulingHint::ScheduleBeforeParent);

	// Set static prio to only replicate with parent
	ReplicationSystem->SetStaticPriority(ServerDependentObject->NetRefHandle, 0.f);
	ReplicationSystem->SetStaticPriority(ServerNestedDependentObject->NetRefHandle, 0.f);

	// Reset RepOrderCounter
	UReplicatedSubObjectOrderObject::RepOrderCounter = 0U;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that objects have replicated
	UReplicatedSubObjectOrderObject* ClientObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientDependentObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientNestedDependentObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerNestedDependentObject->NetRefHandle));

	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_NE(ClientDependentObject, nullptr);
	UE_NET_ASSERT_NE(ClientNestedDependentObject, nullptr);

	// Verify that they have replicated in expected order for initial objects
	UE_NET_ASSERT_LT(ClientNestedDependentObject->LastRepOrderCounter, ClientDependentObject->LastRepOrderCounter);
	UE_NET_ASSERT_LT(ClientDependentObject->LastRepOrderCounter, ClientObject->LastRepOrderCounter);

	// Modify both parent and dependent
	ServerObject->IntA = 1;
	ServerDependentObject->IntA = 1;
	ServerNestedDependentObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that they have replicated in expected order 
	UE_NET_ASSERT_LT(ClientNestedDependentObject->LastRepOrderCounter, ClientDependentObject->LastRepOrderCounter);
	UE_NET_ASSERT_LT(ClientDependentObject->LastRepOrderCounter, ClientObject->LastRepOrderCounter);
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestNestedDependentObjectScheduledBeforeParents)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server, abusing some assumptions about scheduling order based on assigned internal indices to try out ordering of dependent objects
	UReplicatedSubObjectOrderObject* ServerNestedDependentObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();
	UReplicatedSubObjectOrderObject* ServerDependentObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();
	UReplicatedSubObjectOrderObject* ServerObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();

	// Setup dependent object with a nested dependent object that is replicating before its parent
	Bridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);
	Bridge->AddDependentObject(ServerDependentObject->NetRefHandle, ServerNestedDependentObject->NetRefHandle, EDependentObjectSchedulingHint::ScheduleBeforeParent);

	// Set static prio to only replicate with parent
	ReplicationSystem->SetStaticPriority(ServerDependentObject->NetRefHandle, 0.f);
	ReplicationSystem->SetStaticPriority(ServerNestedDependentObject->NetRefHandle, 0.f);

	// Reset RepOrderCounter
	UReplicatedSubObjectOrderObject::RepOrderCounter = 0U;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that objects have replicated
	UReplicatedSubObjectOrderObject* ClientObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientDependentObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientNestedDependentObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerNestedDependentObject->NetRefHandle));

	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_NE(ClientDependentObject, nullptr);
	UE_NET_ASSERT_NE(ClientNestedDependentObject, nullptr);

	// Verify that they have replicated in expected order for initial objects
	UE_NET_ASSERT_LT(ClientNestedDependentObject->LastRepOrderCounter, ClientDependentObject->LastRepOrderCounter);
	UE_NET_ASSERT_LT(ClientObject->LastRepOrderCounter, ClientDependentObject->LastRepOrderCounter);

	// Modify both parent and dependent
	ServerObject->IntA = 1;
	ServerDependentObject->IntA = 1;
	ServerNestedDependentObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that they have replicated in expected order 
	UE_NET_ASSERT_LT(ClientNestedDependentObject->LastRepOrderCounter, ClientDependentObject->LastRepOrderCounter);
	UE_NET_ASSERT_LT(ClientObject->LastRepOrderCounter, ClientDependentObject->LastRepOrderCounter);
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestLateAddedNestedDependentObjectScheduledBeforeParents)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server, abusing some assumptions about scheduling order based on assigned internal indices to try out ordering of dependent objects
	UReplicatedSubObjectOrderObject* ServerDependentObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();
	UReplicatedSubObjectOrderObject* ServerObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();

	// Setup dependent object with a nested dependent object that is replicating before its parent
	Bridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	// Set static prio to only replicate with parent
	ReplicationSystem->SetStaticPriority(ServerDependentObject->NetRefHandle, 0.f);

	// Reset RepOrderCounter
	UReplicatedSubObjectOrderObject::RepOrderCounter = 0U;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that objects have replicated
	UReplicatedSubObjectOrderObject* ClientObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientDependentObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));

	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_NE(ClientDependentObject, nullptr);

	// Verify that they have replicated in expected order for initial objects
	UE_NET_ASSERT_LT(ClientObject->LastRepOrderCounter, ClientDependentObject->LastRepOrderCounter);

	// Create new dependent object
	UReplicatedSubObjectOrderObject* ServerNestedDependentObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();
	Bridge->AddDependentObject(ServerDependentObject->NetRefHandle, ServerNestedDependentObject->NetRefHandle, EDependentObjectSchedulingHint::ScheduleBeforeParent);
	ReplicationSystem->SetStaticPriority(ServerNestedDependentObject->NetRefHandle, 0.f);

	// Modify parent to trigger replication of new dependent object
	ServerObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that new dependent object has been created
	UReplicatedSubObjectOrderObject* ClientNestedDependentObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerNestedDependentObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientNestedDependentObject, nullptr);

	// Verify that they have replicated in expected order 
	UE_NET_ASSERT_LT(ClientNestedDependentObject->LastRepOrderCounter, ClientObject->LastRepOrderCounter);
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestLateAddedNestedDependentObjectPendingWaitOnCreateConfirmation)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UReplicatedSubObjectOrderObject* ServerSharedDependentObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();
	UReplicatedSubObjectOrderObject* ServerDependentObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();

	// Set static prio to only replicate with parent
	ReplicationSystem->SetStaticPriority(ServerDependentObject->NetRefHandle, 1.f);

	// Reset RepOrderCounter
	UReplicatedSubObjectOrderObject::RepOrderCounter = 0U;

	// Create server object
	UReplicatedSubObjectOrderObject* ServerObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();

	// Add same dependent to both ServerObject and ServerDependentObject
	Bridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);
	Bridge->AddDependentObject(ServerObject->NetRefHandle, ServerSharedDependentObject->NetRefHandle);
	Bridge->AddDependentObject(ServerDependentObject->NetRefHandle, ServerSharedDependentObject->NetRefHandle);

	// Send data
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that objects have replicated
	UReplicatedSubObjectOrderObject* ClientObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientDependentObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientSharedDependentObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSharedDependentObject->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestDependentObjectIsFilteredOutTogetherWithParent)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	UTestReplicatedIrisObject* ServerDependentObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	Bridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	// Dynamically filter out dependent object
	ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, NotRoutedFilterHandle);

	// Add parent object to MockFilter so we can filter in/out as desired.
	SetMockFilterStatus(UE::Net::ENetFilterStatus::Allow);
	ReplicationSystem->SetFilter(ServerObject->NetRefHandle, MockFilterHandle);

	// Send data
	Server->UpdateAndSend({Client});

	// Verify that objects have replicated
	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UTestReplicatedIrisObject* ClientDependentObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientDependentObject, nullptr);

	// Filter out parent object. Dependent object should be filtered out as well due to being filtered out by default.
	SetMockFilterStatus(UE::Net::ENetFilterStatus::Disallow);

	// Send data
	Server->UpdateAndSend({Client});

	// Both parent and dependent should now be filtered out.
	ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_EQ(ClientObject, nullptr);
	ClientDependentObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));
	UE_NET_ASSERT_EQ(ClientDependentObject, nullptr);

	// Restore filtering such that both objects are expected to be replicated again.
	SetMockFilterStatus(UE::Net::ENetFilterStatus::Allow);

	// Send data
	Server->UpdateAndSend({Client});

	ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	ClientDependentObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientDependentObject, nullptr);
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestDependentObjectScheduledBeforeParentWithHighPriority)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedSubObjectOrderObject* ServerObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();

	// Spawn dependent object
	UReplicatedSubObjectOrderObject* ServerDependentObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();
	Bridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle, EDependentObjectSchedulingHint::ScheduleBeforeParent);

	// Set prio to only replicate with parent
	ReplicationSystem->SetStaticPriority(ServerDependentObject->NetRefHandle, 0.f);

	// Set high prio on parent
	const float VeryHighPriority = 1.0E7f;
	ReplicationSystem->SetStaticPriority(ServerObject->NetRefHandle, VeryHighPriority);

	// Reset RepOrderCounter
	UReplicatedSubObjectOrderObject::RepOrderCounter = 0U;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that objects have replicated
	UReplicatedSubObjectOrderObject* ClientObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientDependentObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));

	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_NE(ClientDependentObject, nullptr);

	// Verify that they have replicated in expected order for initial objects
	UE_NET_ASSERT_LT(ClientDependentObject->LastRepOrderCounter, ClientObject->LastRepOrderCounter);

	// Modify both parent and dependent
	ServerObject->IntA = 1;
	ServerDependentObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that they have replicated in expected order
	UE_NET_ASSERT_LT(ClientDependentObject->LastRepOrderCounter, ClientObject->LastRepOrderCounter);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestCreationDependencyCreationOrder)
{
	//LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::VeryVerbose);
	TGuardConsoleVariable<bool> CVarOverride(TEXT("net.Iris.ScheduleCreationDependenciesFirst"), true);

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedSubObjectOrderObject* ServerObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();

	// Spawn object with a creation dependency on ServerObject
	UReplicatedSubObjectOrderObject* ServerDependentObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();

	Bridge->AddCreationDependencyLink(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	// Set low prio in parent
	ReplicationSystem->SetStaticPriority(ServerObject->NetRefHandle, 0.1f);

	// Set higher prio on dependent object to make it more important than parent, so that it will accrue scheduling priority faster
	ReplicationSystem->SetStaticPriority(ServerDependentObject->NetRefHandle, 10.f);

	// Set mini packet size to fail replication and accrue higher scheduling priorities
	Server->SetMaxSendPacketSize(16);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that objects have not replicated
	UReplicatedSubObjectOrderObject* ClientObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientDependentObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));

	UE_NET_ASSERT_EQ(ClientObject, nullptr);
	UE_NET_ASSERT_EQ(ClientDependentObject, nullptr);

	// Restore packet size
	Server->SetMaxSendPacketSize(2048);

	// Reset RepOrderCounter
	UReplicatedSubObjectOrderObject::RepOrderCounter = 0U;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that objects have replicated
	ClientObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	ClientDependentObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));

	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_NE(ClientDependentObject, nullptr);

	// Parent should have replicated before ClientDependentObject
	UE_NET_ASSERT_GT(ClientDependentObject->LastRepOrderCounter, ClientObject->LastRepOrderCounter);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestNestedCreationDependenciesCreationOrder)
{
	//LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::VeryVerbose);
	TGuardConsoleVariable<bool> CVarOverride(TEXT("net.Iris.ScheduleCreationDependenciesFirst"), true);

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedSubObjectOrderObject* ServerObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();
	ServerObject->IntA = 0U;

	TArray<UReplicatedSubObjectOrderObject*> Objects;
	Objects.Add(ServerObject);

	// Set low prio in parent
	ReplicationSystem->SetStaticPriority(ServerObject->NetRefHandle, 0.1f);

	// Spawn a bunch of dependent objects, setting prio
	const int32 DependentObjectCount = 32;

	for (uint32 ObjectIndex = 1U; ObjectIndex < DependentObjectCount; ++ObjectIndex)
	{
		UReplicatedSubObjectOrderObject* DependentObject = Server->CreateObject<UReplicatedSubObjectOrderObject>();
		DependentObject->IntA = ObjectIndex;

		// Increasing prio
		const float Prio = (float)ObjectIndex * 1.0f;
		ReplicationSystem->SetStaticPriority(DependentObject->NetRefHandle, Prio);

		// Set up a nested creation dependency chain
		Bridge->AddCreationDependencyLink(Objects.Last()->NetRefHandle, DependentObject->NetRefHandle);

		Objects.Add(DependentObject);
	}

	// Set mini packet size to fail replication and accrue higher scheduling priorities
	Server->SetMaxSendPacketSize(16);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Restore packet size.
	Server->SetMaxSendPacketSize(2048);

	const int32 StartSpawnedObjectsCount =  Client->CreatedObjects.Num();

	// Reset RepOrderCounter, important as this is a shared static.
	UReplicatedSubObjectOrderObject::RepOrderCounter = 0U;
	
	// Send packets until objects are created
	int MaxRetryCount = 32;
	do 
	{
		Server->UpdateAndSend({Client});
		--MaxRetryCount;
	}
	while ((Client->CreatedObjects.Num() < (DependentObjectCount + StartSpawnedObjectsCount)) && MaxRetryCount > 0);

	// Verify replication order (even if it is not guaranteed if objects are scheduled independently, but for the context of this test it is)
	UReplicatedSubObjectOrderObject* ClientObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	
	TArray<UReplicatedSubObjectOrderObject*> ClientObjects;
	for (UReplicatedSubObjectOrderObject* ServerDependentObject : Objects)
	{
		UReplicatedSubObjectOrderObject* ClientDependentObject = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientDependentObject, nullptr);
		UE_NET_ASSERT_EQ(ClientDependentObject->LastRepOrderCounter, (uint32)ClientDependentObject->IntA + 1);
		ClientObjects.Add(ClientDependentObject);
	}
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestBasicCreationDependencyFiltering)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn server objects
	UReplicatedTestObject* ParentObject = Server->CreateObject();
	UReplicatedTestObject* ChildObject = Server->CreateObject();
	Bridge->AddCreationDependencyLink(ParentObject->NetRefHandle, ChildObject->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Objects should now exist on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Filter out parent
	ReplicationSystem->SetFilter(ParentObject->NetRefHandle, NotRoutedFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Objects should not exist on client
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestCreationDependencyRemoveLink)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn server objects
	UReplicatedTestObject* ParentObject = Server->CreateObject();
	UReplicatedTestObject* ChildObject = Server->CreateObject();
	Bridge->AddCreationDependencyLink(ParentObject->NetRefHandle, ChildObject->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Objects should now exist on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Filter out parent
	ReplicationSystem->SetFilter(ParentObject->NetRefHandle, NotRoutedFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Objects should not exist on client
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Remove dependency link
	Bridge->RemoveCreationDependencyLink(ParentObject->NetRefHandle, ChildObject->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Child should come back into scope.
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Filter in parent
	ReplicationSystem->SetFilter(ParentObject->NetRefHandle, InvalidNetObjectFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Parent should come back into scope.
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestCreationDependencyTwoParentsRemoveOneLink)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn server objects
	UReplicatedTestObject* ParentObject = Server->CreateObject();
	UReplicatedTestObject* FilteredParentObject = Server->CreateObject();
	UReplicatedTestObject* ChildObject = Server->CreateObject();
	Bridge->AddCreationDependencyLink(ParentObject->NetRefHandle, ChildObject->NetRefHandle);
	Bridge->AddCreationDependencyLink(FilteredParentObject->NetRefHandle, ChildObject->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Objects should now exist on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(FilteredParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Filter out parent
	ReplicationSystem->SetFilter(FilteredParentObject->NetRefHandle, NotRoutedFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Parent should exist, FilteredParent and Child should not.
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(FilteredParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Remove dependency link
	Bridge->RemoveCreationDependencyLink(FilteredParentObject->NetRefHandle, ChildObject->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Child should come back into scope.
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(FilteredParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestCreationDependencyParentDestroyed)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn server objects
	UReplicatedTestObject* ParentObject = Server->CreateObject();
	UReplicatedTestObject* ChildObject = Server->CreateObject();
	Bridge->AddCreationDependencyLink(ParentObject->NetRefHandle, ChildObject->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Objects should now exist on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Filter out parent
	ReplicationSystem->SetFilter(ParentObject->NetRefHandle, NotRoutedFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Both should be filtered
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Destroy parent
	Server->DestroyObject(ParentObject);

	// Send and deliver packets
	// TODO: We update and send here twice because the base implementation of creation dependency handling
	//       doesn't bring the child back into scope until two updates have passed. The second update can be
	//       removed here once we've set bCVarRepFilterUpdateCreationDependenciesOnRelevancyChange=true by default
	//       or once we've fixed the base implementation to filter in the child object immediately.
	Server->UpdateAndSend({ Client });
	Server->UpdateAndSend({ Client });

	// Child should come back into scope.
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestNestedCreationDependencyFiltering)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn root object
	UReplicatedTestObject* RootObject = Server->CreateObject();
	TArray<UReplicatedTestObject*> Objects;
	Objects.Add(RootObject);

	// Spawn chain of dependent objects
	constexpr uint32 NumObjects = 16;
	for (uint32 ObjectIndex = 1U; ObjectIndex < NumObjects; ++ObjectIndex)
	{
		UReplicatedTestObject* ChildObject = Server->CreateObject();
		Bridge->AddCreationDependencyLink(Objects.Last()->NetRefHandle, ChildObject->NetRefHandle);
		Objects.Add(ChildObject);
	}

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Objects should all exist on client
	for (UReplicatedTestObject* Object : Objects)
	{
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(Object->NetRefHandle));
	}

	// Filter out root object
	ReplicationSystem->SetFilter(Objects[0]->NetRefHandle, NotRoutedFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Objects should not exist on client
	for (UReplicatedTestObject* Object : Objects)
	{
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(Object->NetRefHandle));
	}
}

// Tests creation dependencies with multiple parents, multiple children, and parents created first.
UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestComplexCreationDependencyFiltering)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// There are 3 layers of objects in this creation dependency graph.
	// 1st layer is TopObjects[4]. They don't depend on anything.
	// 2nd layer is MiddleObjects[4]. MiddleObjects[i] depends on TopObjects[i] and TopObjects[i+1] if it exists.
	// 3rd layer is BottomObjects[4]. BottomObjects[i] depends on MiddleObjects[i] and MiddleObjects[i+1] if it exists.

	// Create parents first, then children
	constexpr uint32 ArrayObjectCount = 4;

	// Create TopObjects[4]
	UReplicatedTestObject* TopObjects[ArrayObjectCount];
	for (UReplicatedTestObject*& Object : TopObjects)
	{
		Object = Server->CreateObject();
	}

	// Create MiddleObjects[4]
	UReplicatedTestObject* MiddleObjects[ArrayObjectCount];
	for (UReplicatedTestObject*& Object : MiddleObjects)
	{
		Object = Server->CreateObject();
	}

	// Create BottomObjects[4]
	UReplicatedTestObject* BottomObjects[ArrayObjectCount];
	for (UReplicatedTestObject*& Object : BottomObjects)
	{
		Object = Server->CreateObject();
	}

	// Add creation dependency links
	for (uint32 ObjectIndex = 0; ObjectIndex < ArrayObjectCount - 1; ++ObjectIndex)
	{
		Bridge->AddCreationDependencyLink(TopObjects[ObjectIndex]->NetRefHandle, MiddleObjects[ObjectIndex]->NetRefHandle);
		Bridge->AddCreationDependencyLink(TopObjects[ObjectIndex + 1]->NetRefHandle, MiddleObjects[ObjectIndex]->NetRefHandle);
		Bridge->AddCreationDependencyLink(MiddleObjects[ObjectIndex]->NetRefHandle, BottomObjects[ObjectIndex]->NetRefHandle);
		Bridge->AddCreationDependencyLink(MiddleObjects[ObjectIndex + 1]->NetRefHandle, BottomObjects[ObjectIndex]->NetRefHandle);
	}
	Bridge->AddCreationDependencyLink(TopObjects[ArrayObjectCount - 1]->NetRefHandle, MiddleObjects[ArrayObjectCount - 1]->NetRefHandle);
	Bridge->AddCreationDependencyLink(MiddleObjects[ArrayObjectCount - 1]->NetRefHandle, BottomObjects[ArrayObjectCount - 1]->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Objects should all exist on client
	for (uint32 ObjectIndex = 0; ObjectIndex < ArrayObjectCount; ++ObjectIndex)
	{
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[ObjectIndex]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[ObjectIndex]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[ObjectIndex]->NetRefHandle));
	}

	{
		// Filter out MiddleObjects[1]
		ReplicationSystem->SetFilter(MiddleObjects[1]->NetRefHandle, NotRoutedFilterHandle);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// TopObjects should all exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[0]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[1]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[3]->NetRefHandle));

		// MiddleObjects[1] should not exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(MiddleObjects[1]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[3]->NetRefHandle));

		// BottomObjects[0-1] should not exist on the client
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[1]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[3]->NetRefHandle));
	}

	{
		// Filter out TopObjects[2]
		ReplicationSystem->SetFilter(TopObjects[2]->NetRefHandle, NotRoutedFilterHandle);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// TopObjects[2] should not exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[0]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(TopObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[3]->NetRefHandle));

		// MiddleObjects[1-2] should not exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(MiddleObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(MiddleObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[3]->NetRefHandle));

		// BottomObjects[0-2] should not exist on the client
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[3]->NetRefHandle));
	}

	{
		// Filter in MiddleObjects[1]
		// Nothing should change because one of its parents is filtered out
		ReplicationSystem->SetFilter(MiddleObjects[1]->NetRefHandle, InvalidNetObjectFilterHandle);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// TopObjects[2] should not exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[0]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(TopObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[3]->NetRefHandle));

		// MiddleObjects[1-2] should not exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(MiddleObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(MiddleObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[3]->NetRefHandle));

		// BottomObjects[0-2] should not exist on the client
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[3]->NetRefHandle));
	}

	{
		// Filter in TopObjects[2]
		ReplicationSystem->SetFilter(TopObjects[2]->NetRefHandle, InvalidNetObjectFilterHandle);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// Objects should all exist on client
		for (uint32 ObjectIndex = 0U; ObjectIndex < ArrayObjectCount; ++ObjectIndex)
		{
			UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[ObjectIndex]->NetRefHandle));
			UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[ObjectIndex]->NetRefHandle));
			UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[ObjectIndex]->NetRefHandle));
		}
	}
}

// Tests creation dependencies with multiple parents, multiple children, and children created first.
UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestComplexCreationDependencyFilteringReversed)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// There are 3 layers of objects in this creation dependency graph.
	// 1st layer is TopObjects[4]. They don't depend on anything.
	// 2nd layer is MiddleObjects[4]. MiddleObjects[i] depends on TopObjects[i] and TopObjects[i+1] if it exists.
	// 3rd layer is BottomObjects[4]. BottomObjects[i] depends on MiddleObjects[i] and MiddleObjects[i+1] if it exists.

	// Create children first, then parents
	constexpr uint32 ArrayObjectCount = 4;

	// Create BottomObjects[4]
	UReplicatedTestObject* BottomObjects[ArrayObjectCount];
	for (UReplicatedTestObject*& Object : BottomObjects)
	{
		Object = Server->CreateObject();
	}

	// Create MiddleObjects[4]
	UReplicatedTestObject* MiddleObjects[ArrayObjectCount];
	for (UReplicatedTestObject*& Object : MiddleObjects)
	{
		Object = Server->CreateObject();
	}

	// Create TopObjects[4]
	UReplicatedTestObject* TopObjects[ArrayObjectCount];
	for (UReplicatedTestObject*& Object : TopObjects)
	{
		Object = Server->CreateObject();
	}

	// Add creation dependency links
	for (uint32 ObjectIndex = 0U; ObjectIndex < ArrayObjectCount - 1; ++ObjectIndex)
	{
		Bridge->AddCreationDependencyLink(TopObjects[ObjectIndex]->NetRefHandle, MiddleObjects[ObjectIndex]->NetRefHandle);
		Bridge->AddCreationDependencyLink(TopObjects[ObjectIndex + 1]->NetRefHandle, MiddleObjects[ObjectIndex]->NetRefHandle);
		Bridge->AddCreationDependencyLink(MiddleObjects[ObjectIndex]->NetRefHandle, BottomObjects[ObjectIndex]->NetRefHandle);
		Bridge->AddCreationDependencyLink(MiddleObjects[ObjectIndex + 1]->NetRefHandle, BottomObjects[ObjectIndex]->NetRefHandle);
	}
	Bridge->AddCreationDependencyLink(TopObjects[ArrayObjectCount - 1]->NetRefHandle, MiddleObjects[ArrayObjectCount - 1]->NetRefHandle);
	Bridge->AddCreationDependencyLink(MiddleObjects[ArrayObjectCount - 1]->NetRefHandle, BottomObjects[ArrayObjectCount - 1]->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Objects should all exist on client
	for (uint32 ObjectIndex = 0U; ObjectIndex < ArrayObjectCount; ++ObjectIndex)
	{
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[ObjectIndex]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[ObjectIndex]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[ObjectIndex]->NetRefHandle));
	}

	{
		// Filter out MiddleObjects[1]
		ReplicationSystem->SetFilter(MiddleObjects[1]->NetRefHandle, NotRoutedFilterHandle);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// TopObjects should all exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[0]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[1]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[3]->NetRefHandle));

		// MiddleObjects[1] should not exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(MiddleObjects[1]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[3]->NetRefHandle));

		// BottomObjects[0-1] should not exist on the client
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[1]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[3]->NetRefHandle));
	}

	{
		// Filter out TopObjects[2]
		ReplicationSystem->SetFilter(TopObjects[2]->NetRefHandle, NotRoutedFilterHandle);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// TopObjects[2] should not exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[0]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(TopObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[3]->NetRefHandle));

		// MiddleObjects[1-2] should not exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(MiddleObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(MiddleObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[3]->NetRefHandle));

		// BottomObjects[0-2] should not exist on the client
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[3]->NetRefHandle));
	}

	{
		// Filter in MiddleObjects[1]
		// Nothing should change because one of its parents is filtered out
		ReplicationSystem->SetFilter(MiddleObjects[1]->NetRefHandle, InvalidNetObjectFilterHandle);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// TopObjects[2] should not exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[0]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(TopObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[3]->NetRefHandle));

		// MiddleObjects[1-2] should not exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(MiddleObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(MiddleObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[3]->NetRefHandle));

		// BottomObjects[0-2] should not exist on the client
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[3]->NetRefHandle));
	}

	{
		// Filter in TopObjects[2]
		ReplicationSystem->SetFilter(TopObjects[2]->NetRefHandle, InvalidNetObjectFilterHandle);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// Objects should all exist on client
		for (uint32 ObjectIndex = 0U; ObjectIndex < ArrayObjectCount; ++ObjectIndex)
		{
			UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[ObjectIndex]->NetRefHandle));
			UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[ObjectIndex]->NetRefHandle));
			UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[ObjectIndex]->NetRefHandle));
		}
	}
}

// Tests creation dependencies with multiple parents, multiple children, and mixed parent/child object creation order.
UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestComplexCreationDependencyFilteringMixedOrder)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// There are 3 layers of objects in this creation dependency graph.
	// 1st layer is TopObjects[4]. They don't depend on anything.
	// 2nd layer is MiddleObjects[4]. MiddleObjects[i] depends on TopObjects[i] and TopObjects[i+1] if it exists.
	// 3rd layer is BottomObjects[4]. BottomObjects[i] depends on MiddleObjects[i] and MiddleObjects[i+1] if it exists.

	// Create MiddleObjects first to simulate creating some children before their parents and some parents before their children
	constexpr uint32 ArrayObjectCount = 4;

	// Create MiddleObjects[4]
	UReplicatedTestObject* MiddleObjects[ArrayObjectCount];
	for (UReplicatedTestObject*& Object : MiddleObjects)
	{
		Object = Server->CreateObject();
	}

	// Create TopObjects[4]
	UReplicatedTestObject* TopObjects[ArrayObjectCount];
	for (UReplicatedTestObject*& Object : TopObjects)
	{
		Object = Server->CreateObject();
	}

	// Create BottomObjects[4]
	UReplicatedTestObject* BottomObjects[ArrayObjectCount];
	for (UReplicatedTestObject*& Object : BottomObjects)
	{
		Object = Server->CreateObject();
	}

	// Add creation dependency links
	for (uint32 ObjectIndex = 0; ObjectIndex < ArrayObjectCount - 1; ++ObjectIndex)
	{
		Bridge->AddCreationDependencyLink(TopObjects[ObjectIndex]->NetRefHandle, MiddleObjects[ObjectIndex]->NetRefHandle);
		Bridge->AddCreationDependencyLink(TopObjects[ObjectIndex + 1]->NetRefHandle, MiddleObjects[ObjectIndex]->NetRefHandle);
		Bridge->AddCreationDependencyLink(MiddleObjects[ObjectIndex]->NetRefHandle, BottomObjects[ObjectIndex]->NetRefHandle);
		Bridge->AddCreationDependencyLink(MiddleObjects[ObjectIndex + 1]->NetRefHandle, BottomObjects[ObjectIndex]->NetRefHandle);
	}
	Bridge->AddCreationDependencyLink(TopObjects[ArrayObjectCount - 1]->NetRefHandle, MiddleObjects[ArrayObjectCount - 1]->NetRefHandle);
	Bridge->AddCreationDependencyLink(MiddleObjects[ArrayObjectCount - 1]->NetRefHandle, BottomObjects[ArrayObjectCount - 1]->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Objects should all exist on client
	for (uint32 ObjectIndex = 0U; ObjectIndex < ArrayObjectCount; ++ObjectIndex)
	{
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[ObjectIndex]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[ObjectIndex]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[ObjectIndex]->NetRefHandle));
	}

	{
		// Filter out MiddleObjects[1]
		ReplicationSystem->SetFilter(MiddleObjects[1]->NetRefHandle, NotRoutedFilterHandle);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// TopObjects should all exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[0]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[1]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[3]->NetRefHandle));

		// MiddleObjects[1] should not exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(MiddleObjects[1]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[3]->NetRefHandle));

		// BottomObjects[0-1] should not exist on the client
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[1]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[3]->NetRefHandle));
	}

	{
		// Filter out TopObjects[2]
		ReplicationSystem->SetFilter(TopObjects[2]->NetRefHandle, NotRoutedFilterHandle);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// TopObjects[2] should not exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[0]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(TopObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[3]->NetRefHandle));

		// MiddleObjects[1-2] should not exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(MiddleObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(MiddleObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[3]->NetRefHandle));

		// BottomObjects[0-2] should not exist on the client
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[3]->NetRefHandle));
	}

	{
		// Filter in MiddleObjects[1]
		// Nothing should change because one of its parents is filtered out
		ReplicationSystem->SetFilter(MiddleObjects[1]->NetRefHandle, InvalidNetObjectFilterHandle);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// TopObjects[2] should not exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[0]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(TopObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[3]->NetRefHandle));

		// MiddleObjects[1-2] should not exist on the client
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(MiddleObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(MiddleObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[3]->NetRefHandle));

		// BottomObjects[0-2] should not exist on the client
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[0]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[1]->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(BottomObjects[2]->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[3]->NetRefHandle));
	}

	{
		// Filter in TopObjects[2]
		ReplicationSystem->SetFilter(TopObjects[2]->NetRefHandle, InvalidNetObjectFilterHandle);

		// Send and deliver packets
		Server->UpdateAndSend({ Client });

		// Objects should all exist on client
		for (uint32 ObjectIndex = 0U; ObjectIndex < ArrayObjectCount; ++ObjectIndex)
		{
			UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(TopObjects[ObjectIndex]->NetRefHandle));
			UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(MiddleObjects[ObjectIndex]->NetRefHandle));
			UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(BottomObjects[ObjectIndex]->NetRefHandle));
		}
	}
}

// Add a creation dependency link after the parent has already been filtered out.
UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestCreationDependencyAddedAfterParentFilteredOut)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn server objects
	UReplicatedTestObject* ParentObject = Server->CreateObject();
	UReplicatedTestObject* ChildObject = Server->CreateObject();

	// Filter out the parent before adding the creation dependency
	ReplicationSystem->SetFilter(ParentObject->NetRefHandle, NotRoutedFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Parent should not exist, child should exist (no dependency yet)
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Add the creation dependency while the parent is filtered out
	Bridge->AddCreationDependencyLink(ParentObject->NetRefHandle, ChildObject->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Both should be gone since the parent is filtered out and the child depends on it
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Filter parent back in
	ReplicationSystem->SetFilter(ParentObject->NetRefHandle, InvalidNetObjectFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Both objects should now exist on the client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
}

// Filter a creation dependency parent out via an exclusion group filter
UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestCreationDependencyFilteredByExclusionGroup)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn server objects
	UReplicatedTestObject* ParentObject = Server->CreateObject();
	UReplicatedTestObject* ChildObject = Server->CreateObject();
	Bridge->AddCreationDependencyLink(ParentObject->NetRefHandle, ChildObject->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Both objects should exist on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Add parent to an exclusion group and filter it out
	FNetObjectGroupHandle ExclusionGroup = ReplicationSystem->CreateGroup(NAME_None);
	ReplicationSystem->AddExclusionFilterGroup(ExclusionGroup);
	ReplicationSystem->AddToGroup(ExclusionGroup, ParentObject->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Both objects should be filtered out
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Filter the parent back in
	ReplicationSystem->SetGroupFilterStatus(ExclusionGroup, ENetFilterStatus::Allow);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Both objects should exist on client again
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
}

// Filter a creation dependency parent out via an owner filter
UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestCreationDependencyFilteredByOwnerFilter)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn server objects
	UReplicatedTestObject* ParentObject = Server->CreateObject();
	UReplicatedTestObject* ChildObject = Server->CreateObject();
	Bridge->AddCreationDependencyLink(ParentObject->NetRefHandle, ChildObject->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Both objects should exist on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Set owner filter on parent without setting an owning connection, which filters it out for all connections
	ReplicationSystem->SetFilter(ParentObject->NetRefHandle, ToOwnerFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Both objects should be filtered out
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Set the client as the owning connection to filter the parent back in
	ReplicationSystem->SetOwningNetConnection(ParentObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Both objects should exist on client again
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
}

// Filter a creation dependency out via dynamic filter, then filter in with an inclusion group.
UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestCreationDependencyFilteredByDynamicFilterOverriddenByInclusionGroup)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn server objects
	UReplicatedTestObject* ParentObject = Server->CreateObject();
	UReplicatedTestObject* ChildObject = Server->CreateObject();
	Bridge->AddCreationDependencyLink(ParentObject->NetRefHandle, ChildObject->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Both objects should exist on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Filter out parent
	ReplicationSystem->SetFilter(ParentObject->NetRefHandle, NotRoutedFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Both objects should be filtered out
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));

	// Add parent to an inclusion group and allow it for this connection, overriding the dynamic filter
	FNetObjectGroupHandle InclusionGroup = ReplicationSystem->CreateGroup(NAME_None);
	ReplicationSystem->AddInclusionFilterGroup(InclusionGroup);
	ReplicationSystem->AddToGroup(InclusionGroup, ParentObject->NetRefHandle);
	ReplicationSystem->SetGroupFilterStatus(InclusionGroup, Client->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Both objects should exist on client again since the inclusion group overrides the dynamic filter
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
}

// Add a replicated subobject to a child in a creation dependency, filter out the parent, then filter it back in.
UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestCreationDependencyChildSubObjectFilteredWithParent)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn server objects and set up creation dependency
	UReplicatedTestObject* ParentObject = Server->CreateObject();
	UReplicatedTestObject* ChildObject = Server->CreateObject();
	Bridge->AddCreationDependencyLink(ParentObject->NetRefHandle, ChildObject->NetRefHandle);

	// Add a replicated subobject to the child
	UReplicatedTestObject* ChildSubObject = Server->CreateSubObject(ChildObject->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// All objects should exist on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildSubObject->NetRefHandle));

	// Filter out the parent
	ReplicationSystem->SetFilter(ParentObject->NetRefHandle, NotRoutedFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Parent, child, and the child's subobject should all be filtered out
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildSubObject->NetRefHandle));

	// Add a replicated subobject to the child AFTER parent is filtered
	UReplicatedTestObject* ChildSubObjectAddedAfterFilterApplied = Server->CreateSubObject(ChildObject->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// All objects should be filtered, including the child subobject added after the parent was filtered.
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildSubObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildSubObjectAddedAfterFilterApplied->NetRefHandle));

	// Filter the parent back in
	ReplicationSystem->SetFilter(ParentObject->NetRefHandle, InvalidNetObjectFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// All objects should exist on client again
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildSubObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildSubObjectAddedAfterFilterApplied->NetRefHandle));
}

// Destroy a subobject filtered by creation dependency, add a new object and verify stale state doesn't filter it out
UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestCreationDependencyChildSubObjectDeleted)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn server objects and set up creation dependency
	UReplicatedTestObject* ParentObject = Server->CreateObject();
	UReplicatedTestObject* ChildObject = Server->CreateObject();
	Bridge->AddCreationDependencyLink(ParentObject->NetRefHandle, ChildObject->NetRefHandle);

	// Add a replicated subobject to the child
	UReplicatedTestObject* ChildSubObject = Server->CreateSubObject(ChildObject->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// All objects should exist on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildSubObject->NetRefHandle));

	// Filter out the parent
	ReplicationSystem->SetFilter(ParentObject->NetRefHandle, NotRoutedFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Parent, child, and the child's subobject should all be filtered out
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildSubObject->NetRefHandle));

	// Destroy child subobject
	const FNetRefHandle ChildSubObjectHandle = ChildSubObject->NetRefHandle;
	Server->DestroyObject(ChildSubObject);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Parent, child, and the child's subobject should all not exist on the client
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildSubObjectHandle));

	// Filter the parent back in
	ReplicationSystem->SetFilter(ParentObject->NetRefHandle, InvalidNetObjectFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// All objects except the destroyed subobject should exist on client again
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildSubObjectHandle));

	// Create another object, should reuse internal index of destroyed subobject
	UReplicatedTestObject* ObjectReusingInternalIndex = Server->CreateObject();

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// All objects should exist on the client (old creation dependency state from the destroyed subobject shouldn't filter out the new object)
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ObjectReusingInternalIndex->NetRefHandle));
}

// Destroy a child + subobject filtered by creation dependency, add two new objects and verify stale state doesn't filter them out
UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestCreationDependencyChildAndSubObjectDeleted)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn server objects and set up creation dependency
	UReplicatedTestObject* ParentObject = Server->CreateObject();
	UReplicatedTestObject* ChildObject = Server->CreateObject();
	Bridge->AddCreationDependencyLink(ParentObject->NetRefHandle, ChildObject->NetRefHandle);

	// Add a replicated subobject to the child
	UReplicatedTestObject* ChildSubObject = Server->CreateSubObject(ChildObject->NetRefHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// All objects should exist on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ChildSubObject->NetRefHandle));

	// Filter out the parent
	ReplicationSystem->SetFilter(ParentObject->NetRefHandle, NotRoutedFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Parent, child, and the child's subobject should all be filtered out
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildSubObject->NetRefHandle));

	// Destroy child object and subobject.
	const FNetRefHandle ChildObjectHandle = ChildObject->NetRefHandle;
	const FNetRefHandle ChildSubObjectHandle = ChildSubObject->NetRefHandle;
	Server->DestroyObject(ChildObject);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Parent, child, and the child's subobject should all not exist on the client
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildObjectHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildSubObjectHandle));

	// Filter the parent back in
	ReplicationSystem->SetFilter(ParentObject->NetRefHandle, InvalidNetObjectFilterHandle);

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// Only the parent object should exist.
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildObjectHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ChildSubObjectHandle));

	// Create two objects, should reuse internal index of the destroyed child + subobject
	UReplicatedTestObject* ObjectReusingChildInternalIndex = Server->CreateObject();
	UReplicatedTestObject* ObjectReusingChildSubObjectInternalIndex = Server->CreateObject();	

	// Send and deliver packets
	Server->UpdateAndSend({ Client });

	// All objects should exist on the client (old creation dependency state from the destroyed objects shouldn't filter out the new objects)
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ParentObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ObjectReusingChildInternalIndex->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ObjectReusingChildSubObjectInternalIndex->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, TestCircularDependencyIsDenied)
{
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Spawn at least two objects on the server
	constexpr uint32 ObjectCount = 4;
	UTestReplicatedIrisObject* ServerObjects[ObjectCount];
	for (UTestReplicatedIrisObject*& ServerObject : ServerObjects)
	{
		ServerObject = Server->CreateObject();
	}

	// Add all dependencies but the one creating the circular dependency
	for (uint32 ObjectIt = 0, ObjectEndIt = ObjectCount - 1; ObjectIt < ObjectEndIt; ++ObjectIt)
	{
		Bridge->AddDependentObject(ServerObjects[ObjectIt]->NetRefHandle, ServerObjects[ObjectIt + 1]->NetRefHandle);
	}

	// Create circular dependency
	{
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Fatal);
		FTestEnsureScope SuppressEnsureScope;
		Bridge->AddDependentObject(ServerObjects[ObjectCount - 1]->NetRefHandle, ServerObjects[0]->NetRefHandle);
		UE_NET_ASSERT_EQ(SuppressEnsureScope.GetCount(), (DO_ENSURE ? 1 : 0));
	}
}

// Create a dependency chain with three objects. Mark dependent objects as not routed. All three objects should replicate.
UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, DependentObjectIsInScopeIfParentChainIsInScope)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Objects are created in arbitrary order
	{
		UTestReplicatedIrisObject* ServerRootObject = Server->CreateObject();
		UTestReplicatedIrisObject* ServerParentObject = Server->CreateObject();
		UTestReplicatedIrisObject* ServerDependentObject = Server->CreateObject();

		ReplicationSystem->SetFilter(ServerParentObject->NetRefHandle, NotRoutedFilterHandle);
		ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, NotRoutedFilterHandle);

		Bridge->AddDependentObject(ServerRootObject->NetRefHandle, ServerParentObject->NetRefHandle);
		Bridge->AddDependentObject(ServerParentObject->NetRefHandle, ServerDependentObject->NetRefHandle);

		Server->UpdateAndSend({ Client });

		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerRootObject->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerParentObject->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerDependentObject->NetRefHandle));

		// Cleanup a bit
		Server->DestroyObject(ServerDependentObject);
		Server->DestroyObject(ServerParentObject);
		Server->DestroyObject(ServerRootObject);

		Server->UpdateAndSend({ Client });
	}

	// Objects are created in reverse order to the previous test
	{
		UTestReplicatedIrisObject* ServerDependentObject = Server->CreateObject();
		UTestReplicatedIrisObject* ServerParentObject = Server->CreateObject();
		UTestReplicatedIrisObject* ServerRootObject = Server->CreateObject();

		ReplicationSystem->SetFilter(ServerParentObject->NetRefHandle, NotRoutedFilterHandle);
		ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, NotRoutedFilterHandle);

		Bridge->AddDependentObject(ServerRootObject->NetRefHandle, ServerParentObject->NetRefHandle);
		Bridge->AddDependentObject(ServerParentObject->NetRefHandle, ServerDependentObject->NetRefHandle);

		Server->UpdateAndSend({ Client });

		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerRootObject->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerParentObject->NetRefHandle));
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerDependentObject->NetRefHandle));

		// Cleanup a bit
		Server->DestroyObject(ServerDependentObject);
		Server->DestroyObject(ServerParentObject);
		Server->DestroyObject(ServerRootObject);

		Server->UpdateAndSend({ Client });
	}
}

// Create a dependency chain with three objects. Mark dependent objects as not routed. Exclusion filter out middle object. Only the root object should replicate.
UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, DependentObjectIsNotInScopeIfParentIsNotInScope)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Objects are created in arbitrary order
	{
		UTestReplicatedIrisObject* ServerRootObject = Server->CreateObject();
		UTestReplicatedIrisObject* ServerParentObject = Server->CreateObject();
		UTestReplicatedIrisObject* ServerDependentObject = Server->CreateObject();

		ReplicationSystem->SetFilter(ServerParentObject->NetRefHandle, NotRoutedFilterHandle);
		ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, NotRoutedFilterHandle);

		Bridge->AddDependentObject(ServerRootObject->NetRefHandle, ServerParentObject->NetRefHandle);
		Bridge->AddDependentObject(ServerParentObject->NetRefHandle, ServerDependentObject->NetRefHandle);

		FNetObjectGroupHandle ExclusionGroup = ReplicationSystem->CreateGroup(NAME_None);
		ReplicationSystem->AddToGroup(ExclusionGroup, ServerParentObject->NetRefHandle);
		ReplicationSystem->AddExclusionFilterGroup(ExclusionGroup);

		Server->UpdateAndSend({ Client });

		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerRootObject->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerParentObject->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerDependentObject->NetRefHandle));

		// Cleanup a bit
		ReplicationSystem->DestroyGroup(ExclusionGroup);

		Server->DestroyObject(ServerDependentObject);
		Server->DestroyObject(ServerParentObject);
		Server->DestroyObject(ServerRootObject);

		Server->UpdateAndSend({ Client });
	}

	// Objects are created in reverse order to the previous test
	{
		UTestReplicatedIrisObject* ServerDependentObject = Server->CreateObject();
		UTestReplicatedIrisObject* ServerParentObject = Server->CreateObject();
		UTestReplicatedIrisObject* ServerRootObject = Server->CreateObject();

		ReplicationSystem->SetFilter(ServerParentObject->NetRefHandle, NotRoutedFilterHandle);
		ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, NotRoutedFilterHandle);

		Bridge->AddDependentObject(ServerRootObject->NetRefHandle, ServerParentObject->NetRefHandle);
		Bridge->AddDependentObject(ServerParentObject->NetRefHandle, ServerDependentObject->NetRefHandle);

		FNetObjectGroupHandle ExclusionGroup = ReplicationSystem->CreateGroup(NAME_None);
		ReplicationSystem->AddToGroup(ExclusionGroup, ServerParentObject->NetRefHandle);
		ReplicationSystem->AddExclusionFilterGroup(ExclusionGroup);

		Server->UpdateAndSend({ Client });

		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerRootObject->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerParentObject->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerDependentObject->NetRefHandle));

		// Cleanup a bit
		ReplicationSystem->DestroyGroup(ExclusionGroup);

		Server->DestroyObject(ServerDependentObject);
		Server->DestroyObject(ServerParentObject);
		Server->DestroyObject(ServerRootObject);

		Server->UpdateAndSend({ Client });
	}
}

// Create a dependency chain with three objects. Mark all objects as not routed. No objects should replicate.
UE_NET_TEST_FIXTURE(FTestFilteredDependentObjectFixture, DependentObjectIsNotInScopeIfRootIsDynamicallyFilteredOut)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Objects are created in arbitrary order
	{
		UTestReplicatedIrisObject* ServerRootObject = Server->CreateObject();
		UTestReplicatedIrisObject* ServerParentObject = Server->CreateObject();
		UTestReplicatedIrisObject* ServerDependentObject = Server->CreateObject();

		ReplicationSystem->SetFilter(ServerRootObject->NetRefHandle, NotRoutedFilterHandle);
		ReplicationSystem->SetFilter(ServerParentObject->NetRefHandle, NotRoutedFilterHandle);
		ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, NotRoutedFilterHandle);

		Bridge->AddDependentObject(ServerRootObject->NetRefHandle, ServerParentObject->NetRefHandle);
		Bridge->AddDependentObject(ServerParentObject->NetRefHandle, ServerDependentObject->NetRefHandle);

		Server->UpdateAndSend({ Client });

		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerRootObject->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerParentObject->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerDependentObject->NetRefHandle));

		// Cleanup a bit
		Server->DestroyObject(ServerDependentObject);
		Server->DestroyObject(ServerParentObject);
		Server->DestroyObject(ServerRootObject);

		Server->UpdateAndSend({ Client });
	}

	// Objects are created in reverse order to the previous test
	{
		UTestReplicatedIrisObject* ServerDependentObject = Server->CreateObject();
		UTestReplicatedIrisObject* ServerParentObject = Server->CreateObject();
		UTestReplicatedIrisObject* ServerRootObject = Server->CreateObject();

		ReplicationSystem->SetFilter(ServerRootObject->NetRefHandle, NotRoutedFilterHandle);
		ReplicationSystem->SetFilter(ServerParentObject->NetRefHandle, NotRoutedFilterHandle);
		ReplicationSystem->SetFilter(ServerDependentObject->NetRefHandle, NotRoutedFilterHandle);

		Bridge->AddDependentObject(ServerRootObject->NetRefHandle, ServerParentObject->NetRefHandle);
		Bridge->AddDependentObject(ServerParentObject->NetRefHandle, ServerDependentObject->NetRefHandle);

		Server->UpdateAndSend({ Client });

		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerRootObject->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerParentObject->NetRefHandle));
		UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerDependentObject->NetRefHandle));

		// Cleanup a bit
		Server->DestroyObject(ServerDependentObject);
		Server->DestroyObject(ServerParentObject);
		Server->DestroyObject(ServerRootObject);

		Server->UpdateAndSend({ Client });
	}
}

}
