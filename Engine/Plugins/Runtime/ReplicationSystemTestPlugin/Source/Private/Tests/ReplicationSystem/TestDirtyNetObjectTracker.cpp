// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Iris/ReplicationSystem/DirtyNetObjectTracker.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Tests/ReplicationSystem/ReplicationSystemTestFixture.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"

namespace UE::Net::Private
{

class FDirtyNetObjectTrackerTestFixture : public FReplicationSystemTestFixture
{
	using Super = FReplicationSystemTestFixture;

public:
	virtual void SetUp() override
	{
		Super::SetUp();
		DirtyNetObjectTracker = &ReplicationSystem->GetReplicationSystemInternal()->GetDirtyNetObjectTracker();
		ReplicationSystemId = ReplicationSystem->GetId();
		
		FDirtyObjectsAccessor DirtyObjectsAccessor(*DirtyNetObjectTracker);
		NetObjectIndexRangeEnd = DirtyObjectsAccessor.GetDirtyNetObjects().GetNumBits() - 1U;
	}

	virtual void TearDown() override
	{
		DirtyNetObjectTracker = nullptr;
		ReplicationSystemId = ~0U;
		Super::TearDown();
	}

protected:
	static constexpr uint32 NetObjectIndexRangeStart = 1;

	FDirtyNetObjectTracker* DirtyNetObjectTracker = nullptr;
	uint32 ReplicationSystemId = ~0U;
	uint32 NetObjectIndexRangeEnd = 1;
};

UE_NET_TEST_FIXTURE(FDirtyNetObjectTrackerTestFixture, TestNoObjectIsDirtyFromStart)
{
	FDirtyObjectsAccessor DirtyObjectsAccessor(*DirtyNetObjectTracker);

	const FNetBitArrayView DirtyObjects = DirtyObjectsAccessor.GetDirtyNetObjects();
	UE_NET_ASSERT_EQ(DirtyObjects.GetNumBits(), NetObjectIndexRangeEnd + 1U);

	UE_NET_ASSERT_FALSE_MSG(DirtyObjects.IsAnyBitSet(), "Objects are marked as dirty before marking any object state as dirty");
}

UE_NET_TEST_FIXTURE(FDirtyNetObjectTrackerTestFixture, CannotMarkInvalidObjectAsDirty)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogIrisDirtyTracker, ELogVerbosity::Error);

	MarkNetObjectStateDirty(ReplicationSystemId, InvalidInternalNetRefIndex);
	MarkNetObjectStateDirty(ReplicationSystemId, NetObjectIndexRangeEnd + 1);

	FDirtyObjectsAccessor DirtyObjectsAccessor(*DirtyNetObjectTracker);
	const FNetBitArrayView DirtyObjects = DirtyObjectsAccessor.GetDirtyNetObjects();
	UE_NET_ASSERT_FALSE(DirtyObjects.IsAnyBitSet());
}

UE_NET_TEST_FIXTURE(FDirtyNetObjectTrackerTestFixture, CanMarkValidObjectAsDirty)
{
	const uint32 IndexInRange = NetObjectIndexRangeStart + (NetObjectIndexRangeEnd - NetObjectIndexRangeStart)/2;
	MarkNetObjectStateDirty(ReplicationSystemId, IndexInRange);
	MarkNetObjectStateDirty(ReplicationSystemId, NetObjectIndexRangeStart);
	MarkNetObjectStateDirty(ReplicationSystemId, NetObjectIndexRangeEnd);

	FDirtyObjectsAccessor DirtyObjectsAccessor(*DirtyNetObjectTracker);
	const FNetBitArrayView DirtyObjects = DirtyObjectsAccessor.GetDirtyNetObjects();
	UE_NET_ASSERT_TRUE(DirtyObjects.GetBit(IndexInRange));
	UE_NET_ASSERT_TRUE(DirtyObjects.GetBit(NetObjectIndexRangeStart));
	UE_NET_ASSERT_TRUE(DirtyObjects.GetBit(NetObjectIndexRangeEnd));
}

UE_NET_TEST_FIXTURE(FDirtyNetObjectTrackerTestFixture, CanClearDirtyObjects)
{
	MarkNetObjectStateDirty(ReplicationSystemId, NetObjectIndexRangeStart);
	MarkNetObjectStateDirty(ReplicationSystemId, NetObjectIndexRangeEnd);

	FNetBitArray CleanedObjects;
	CleanedObjects.Init(NetObjectIndexRangeEnd+1);

	CleanedObjects.SetBit(NetObjectIndexRangeStart);
	CleanedObjects.SetBit(NetObjectIndexRangeEnd);

	DirtyNetObjectTracker->UpdateAccumulatedDirtyList();
	DirtyNetObjectTracker->ReconcilePolledList(MakeNetBitArrayView(CleanedObjects));

	const FNetBitArrayView AccumulatedDirtyObjects = DirtyNetObjectTracker->GetAccumulatedDirtyNetObjects();
	UE_NET_ASSERT_FALSE(AccumulatedDirtyObjects.IsAnyBitSet());
}

UE_NET_TEST_FIXTURE(FDirtyNetObjectTrackerTestFixture, DelayedDirtyBitTracking)
{

	const uint32 FirstObjectIndex = NetObjectIndexRangeStart;
	const uint32 SecondObjectIndex = NetObjectIndexRangeStart + 1;
	MarkNetObjectStateDirty(ReplicationSystemId, FirstObjectIndex);
	MarkNetObjectStateDirty(ReplicationSystemId, SecondObjectIndex);

	// Clean first object
	{
		FNetBitArray CleanedObjects;
		CleanedObjects.Init(NetObjectIndexRangeEnd + 1);
		CleanedObjects.SetBit(FirstObjectIndex);

		DirtyNetObjectTracker->UpdateAccumulatedDirtyList();
		DirtyNetObjectTracker->ReconcilePolledList(MakeNetBitArrayView(CleanedObjects));
	}

	const FNetBitArrayView AccumulatedDirtyObjects = DirtyNetObjectTracker->GetAccumulatedDirtyNetObjects();

	UE_NET_ASSERT_FALSE(AccumulatedDirtyObjects.GetBit(FirstObjectIndex));
	UE_NET_ASSERT_TRUE(AccumulatedDirtyObjects.GetBit(SecondObjectIndex));

	// Clean second object
	{
		FNetBitArray CleanedObjects;
		CleanedObjects.Init(NetObjectIndexRangeEnd + 1);
		CleanedObjects.SetBit(SecondObjectIndex);

		DirtyNetObjectTracker->UpdateAccumulatedDirtyList();
		DirtyNetObjectTracker->ReconcilePolledList(MakeNetBitArrayView(CleanedObjects));
	}

	UE_NET_ASSERT_FALSE(AccumulatedDirtyObjects.GetBit(FirstObjectIndex));
	UE_NET_ASSERT_FALSE(AccumulatedDirtyObjects.GetBit(SecondObjectIndex));
}

UE_NET_TEST(DirtyNetObjectTracker, MarkingObjectAsDirtyInNonExistingSystemDoesNotCrash)
{
	constexpr uint32 NonExistingReplicationSystemId = 4711;
	constexpr uint32 ArbitraryNetObjectIndex = 1174;
	MarkNetObjectStateDirty(NonExistingReplicationSystemId, ArbitraryNetObjectIndex);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, GlobalDirtyTrackerTest)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server that is polled only every 3 frames
	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	const uint32 PollPeriod = 2;
	const float PollFrequency = Server->ConvertPollPeriodIntoFrequency(PollPeriod);
	Params.PollFrequency = PollFrequency;
	Params.bUseClassConfigDynamicFilter = true;
	Params.bNeedsPreUpdate = true;

	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(Params);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Object should have been created on the client
	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Set a replicated variable, but don't mark it dirty
	ServerObject->IntA = 0xFF;

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Client replicated property should not have changed
	UE_NET_ASSERT_NE(ClientObject->IntA, ServerObject->IntA);

	// Now mark the object dirty
	FGlobalDirtyNetObjectTracker::MarkNetObjectStateDirty(FNetHandleManager::GetOrCreateNetHandle(ServerObject));

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Client replicated propertyshould have changed now
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);

	Server->DestroyObject(ServerObject);
}

/** Test that validates behavior when dirtying other actors inside PreUpdate/PreReplication */
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, NetForceUpdateOtherObjectInsidePreUpdateTest)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server that is polled late in order to test ForceNetUpdate
	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	const uint32 PollPeriod = 100;
	const float PollFrequency = Server->ConvertPollPeriodIntoFrequency(PollPeriod);
	Params.PollFrequency = PollFrequency;
	Params.bUseClassConfigDynamicFilter = true;
	Params.bNeedsPreUpdate = true;
	UTestReplicatedIrisObject* ServerObjectA = Server->CreateObject(Params);
	UTestReplicatedIrisObject* ServerObjectB = Server->CreateObject(Params);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Object should have been created on the client
	UTestReplicatedIrisObject* ClientObjectA = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectA->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObjectA, nullptr);
	UTestReplicatedIrisObject* ClientObjectB = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectB->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObjectB, nullptr);
	
	// Send and deliver packet twice
	Server->UpdateAndSend({ Client });
	Server->UpdateAndSend({ Client });

	// Set a replicated variable, but don't mark it dirty
	ServerObjectA->IntA = 0xAA;

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Client replicated property should not have changed
	UE_NET_ASSERT_NE(ClientObjectA->IntA, ServerObjectA->IntA);

	// Set a replicated variable, but don't mark it dirty
	ServerObjectB->IntA = 0xAA;

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Client replicated property should not have changed
	UE_NET_ASSERT_NE(ClientObjectA->IntA, ServerObjectA->IntA);
	UE_NET_ASSERT_NE(ClientObjectB->IntA, ServerObjectB->IntA);
	
	// Force ObjectA to replicate
	Server->ReplicationSystem->ForceNetUpdate(ServerObjectA->NetRefHandle);

	auto PreUpdateObjectA = [&](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge)
	{
		for (UObject* ReplicatedObject : Instances)
		{
			// Inside ObjectA PreReplicationUpdate, force ObjectB to be replicated
			if (ServerObjectA == ReplicatedObject)
			{
				Server->ReplicationSystem->ForceNetUpdate(ServerObjectB->NetRefHandle);
			}
		}
	};

	// Now add a dependency where the poll of the first object makes the second one dirty
	Server->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObjectA);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// ObjectA should be replicated now
	UE_NET_ASSERT_EQ(ClientObjectA->IntA, ServerObjectA->IntA);

	// But not ObjectB because it was called inside PreUpdate
	UE_NET_ASSERT_NE(ClientObjectB->IntA, ServerObjectB->IntA);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Now the ForceNetUpdate on ObjectB is applied and it is replicated
	UE_NET_ASSERT_EQ(ClientObjectB->IntA, ServerObjectB->IntA);

	auto PreUpdateObjectB = [&](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge)
	{
		for (UObject* ReplicatedObject : Instances)
		{
			// Inside ObjectB PreReplicationUpdate, force ObjectA to be replicated
			if (ServerObjectB == ReplicatedObject)
			{
				Server->ReplicationSystem->ForceNetUpdate(ServerObjectA->NetRefHandle);
			}
		}
	};
	Server->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObjectB);

	// Dirty both Objects
	ServerObjectA->IntA = 0xBB;
	ServerObjectB->IntA = 0xBB;

	// But only force update ObjectB
	Server->ReplicationSystem->ForceNetUpdate(ServerObjectB->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// ObjectB should have replicated
	UE_NET_ASSERT_EQ(ClientObjectB->IntA, ServerObjectB->IntA);

	// But not ObjectA because it was forced inside PreReplicate()
	UE_NET_ASSERT_NE(ClientObjectA->IntA, ServerObjectA->IntA);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Now ObjectA's force net update is applied and ObjectA is replicated
	UE_NET_ASSERT_EQ(ClientObjectA->IntA, ServerObjectA->IntA);

	Server->DestroyObject(ServerObjectA);
	Server->DestroyObject(ServerObjectB);
}

/** Test that validates behavior when dirtying other actors inside PreUpdate/PreReplication */
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, DirtyOtherObjectInsidePreUpdateTest)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server that is polled every frame.
	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	Params.bUseClassConfigDynamicFilter = true;
	Params.bNeedsPreUpdate = true;

	UTestReplicatedIrisObject::FComponents ComponentsToCreate = { .ObjectReferenceComponentCount = 1 };

	UTestReplicatedIrisObject* ServerObjectA = Server->CreateObject(Params, &ComponentsToCreate);
	UTestReplicatedIrisObject* ServerObjectB = Server->CreateObject(Params, &ComponentsToCreate);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Object should have been created on the client
	UTestReplicatedIrisObject* ClientObjectA = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectA->NetRefHandle));
	UTestReplicatedIrisObject* ClientObjectB = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectB->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObjectA, nullptr);
	UE_NET_ASSERT_NE(ClientObjectB, nullptr);

	UTestReplicatedIrisObject* ObjectToDirty = nullptr;
	auto PreUpdateObject = [&](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge)
	{
		for (UObject* ReplicatedObject : Instances)
		{
			// There's only two objects, so when we update one we dirty the other
			if (ObjectToDirty != ReplicatedObject)
			{
				ObjectToDirty->ObjectReferenceComponents[0]->ModifyIntA();
			}
		}
	};
	
	// When ObjectA is polled modify and dirty ObjectB
	ObjectToDirty = ServerObjectB;
	Server->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObject);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_EQ(ClientObjectB->ObjectReferenceComponents[0]->IntA, ServerObjectB->ObjectReferenceComponents[0]->IntA);

	// Now inverse it so when polling ObjectB we modify and dirty ObjectA
	ObjectToDirty = ServerObjectA;

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_EQ(ClientObjectA->ObjectReferenceComponents[0]->IntA, ServerObjectA->ObjectReferenceComponents[0]->IntA);

	Server->DestroyObject(ServerObjectA);
	Server->DestroyObject(ServerObjectB);
}


/** Test that validates that a push model enabled object is marked as dirty inside PreUpdate/PreReplication */
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, PushModelMarkSelfDirtyInsidePreUpdateTest)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with a PreUpdate call
	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	Params.bNeedsPreUpdate = true;
	UTestReplicatedIrisObject::FComponents ComponentsToCreate = { .ObjectReferenceComponentCount = 1 };

	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(Params, &ComponentsToCreate);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Object should have been created on the client
	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	auto PreUpdateObject = [&](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge)
	{
		for (UObject* InObject : Instances)
		{
			if (InObject == ServerObject)
			{
				ServerObject->ObjectReferenceComponents[0]->ModifyIntA();
			}
		}
	};

	// Mark a property dirty during PreUpdate. As the object is polled every frame we expect the property to be updated on the client.
	Server->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObject);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// The property should be updated on the client
	UE_NET_ASSERT_EQ(ClientObject->ObjectReferenceComponents[0]->IntA, ServerObject->ObjectReferenceComponents[0]->IntA);
}

}
