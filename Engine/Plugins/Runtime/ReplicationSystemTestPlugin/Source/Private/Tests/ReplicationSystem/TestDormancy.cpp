// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Tests/ReplicationSystem/ReplicationSystemTestFixture.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Tests/ReplicationSystem/RPC/ReplicatedTestObjectWithRPC.h"
#include "Tests/ReplicationSystem/RPC/RPCTestFixture.h"

namespace UE::Net::Private
{

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestDormancyWithPushModel)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisPushModelObject* ServerObject = Server->CreateObject<UTestReplicatedIrisPushModelObject>();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Object should be replicated on the client
	UTestReplicatedIrisPushModelObject* ClientObject = Cast<UTestReplicatedIrisPushModelObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientObject != nullptr);

	// Set a property dirty
	ServerObject->SetIntA(1);

	// Property should have been received
	Server->UpdateAndSend({ Client });	
	UE_NET_ASSERT_EQ(ClientObject->GetIntA(), 1);

	// Set object dormant
	Server->GetReplicationBridge()->SetObjectWantsToBeDormant(ServerObject->NetRefHandle, true);

	// Set a property dirty on the same frame as setting the object dormant
	ServerObject->SetIntA(2);

	// Property should have been received
	Server->UpdateAndSend({ Client });
	UE_NET_ASSERT_EQ(ClientObject->GetIntA(), 2);

	// Set a property dirty again
	ServerObject->SetIntA(3);

	// The property on the client should not be updated
	Server->UpdateAndSend({ Client });
	UE_NET_ASSERT_NE(ClientObject->GetIntA(), 3);

	// Flush dormancy on the object
	Server->GetReplicationBridge()->NetFlushDormantObject(ServerObject->NetRefHandle);

	// The property on the client should now be received
	Server->UpdateAndSend({ Client });
	UE_NET_ASSERT_EQ(ClientObject->GetIntA(), 3);

	// Set a property dirty again
	ServerObject->SetIntA(4);

	// The property on the client should not be updated
	Server->UpdateAndSend({ Client });
	UE_NET_ASSERT_NE(ClientObject->GetIntA(), 4);

	// Wake object from dormancy
	Server->GetReplicationBridge()->SetObjectWantsToBeDormant(ServerObject->NetRefHandle, false);

	// The property on the client should be received now
	Server->UpdateAndSend({ Client });
	UE_NET_ASSERT_EQ(ClientObject->GetIntA(), 4);
}

} // end namespace UE::Net::Private