// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSystemServerClientTestFixture.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Misc/ScopeExit.h"
#include "NetBlob/PartialNetBlobTestFixture.h"
#include "NetBlob/MockNetBlob.h"
#include "UObject/CoreNetTypes.h"

namespace UE::Net::Private
{

class FTestFlushBeforeDestroyFixture : public FPartialNetBlobTestFixture
{
public:

	void EnqueueAttachmentToClients(const TArrayView<FReplicationSystemTestClient*const>& TargetClients, FNetRefHandle TargetNetRefHandle)
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(TargetNetRefHandle);

		for (FReplicationSystemTestClient* Client : TargetClients)
		{
			Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
		}
	}
};

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestReliableAttachmentFlushedBeforeDestroy)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Destroy object, on server
	Server->DestroyObject(ServerObject);

	// Deliver a packet, this should flush the object and deliver the attachment
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the attachment has been received
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);

	// Deliver a packet. Should destroy the object on the client unless that was done
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that object is destroyed
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestObjectCreatedAndDestroyedSameFrameReplicatesIfFlushed)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Create and start to replicate object
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;

	// Destroy object indicating that it should be flushed that is that the final state should be replicated to all clients with the object in scope, this invalidates the creationinfo which has to be cached in order for this to work.
	Server->DestroyObject(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Send update, it should send the data.
	Server->UpdateAndSend({Client});

	// Verify that object is created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);

	// Deliver a packet, make sure that object is destroyed on the client.
	Server->UpdateAndSend({Client});

	// Verify that object is destroyed
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestObjectCreatedAndDestroyedSameFrameReplicatesIfFlushedWithPacketLoss)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Create and start to replicate object
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;

	// Destroy object indicating that it should be flushed that is that the final state should be replicated to all clients with the object in scope, this invalidates the creationinfo which has to be cached in order for this to work.
	Server->DestroyObject(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Send update, dropping data
	Server->UpdateAndSend({Client}, false);

	// Send update, it should send the data.
	Server->UpdateAndSend({Client});

	// Verify that object is created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);

	// Deliver a packet, make sure that object is destroyed on the client.
	Server->UpdateAndSend({Client});

	// Verify that object is destroyed
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestObjectAddedToScopeReplicatesIfFlushed)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Create and start to replicate object
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;

	// Destroy object indicating that it should be flushed that is that the final state should be replicated to all clients with the object in scope, this invalidates the creationinfo which has to be cached in order for this to work.
	Server->DestroyObject(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Update without sending, should add the object marked for flush to the clients scope
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Create another client
	FReplicationSystemTestClient* Client2 = CreateClient();
	RegisterNetBlobHandlers(Client2);

	// Send update, we expect the flushed object to be sent to the first client but not the one that joined after the flushed object was removed from scope.
	Server->UpdateAndSend({Client, Client2});

	// Verify that object is created on client but not on client2
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
	UE_NET_ASSERT_EQ(Client2->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);

	// Deliver a packet, make sure that object is destroyed on the client.
	Server->UpdateAndSend({Client, Client2});

	// Verify that object does not exist on any of the clients
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
	UE_NET_ASSERT_EQ(Client2->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestObjectAndSubObjectCreatedAndDestroyedSameFrameReplicatesIfFlushed)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Create and start to replicate object
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ObjectHandle, 0, 0);
	FNetRefHandle SubObjectHandle = ServerSubObject->NetRefHandle;

	// Destroy object indicating that it should be flushed that is that the final state should be replicated to all clients with the object in scope, this invalidates the creationinfo which has to be cached in order for this to work.
	Server->DestroyObject(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Send update, it should send the data.
	Server->UpdateAndSend({Client});

	// Verify that objects are created
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle) != nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle) != nullptr);

	// Deliver a packet, make sure that object is destroyed on the client.
	Server->UpdateAndSend({Client});

	// Verify that objects are destroyed
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle) == nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle) == nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestSubObjectCreatedAndDestroyedSameFrameReplicatesIfFlushed)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Create and start to replicate object
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ObjectHandle, 0, 0);
	FNetRefHandle SubObjectHandle = ServerSubObject->NetRefHandle;

	// Destroy SubObject indicating that it should be flushed that is that the final state should be replicated to all clients with the object in scope, this invalidates the creationinfo which has to be cached in order for this to work.
	Server->DestroyObject(ServerSubObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Send update, it should send the data.
	Server->UpdateAndSend({Client});

	// Verify that objects are created
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle) != nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle) != nullptr);

	// Deliver a packet, make sure that object is destroyed on the client.
	Server->UpdateAndSend({Client});

	// Verify that objects are destroyed
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle) != nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle) == nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestReliableAttachmentFlushedBeforeDestroyIfObjectCreatedAndDestroyedSameFrame)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Create and start to replicate object
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Destroy object, it should be implicitly flushed due to pending attachment.
	Server->DestroyObject(ServerObject, EEndReplicationFlags::Destroy);

	// Send update, it should send the data.
	Server->UpdateAndSend({Client});

	// Verify that the attachment has been received
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);

	// Deliver a packet, make sure that object is destroyed on the client.
	Server->UpdateAndSend({Client});

	// Verify that object is destroyed
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestReliableAttachmentForSubObjectFlushedBeforeDestroyIfObjectCreatedAndDestroyedSameFrame)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Create and start to replicate object with subobject
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ObjectHandle, 0, 0);
	FNetRefHandle SubObjectHandle = ServerSubObject->NetRefHandle;

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(SubObjectHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Destroy object, it should be implicitly flushed due to pending attachment.
	Server->DestroyObject(ServerObject, EEndReplicationFlags::Destroy);

	// Send update, it should send the data.
	Server->UpdateAndSend({Client});

	// Verify that the attachment has been received
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);

	// Deliver a packet, make sure that objects is destroyed on the client.
	Server->UpdateAndSend({Client});

	// Verify that objects are destroyed
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestReliableAttachmentForSubObjecFlushedBeforeDestroyIfSubObjectCreatedAndDestroyedSameFrame)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Create and start to replicate object with subobject
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ObjectHandle, 0, 0);
	FNetRefHandle SubObjectHandle = ServerSubObject->NetRefHandle;

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(SubObjectHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Destroy subobject, it should be implicitly flushed due to pending attachment.
	Server->DestroyObject(ServerSubObject, EEndReplicationFlags::Destroy);

	// Send update, it should send the data.
	Server->UpdateAndSend({Client});

	// Verify that the attachment has been received
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);

	// Deliver a packet, make sure that object is destroyed on the client.
	Server->UpdateAndSend({Client});

	// Verify that objects is destroyed
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestReliableAttachmentFlushedWithDataInflightBeforeDestroy)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Setup a situation where we have reliable data in flight when the object is destroyed
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Destroy object, on server
	Server->DestroyObject(ServerObject);

	// Drop the data and notify server
	Server->DeliverTo(Client, false);

	// Deliver a packet, this should flush the object and deliver the attachment
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the attachment has been received
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);

	// Deliver a packet. Should destroy the object on the client unless that was done
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that object is destroyed
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}

// This test exercises what was a bad case where we was posting RPC:s to a not yet confirmed objects which also was marked for destroy
// This put the replication system in a state where it wrote data that the client could not process.
// Currently we will just drop the data if the initial create packet is lost as we cannot yet send creation info for
// destroyed objects.
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestReliableAttachmentFlushedWithPendingCreationLostBeforeDestroy)
{
	// Disable flushing / caching for this test as we want to keep exercising the bad path regardless of if we force flushing or not.
	IConsoleVariable* CVarEnableFlushReliableRPCOnDestroy = IConsoleManager::Get().FindConsoleVariable(TEXT("net.Iris.EnableFlushReliableRPCOnDestroy"));
	check(CVarEnableFlushReliableRPCOnDestroy != nullptr && CVarEnableFlushReliableRPCOnDestroy->IsVariableBool());
	const bool bPrevEnableFlushReliableRPCOnDestroy = CVarEnableFlushReliableRPCOnDestroy->GetBool();
	CVarEnableFlushReliableRPCOnDestroy->Set(false, ECVF_SetByCode);

	ON_SCOPE_EXIT
	{
		// Restore cvars
		CVarEnableFlushReliableRPCOnDestroy->Set(bPrevEnableFlushReliableRPCOnDestroy, ECVF_SetByCode);
	};

	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;

	// Setup a situation where we have creation info in flight when the object is destroyed

	// Send creation info
	Server->NetUpdate();
	Server->SendTo(Client, TEXT("WaitOnCreateConfirmation"));
	Server->PostSendUpdate();

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Destroy object, on server
	Server->DestroyObject(ServerObject);

	// Previously this would issue a flush and send the attachment data even though creation was not yet confirmed leading to a client disconnect..
	Server->NetUpdate();
	const bool bDataWasSent = Server->SendTo(Client, TEXT("State should still be WaitOnCreateConfirmation"));
	Server->PostSendUpdate();

	// We do not expect any data to be in this packet.
	UE_NET_ASSERT_FALSE(bDataWasSent);

	// Drop the data and notify server
	Server->DeliverTo(Client, false);

	// Deliver data
	if (bDataWasSent)
	{
		// Caused bitstream error on client.
		Server->DeliverTo(Client, true);
	}

	// Update to drive the last transition which we expect to be
	Server->UpdateAndSend({Client});

	// Verify that the attachment has not received
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 0U);

	// Verify that object does not exist
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestReliableAttachmentFlushedWithPendingCreationInflightBeforeDestroy)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;

	// Setup a situation where we have creation info in flight when the object is destroyed

	// Send creation info
	Server->NetUpdate();
	Server->SendTo(Client, TEXT("WaitOnCreateConfirmation"));
	Server->PostSendUpdate();

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Destroy object, on server
	Server->DestroyObject(ServerObject);

	// Previously this would issue a flush and send data before creation is confirmed.
	Server->NetUpdate();
	const bool bDataWasSentInError = Server->SendTo(Client, TEXT("State should still be WaitOnCreateConfirmation"));
	Server->PostSendUpdate();
	
	// We do not expect any data to be in this packet.
	UE_NET_ASSERT_FALSE(bDataWasSentInError);

	// Deliver the packet with CreationInfo
	Server->DeliverTo(Client, true);

	// Deliver data if we sent data.
	if (bDataWasSentInError)
	{
		// Caused bitstream error on client.
		Server->DeliverTo(Client, true);
	}

	// Expected to write the attachment
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket, TEXT("WaitOnFlush"));
	Server->PostSendUpdate();

	// Verify that the attachment has been received
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);

	// Expected to destroy the object
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket, TEXT("Destroy"));
	Server->PostSendUpdate();

	// Verify that object does not exist
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestReliableAttachmentFlushedWithLostPendingCreationInflightBeforeDestroy)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;

	// Setup a situation where we have creation info in flight when the object is destroyed

	// Send creation info
	Server->NetUpdate();
	Server->SendTo(Client, TEXT("WaitOnCreateConfirmation"));
	Server->PostSendUpdate();

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Destroy object, on server
	Server->DestroyObject(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Previously this would issue a flush and send data before creation is confirmed.
	Server->NetUpdate();
	const bool bDataWasSentInError = Server->SendTo(Client, TEXT("State should still be WaitOnCreateConfirmation"));
	Server->PostSendUpdate();
	
	// We do not expect any data to be in this packet.
	UE_NET_ASSERT_FALSE(bDataWasSentInError);

	// Drop initial creation info.
	Server->DeliverTo(Client, false);

	// Deliver data if we sent data.
	if (bDataWasSentInError)
	{
		// Caused bitstream error on client.
		Server->DeliverTo(Client, true);
	}

	// Expected to create object and send attachment
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket, TEXT("CreateResend"));
	Server->PostSendUpdate();

	// Verify that object does not exist
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);

	// Verify that the attachment has been received
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);

	// Expected to write the attachment
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket, TEXT("WaitOnFlush"));
	Server->PostSendUpdate();

	// Expected to destroy the object
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket, TEXT("Destroy"));
	Server->PostSendUpdate();

	// Verify that object has been destroyed
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}


UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestReliableAttachmentSubObjectFlushedBeforeDestroy)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ObjectHandle, 0, 0);
	FNetRefHandle SubObjectHandle = ServerSubObject->NetRefHandle;
	
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerSubObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Destroy sub object, on server
	Server->DestroyObject(ServerSubObject);

	// Deliver a packet, this should flush the object and deliver the attachment
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the attachment has been received
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);

	// Deliver a packet. Should destroy the object on the client unless that was done
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that object is destroyed
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestReliableAttachmentSubObjectFlushedBeforeDestroyIfOwnerIsDestroyed)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ObjectHandle, 0, 0);
	FNetRefHandle SubObjectHandle = ServerSubObject->NetRefHandle;
	
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerSubObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Destroy object which should flush subobject and then destroy both subobject and object
	Server->DestroyObject(ServerObject);

	// Deliver a packet, this should flush the object and deliver the attachment
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the attachment has been received
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);

	// Deliver a packet. Should destroy the object on the client unless that was done
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that both object and subobject are destroyed
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle), nullptr);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestStateFlushedBeforeDestroy)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;
	
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that object is created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);

	// Modify state
	ServerObject->IntA = 3;

	// Destroy object with flush flag which should flush the state before destroying the object
	Server->DestroyObject(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Deliver a packet, this should flush the object and deliver the last state
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle));

	// Verify that object is created
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Verify that we got the expected state
	UE_NET_ASSERT_EQ(ClientObject->IntA, 3);
	
	// Deliver a packet.
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that object is destroyed
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestStateInFlightFlushedBeforeDestroy)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;
	
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that object is created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);

	// Modify state
	ServerObject->IntA = 3;

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Modify state
	ServerObject->IntB = 4;

	// Destroy object with flush flag which should flush the state before destroying the object
	Server->DestroyObject(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Drop the data we had in flight and notify server
	Server->DeliverTo(Client, false);

	// Deliver a packet, this should flush the object and deliver the complete last state
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle));

	// Verify that object is created
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Verify that we got the expected state
	UE_NET_ASSERT_EQ(ClientObject->IntA, 3);
	UE_NET_ASSERT_EQ(ClientObject->IntB, 4);

	// Deliver a packet. Should destroy the object on the client unless that was done
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that object is destroyed
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestPendingCreateTearOffIsNotCancelledByEndReplication)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Setup case where we have a new object for which we have an attachment which should execute a tearoff after we have confirmed creation
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Request tearoff
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// PreUpdate to update scoping to get the object into the PendingCreate state.
	Server->NetUpdate();
	Server->PostSendUpdate();

	// This should be ignored as we are already pending tear off.
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Destroy);

	// Deliver a packet, this should flush the object and deliver the attachment
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the attachment has been received
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);

	// Verify that object is created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestDroppedTearOffIsNotCancelledByEndReplication)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Setup case where we have a new object for which we have an attachment which should execute a tearoff after we have confirmed creation
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Request tearoff
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Send packet so that we have creationdata in flight
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Force destroy object already pending tearoff/flush. This should be ignored
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Destroy);

	// Drop and notify that the packet while object still is the state waitoncreateconfirmation as we have not yet updated scope.
	Server->DeliverTo(Client, false);

	// Deliver a packet, this should flush the object and deliver the attachment
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that the attachment has been received
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);

	// Deliver a packet. Should tear off the object on the client
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that object is not findable
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestSubObjectStateFlushedBeforeOwnerDestroy)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ObjectHandle, 0, 0);
	FNetRefHandle SubObjectHandle = ServerSubObject->NetRefHandle;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that objects is created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle), nullptr);

	// Modify state
	ServerSubObject->IntA = 3;

	// Destroy object with flush flag which should flush the state including before destroying the object
	Server->DestroyObject(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Deliver a packet, this should flush the object and deliver the last state
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle));
	UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle));

	// Verify that objects is created
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_NE(ClientSubObject, nullptr);

	// Verify that we got the expected state
	UE_NET_ASSERT_EQ(ClientSubObject->IntA, 3);
	
	// Deliver a packet.
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that both objects are destroyed
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle) == nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle) == nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestSubObjectStateFlushedBeforeSubObjectDestroy)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ObjectHandle, 0, 0);
	FNetRefHandle SubObjectHandle = ServerSubObject->NetRefHandle;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that objects is created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle), nullptr);

	// Modify state on SubObject
	ServerSubObject->IntA = 3;

	// Destroy object with flush flag which should flush the state including before destroying the object
	Server->DestroyObject(ServerSubObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Deliver a packet, this should flush the object and deliver the last state
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle));
	UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle));

	// Verify that objects are created
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_NE(ClientSubObject, nullptr);

	// Verify that we got the expected state
	UE_NET_ASSERT_EQ(ClientSubObject->IntA, 3);
	
	// Deliver a packet.
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify subobject is destroyed now that last state was confirmed flushed while the main object still is around
	UE_NET_ASSERT_FALSE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle) == nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle) == nullptr);
}

// Test TearOff for existing confirmed object
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestTearOffNewObjectWithReliableAttachment)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Trigger replication
	ServerObject->IntA = 1;

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// TearOff the object
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that object got created
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	UE_NET_ASSERT_TRUE(ClientObjectThatWillBeTornOff != nullptr);

	// Verify that ClientObject got final state and that the attachement was received
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that ClientObject now has been teared off
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle) == nullptr);
}

// Test TearOff for existing confirmed object
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestTearOffExistingObjectWithReliableAttachment)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Trigger replication
	ServerObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Store Pointer to object 
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	UE_NET_ASSERT_TRUE(ClientObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);

	// Modify the value
	ServerObject->IntA = 2;

	// TearOff the object
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that ClientObject got final state and that the attachement was received
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);

	// Verify that ClientObject still is around (from a network perspective)
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that ClientObject now has been teared off
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
}

// Test TearOff and SubObjects, SubObjects must apply state?
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestImmediateTearOffExistingObjectWithSubObjectWithReliableAttachment)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	// Trigger replication
	ServerObject->IntA = 1;
	ServerSubObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Store Pointer to objects
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);

	UTestReplicatedIrisObject* ClientSubObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientSubObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObjectThatWillBeTornOff->IntA);

	// Modify the value of subobject only
	ServerSubObject->IntA = 2;

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerSubObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// TearOff the object using immediate tear-off
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that ClientObject got final state and that the attachement was received
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);
	UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObjectThatWillBeTornOff->IntA);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that ClientObject is torn-off
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle)) == nullptr);
}

// Test to recreate a very specific bug where owner being torn-off has in flight rpc requiring a flush
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestImmediateTearOffWithSubObjectAndInFlightAttachmentsAndPacketLoss)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	// Trigger replication
	ServerObject->IntA = 1;
	ServerSubObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true, TEXT("Create Objects"));
	Server->PostSendUpdate();

	// Store Pointer to objects
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);

	UTestReplicatedIrisObject* ClientSubObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientSubObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObjectThatWillBeTornOff->IntA);

	// Modify the value of object only
	ServerObject->IntA = 2;

	// Create attachment to force flush behavior by having a rpc in flight
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	Server->NetUpdate();
	Server->SendTo(Client, TEXT("State data + Attachment"));
	Server->PostSendUpdate();

	// Modify the value of object only
	++ServerObject->IntA ;

	Server->NetUpdate();
	Server->SendTo(Client, TEXT("State data"));
	Server->PostSendUpdate();

	// TearOff the object using immediate tear-off
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	Server->NetUpdate();
	Server->SendTo(Client, TEXT("Tear off"));
	Server->PostSendUpdate();

	// Deliver packet to drive PendingTearOff -> WaitOnFlush
	Server->DeliverTo(Client, true);

	// Notify that we dropped tear off data
	Server->DeliverTo(Client, false);

	// This earlier caused an unwanted state transition
	Server->NetUpdate();
	Server->SendTo(Client, TEXT("Packet after tearoff"));
	Server->PostSendUpdate();

	// Drop the packet containing the original tear-off
	Server->DeliverTo(Client, false);

	// Deliver a packet
	Server->DeliverTo(Client, true);

	// This should contain resend of lost state
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true, TEXT("Resending tearoff"));
	Server->PostSendUpdate();

	// Verify that ClientObject is torn-off and that the final state was applied
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);
	UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObjectThatWillBeTornOff->IntA);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle)) == nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
}

// Test to recreate a path where we cancel destroy for object pending flush
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestCancelPendingDestroyWaitOnFlushDoesNotMissChanges)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client0 = CreateClient();
	FReplicationSystemTestClient* Client1 = CreateClient();

	RegisterNetBlobHandlers(Client0);
	RegisterNetBlobHandlers(Client1);

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client0, true, TEXT("Create Objects"));
	Server->PostSendUpdate();

	// Store Pointer to objects
	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client0->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientObject != nullptr);
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObject->IntA);

	// Create attachment to force flush behavior by having a rpc in flight
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client0->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	Server->NetUpdate();
	Server->SendTo(Client0, TEXT("Attachment"));
	Server->PostSendUpdate();

	// Filter out object to cause a flush for Client0
	FNetObjectGroupHandle ExclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(ExclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(ExclusionGroupHandle);

	Server->ReplicationSystem->SetGroupFilterStatus(ExclusionGroupHandle, Client0->ConnectionIdOnServer, ENetFilterStatus::Disallow);
	Server->ReplicationSystem->SetGroupFilterStatus(ExclusionGroupHandle, Client1->ConnectionIdOnServer, ENetFilterStatus::Allow);

	Server->NetUpdate();
	Server->SendTo(Client0, TEXT("Out of scope"));
	Server->PostSendUpdate();

	// Modify the value of object only
	++ServerObject->IntA ;

	// Trigger poll + propagate of state
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	// Trigger WaitOnFlush -> Created
	Server->ReplicationSystem->SetGroupFilterStatus(ExclusionGroupHandle, Client0->ConnectionIdOnServer, ENetFilterStatus::Allow);

	// Drop some packets to stay in state
	Server->DeliverTo(Client0, false);
	Server->DeliverTo(Client0, false);

	// Do a normal update, should send state changed that occurred while we where in pending flush state
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client0, DeliverPacket, TEXT("Expected state"));
	Server->PostSendUpdate();

	// Verify that ClientObject is torn-off and that the final state was applied
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObject->IntA);
}

// Test modifying an OwnerOnly property on an object and tear it off
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestModifyingOwnerOnlyProperty)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject({.ConnectionFilteredComponentCount = 1U});
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Send update
	Server->UpdateAndSend({Client});

	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	const int PrevSkipOwnerA = ClientObject->ConnectionFilteredComponents[0]->SkipOwnerA;

	ServerObject->ConnectionFilteredComponents[0]->ToOwnerA = 11;
	ServerObject->ConnectionFilteredComponents[0]->SkipOwnerA = 17;

	// Request tear off of object
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Update once and fail sending to cause torn off object to no longer be in scope
	Server->UpdateAndSend({Client}, DoNotDeliverPacket);

	// Send update
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Verify we got the updated owner only state
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->ToOwnerA, ServerObject->ConnectionFilteredComponents[0]->ToOwnerA);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SkipOwnerA, PrevSkipOwnerA);
}

// Test modifying a OwnerOnly property on a subobject and tear it off
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestModifyingOwnerOnlyPropertyOnSubobject)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, {.ConnectionFilteredComponentCount = 1U});
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Send update
	Server->UpdateAndSend({Client});

	const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	const int PrevSkipOwnerA = ClientSubObject->ConnectionFilteredComponents[0]->SkipOwnerA;

	ServerSubObject->ConnectionFilteredComponents[0]->ToOwnerA = 11;
	ServerSubObject->ConnectionFilteredComponents[0]->SkipOwnerA = 17;

	// Request tear off of subobject
	Server->ReplicationBridge->EndReplication(ServerSubObject, EEndReplicationFlags::TearOff);

	// Update once and fail sending to cause torn off subobject to no longer be in scope
	Server->UpdateAndSend({Client}, DoNotDeliverPacket);

	// Send update
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Verify we got the updated owner only state
	UE_NET_ASSERT_EQ(ClientSubObject->ConnectionFilteredComponents[0]->ToOwnerA, ServerSubObject->ConnectionFilteredComponents[0]->ToOwnerA);
	UE_NET_ASSERT_EQ(ClientSubObject->ConnectionFilteredComponents[0]->SkipOwnerA, PrevSkipOwnerA);
}

// Test modifying a OwnerOnly property on a subobject and tear it off and switch owners before subobject state has flushed
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestModifyingOwnerOnlyPropertyOnSubobjectAndSwitchOwnerBeforeFlushed)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, {.ConnectionFilteredComponentCount = 1U});

	// Send update
	Server->UpdateAndSend({Client});

	const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	const int PrevSkipOwnerA = ClientSubObject->ConnectionFilteredComponents[0]->SkipOwnerA;

	ServerSubObject->ConnectionFilteredComponents[0]->ToOwnerA = 11;
	ServerSubObject->ConnectionFilteredComponents[0]->SkipOwnerA = 17;

	// Request tear off of subobject
	Server->ReplicationBridge->EndReplication(ServerSubObject, EEndReplicationFlags::TearOff);

	// Update once and fail sending to cause torn off subobject to no longer be in scope
	Server->UpdateAndSend({Client}, DoNotDeliverPacket);

	// Switch owners on root object
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Send update
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Verify we got the updated owner only state
	UE_NET_ASSERT_EQ(ClientSubObject->ConnectionFilteredComponents[0]->ToOwnerA, ServerSubObject->ConnectionFilteredComponents[0]->ToOwnerA);
	UE_NET_ASSERT_EQ(ClientSubObject->ConnectionFilteredComponents[0]->SkipOwnerA, PrevSkipOwnerA);
}

// Test modifying a SkipOwner property on a subobject and tear it off and switch owners before subobject state has flushed
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestModifyingSkipOwnerPropertyOnSubobjectAndSwitchOwnerBeforeFlushed)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, {.ConnectionFilteredComponentCount = 1U});

	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Send update
	Server->UpdateAndSend({Client});

	const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	const int PrevOwnerA = ClientSubObject->ConnectionFilteredComponents[0]->ToOwnerA;

	ServerSubObject->ConnectionFilteredComponents[0]->ToOwnerA = 11;
	ServerSubObject->ConnectionFilteredComponents[0]->SkipOwnerA = 17;

	// Request tear off of subobject
	Server->ReplicationBridge->EndReplication(ServerSubObject, EEndReplicationFlags::TearOff);

	// Update once and fail sending to cause torn off subobject to no longer be in scope
	Server->UpdateAndSend({Client}, DoNotDeliverPacket);

	// Switch owners on root object
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, InvalidConnectionId);

	// Send update
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Verify we got the updated owner only state
	UE_NET_ASSERT_EQ(ClientSubObject->ConnectionFilteredComponents[0]->ToOwnerA, PrevOwnerA);
	UE_NET_ASSERT_EQ(ClientSubObject->ConnectionFilteredComponents[0]->SkipOwnerA, ServerSubObject->ConnectionFilteredComponents[0]->SkipOwnerA);
}

// Test setting subobject conditions before root object tear off
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestSubobjectConditionOnRootObjectTearOff)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject0 = Server->CreateSubObject(ServerObject->NetRefHandle, {.ConnectionFilteredComponentCount = 1U});
	UTestReplicatedIrisObject* ServerSubObject1 = Server->CreateSubObject(ServerObject->NetRefHandle, {.ConnectionFilteredComponentCount = 1U});

	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Send update
	Server->UpdateAndSend({Client});

	const UTestReplicatedIrisObject* ClientSubObject0 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject0->NetRefHandle));
	const UTestReplicatedIrisObject* ClientSubObject1 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject1->NetRefHandle));

	const int SubObjectPrevIntA = ClientSubObject1->IntA;
	const int SubObjectPrevOwnerA = ClientSubObject1->ConnectionFilteredComponents[0]->ToOwnerA;
	const int SubObjectPrevSkipOwnerA = ClientSubObject1->ConnectionFilteredComponents[0]->SkipOwnerA;

	ServerSubObject0->IntA += 11;
	ServerSubObject0->ConnectionFilteredComponents[0]->ToOwnerA += 12;
	ServerSubObject0->ConnectionFilteredComponents[0]->SkipOwnerA += 13;
	ServerSubObject1->IntA += 14;
	ServerSubObject1->ConnectionFilteredComponents[0]->ToOwnerA += 15;
	ServerSubObject1->ConnectionFilteredComponents[0]->SkipOwnerA += 16;

	// Set different subobject conditions
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject0->NetRefHandle, ELifetimeCondition::COND_OwnerOnly);
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject1->NetRefHandle, ELifetimeCondition::COND_SkipOwner);

	// Request tear off of object
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Update once and fail sending to cause torn off object to no longer be in scope
	Server->UpdateAndSend({Client}, DoNotDeliverPacket);

	// Send update
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Verify we got the updated state only for the OwnerOnly subobject
	UE_NET_ASSERT_EQ(ClientSubObject0->IntA, ServerSubObject0->IntA);
	UE_NET_ASSERT_EQ(ClientSubObject0->ConnectionFilteredComponents[0]->ToOwnerA, ServerSubObject0->ConnectionFilteredComponents[0]->ToOwnerA);
	UE_NET_ASSERT_EQ(ClientSubObject0->ConnectionFilteredComponents[0]->SkipOwnerA, SubObjectPrevSkipOwnerA);

	UE_NET_ASSERT_EQ(ClientSubObject1->IntA, SubObjectPrevIntA);
	UE_NET_ASSERT_EQ(ClientSubObject1->ConnectionFilteredComponents[0]->ToOwnerA, SubObjectPrevOwnerA);
	UE_NET_ASSERT_EQ(ClientSubObject1->ConnectionFilteredComponents[0]->SkipOwnerA, SubObjectPrevSkipOwnerA);
}

// Test setting subobject conditions before subobject tear off
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestSubobjectConditionOnSubobjectTearOff)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject0 = Server->CreateSubObject(ServerObject->NetRefHandle, {.ConnectionFilteredComponentCount = 1U});
	UTestReplicatedIrisObject* ServerSubObject1 = Server->CreateSubObject(ServerObject->NetRefHandle, {.ConnectionFilteredComponentCount = 1U});

	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Send update
	Server->UpdateAndSend({Client});

	const UTestReplicatedIrisObject* ClientSubObject0 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject0->NetRefHandle));
	const UTestReplicatedIrisObject* ClientSubObject1 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject1->NetRefHandle));

	const int SubObjectPrevIntA = ClientSubObject1->IntA;
	const int SubObjectPrevOwnerA = ClientSubObject1->ConnectionFilteredComponents[0]->ToOwnerA;
	const int SubObjectPrevSkipOwnerA = ClientSubObject1->ConnectionFilteredComponents[0]->SkipOwnerA;

	ServerSubObject0->IntA += 11;
	ServerSubObject0->ConnectionFilteredComponents[0]->ToOwnerA += 12;
	ServerSubObject0->ConnectionFilteredComponents[0]->SkipOwnerA += 13;
	ServerSubObject1->IntA += 14;
	ServerSubObject1->ConnectionFilteredComponents[0]->ToOwnerA += 15;
	ServerSubObject1->ConnectionFilteredComponents[0]->SkipOwnerA += 16;

	// Set different subobject conditions
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject0->NetRefHandle, ELifetimeCondition::COND_OwnerOnly);
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject1->NetRefHandle, ELifetimeCondition::COND_SkipOwner);

	// Request tear off of subobjects
	Server->ReplicationBridge->EndReplication(ServerSubObject0, EEndReplicationFlags::TearOff);
	Server->ReplicationBridge->EndReplication(ServerSubObject1, EEndReplicationFlags::TearOff);

	// Update once and fail sending to cause torn off object to no longer be in scope
	Server->UpdateAndSend({Client}, DoNotDeliverPacket);

	// Send update
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Verify we got the updated state only for the OwnerOnly subobject
	UE_NET_ASSERT_EQ(ClientSubObject0->IntA, ServerSubObject0->IntA);
	UE_NET_ASSERT_EQ(ClientSubObject0->ConnectionFilteredComponents[0]->ToOwnerA, ServerSubObject0->ConnectionFilteredComponents[0]->ToOwnerA);
	UE_NET_ASSERT_EQ(ClientSubObject0->ConnectionFilteredComponents[0]->SkipOwnerA, SubObjectPrevSkipOwnerA);

	UE_NET_ASSERT_EQ(ClientSubObject1->IntA, SubObjectPrevIntA);
	UE_NET_ASSERT_EQ(ClientSubObject1->ConnectionFilteredComponents[0]->ToOwnerA, SubObjectPrevOwnerA);
	UE_NET_ASSERT_EQ(ClientSubObject1->ConnectionFilteredComponents[0]->SkipOwnerA, SubObjectPrevSkipOwnerA);
}

// Test setting subobject condition on subobjects before subobject tear off and set owner before subobject state has flushed
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestSubobjectConditionOnSubobjectTearOffAndSetOwnerBeforeFlushed)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject0 = Server->CreateSubObject(ServerObject->NetRefHandle, {.ConnectionFilteredComponentCount = 1U});
	UTestReplicatedIrisObject* ServerSubObject1 = Server->CreateSubObject(ServerObject->NetRefHandle, {.ConnectionFilteredComponentCount = 1U});

	// Send update
	Server->UpdateAndSend({Client});

	const UTestReplicatedIrisObject* ClientSubObject0 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject0->NetRefHandle));
	const UTestReplicatedIrisObject* ClientSubObject1 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject1->NetRefHandle));

	const int SubObjectPrevIntA = ClientSubObject1->IntA;
	const int SubObjectPrevOwnerA = ClientSubObject1->ConnectionFilteredComponents[0]->ToOwnerA;
	const int SubObjectPrevSkipOwnerA = ClientSubObject1->ConnectionFilteredComponents[0]->SkipOwnerA;

	ServerSubObject0->IntA += 11;
	ServerSubObject0->ConnectionFilteredComponents[0]->ToOwnerA += 12;
	ServerSubObject0->ConnectionFilteredComponents[0]->SkipOwnerA += 13;
	ServerSubObject1->IntA += 14;
	ServerSubObject1->ConnectionFilteredComponents[0]->ToOwnerA += 15;
	ServerSubObject1->ConnectionFilteredComponents[0]->SkipOwnerA += 16;

	// Set different subobject conditions
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject0->NetRefHandle, ELifetimeCondition::COND_OwnerOnly);
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject1->NetRefHandle, ELifetimeCondition::COND_SkipOwner);

	// Request tear off of subobjects
	Server->ReplicationBridge->EndReplication(ServerSubObject0, EEndReplicationFlags::TearOff);
	Server->ReplicationBridge->EndReplication(ServerSubObject1, EEndReplicationFlags::TearOff);

	// Update once and fail sending to cause torn off object to no longer be in scope
	Server->UpdateAndSend({Client}, DoNotDeliverPacket);

	// Set owner on root object
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);
 
	// Send update
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Verify we got the updated state only for the OwnerOnly subobject
	UE_NET_ASSERT_EQ(ClientSubObject0->IntA, ServerSubObject0->IntA);
	UE_NET_ASSERT_EQ(ClientSubObject0->ConnectionFilteredComponents[0]->ToOwnerA, ServerSubObject0->ConnectionFilteredComponents[0]->ToOwnerA);
	UE_NET_ASSERT_EQ(ClientSubObject0->ConnectionFilteredComponents[0]->SkipOwnerA, SubObjectPrevSkipOwnerA);

	UE_NET_ASSERT_EQ(ClientSubObject1->IntA, SubObjectPrevIntA);
	UE_NET_ASSERT_EQ(ClientSubObject1->ConnectionFilteredComponents[0]->ToOwnerA, SubObjectPrevOwnerA);
	UE_NET_ASSERT_EQ(ClientSubObject1->ConnectionFilteredComponents[0]->SkipOwnerA, SubObjectPrevSkipOwnerA);
}

// Test setting subobject condition on subobjects before subobject tear off and switch owners before subobject state has flushed
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestSubobjectConditionOnSubobjectTearOffAndSwitchOwnerBeforeFlushed)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject0 = Server->CreateSubObject(ServerObject->NetRefHandle, {.ConnectionFilteredComponentCount = 1U});
	UTestReplicatedIrisObject* ServerSubObject1 = Server->CreateSubObject(ServerObject->NetRefHandle, {.ConnectionFilteredComponentCount = 1U});

	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Send update
	Server->UpdateAndSend({Client});

	const UTestReplicatedIrisObject* ClientSubObject0 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject0->NetRefHandle));
	const UTestReplicatedIrisObject* ClientSubObject1 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject1->NetRefHandle));

	const int SubObjectPrevIntA = ClientSubObject1->IntA;
	const int SubObjectPrevOwnerA = ClientSubObject1->ConnectionFilteredComponents[0]->ToOwnerA;
	const int SubObjectPrevSkipOwnerA = ClientSubObject1->ConnectionFilteredComponents[0]->SkipOwnerA;

	ServerSubObject0->IntA += 11;
	ServerSubObject0->ConnectionFilteredComponents[0]->ToOwnerA += 12;
	ServerSubObject0->ConnectionFilteredComponents[0]->SkipOwnerA += 13;
	ServerSubObject1->IntA += 14;
	ServerSubObject1->ConnectionFilteredComponents[0]->ToOwnerA += 15;
	ServerSubObject1->ConnectionFilteredComponents[0]->SkipOwnerA += 16;

	// Set different subobject conditions
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject0->NetRefHandle, ELifetimeCondition::COND_OwnerOnly);
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject1->NetRefHandle, ELifetimeCondition::COND_SkipOwner);

	// Request tear off of subobjects
	Server->ReplicationBridge->EndReplication(ServerSubObject0, EEndReplicationFlags::TearOff);
	Server->ReplicationBridge->EndReplication(ServerSubObject1, EEndReplicationFlags::TearOff);

	// Update once and fail sending to cause torn off object to no longer be in scope
	Server->UpdateAndSend({Client}, DoNotDeliverPacket);

	// Switch owners on root object
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, UE::Net::InvalidConnectionId);
 
	// Send update
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Verify we got the updated state only for the SkipOwner subobject
	UE_NET_ASSERT_EQ(ClientSubObject0->IntA, SubObjectPrevIntA);
	UE_NET_ASSERT_EQ(ClientSubObject0->ConnectionFilteredComponents[0]->ToOwnerA, SubObjectPrevOwnerA);
	UE_NET_ASSERT_EQ(ClientSubObject0->ConnectionFilteredComponents[0]->SkipOwnerA, SubObjectPrevSkipOwnerA);

	UE_NET_ASSERT_EQ(ClientSubObject1->IntA, ServerSubObject1->IntA);
	UE_NET_ASSERT_EQ(ClientSubObject1->ConnectionFilteredComponents[0]->ToOwnerA, SubObjectPrevOwnerA);
	UE_NET_ASSERT_EQ(ClientSubObject1->ConnectionFilteredComponents[0]->SkipOwnerA, ServerSubObject1->ConnectionFilteredComponents[0]->SkipOwnerA);
}

// Test sending attachment to owner filtered object and tear it off
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestTearOffOfOwnerFilteredObject)
{
	// Add clients
	FReplicationSystemTestClient* ClientArray[] = {CreateClient(), CreateClient()};
	TStrongObjectPtr<UMockNetObjectAttachmentHandler> ClientMockNetObjectAttachmentHandlers[UE_ARRAY_COUNT(ClientArray)];
	for (FReplicationSystemTestClient*& Client : ClientArray)
	{
		RegisterNetBlobHandlers(Client);
		ClientMockNetObjectAttachmentHandlers[&Client - ClientArray] = ClientMockNetObjectAttachmentHandler;
	}

	// Setup case where we have a new object for which we have an attachment which should execute a tearoff after we have confirmed creation
	UReplicatedTestObject* ServerObject = Server->CreateObject();

	// Apply owner and owner filter
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, ClientArray[0]->ConnectionIdOnServer);
	Server->ReplicationSystem->SetFilter(ServerObject->NetRefHandle, ToOwnerFilterHandle);

	// Send update
	Server->UpdateAndSend({ClientArray}, DeliverPacket);

	// Create attachment
	{
		constexpr uint32 PayloadBitCount = 24;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		for (FReplicationSystemTestClient* Client : ClientArray)
		{
			Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
		}
	}

	// Request tearoff
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Update once and fail sending to cause torn off object to no longer be in scope
	Server->UpdateAndSend({ClientArray}, DoNotDeliverPacket);

	// Deliver a packet, this should flush the object and deliver the attachment to the owning connection
	Server->UpdateAndSend({ClientArray}, DeliverPacket);

	// Verify that the attachment has been received on the allowed connection
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandlers[0]->GetFunctionCallCounts().OnNetBlobReceived, 1U);

	// Verify that the attachment was not received on the disallowed connection
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandlers[1]->GetFunctionCallCounts().OnNetBlobReceived, 0U);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestDelayedTearOffAndSubobjectDestroy)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject0 = Server->CreateSubObject(ServerObject->NetRefHandle);
	UTestReplicatedIrisObject* ServerSubObject1 = Server->CreateSubObject(ServerObject->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket, TEXT("Create objects"));

	Server->ReplicationSystem->TearOffNextUpdate(ServerObject->NetRefHandle);

	Server->UpdateAndSend({Client}, DoNotDeliverPacket, TEXT("Tear off"));

	Server->DestroyObject(ServerSubObject1);

	Server->UpdateAndSend({Client}, DoNotDeliverPacket, TEXT("Tear off and destroy"));
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestTearOffAndFilterOutInSameFrameWhileSubobjectHasReliableAttachment)
{
	// Uncomment the verbosiy override scopes and enable UE_NET_ENABLE_REPLICATIONWRITER_LOG to track down issues if test fails.
	//LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Verbose);

	FTestEnsureScope EnsureScope;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject0 = Server->CreateSubObject(ServerObject->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket, TEXT("Create objects"));

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UTestReplicatedIrisObject* ClientSubObject0 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject0->NetRefHandle));

	UTestReplicatedIrisObject* ServerSubObject1 = Server->CreateSubObject(ServerObject->NetRefHandle);
	// Create reliable attachment to ServerSubObject1
	{
		constexpr uint32 PayloadBitCount = 128U;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerSubObject1->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	Server->DestroyObject(ServerSubObject1);

	// Put data in flight
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Filter out object
	FNetObjectGroupHandle ExclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(ExclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(ExclusionGroupHandle);

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();
	UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 0);

	ServerObject->IntA++;
	ServerSubObject0->IntA++;

	// Filter in object
	Server->ReplicationSystem->RemoveFromGroup(ExclusionGroupHandle, ServerObject->NetRefHandle);

	// Tear off 
	Server->ReplicationSystem->TearOffNextUpdate(ServerObject->NetRefHandle);

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();
	UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 0);

	// Deliver remaining packets
	{
		const SIZE_T PacketCount = Server->GetConnectionInfo(Client->ConnectionIdOnServer).WrittenPackets.Count();
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, DeliverPacket);
		}
	}

	// Send update
	Server->UpdateAndSend({Client});

	// Verify all objects are unresolvable on the client
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerSubObject0->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerSubObject1->NetRefHandle));

	// Verify we got the latest state and received the reliable attachment
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
	UE_NET_ASSERT_EQ(ClientSubObject0->IntA, ServerSubObject0->IntA);
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);
}

// This test is almost like the one above but with some minor differences to updates.
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestTearOffAndFilterOutInSameFrameWhileSubobjectHasReliableAttachment2)
{
	// Uncomment the verbosiy override scopes and enable UE_NET_ENABLE_REPLICATIONWRITER_LOG to track down issues if test fails.
	//LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Verbose);
	//LOG_SCOPE_VERBOSITY_OVERRIDE(LogIrisBridge, ELogVerbosity::Verbose);

	FTestEnsureScope EnsureScope;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject0 = Server->CreateSubObject(ServerObject->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket, TEXT("Create objects"));

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UTestReplicatedIrisObject* ClientSubObject0 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject0->NetRefHandle));

	UTestReplicatedIrisObject* ServerSubObject1 = Server->CreateSubObject(ServerObject->NetRefHandle);
	// Create reliable attachment to ServerSubObject1
	{
		constexpr uint32 PayloadBitCount = 128U;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerSubObject1->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	Server->DestroyObject(ServerSubObject1);

	// Put data in flight
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Filter out object
	FNetObjectGroupHandle ExclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(ExclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(ExclusionGroupHandle);

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();
	UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 0);

	ServerObject->IntA++;
	ServerSubObject0->IntA++;

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();
	UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 0);

	Server->DeliverTo(Client, DeliverPacket);

	// Filter in object
	Server->ReplicationSystem->RemoveFromGroup(ExclusionGroupHandle, ServerObject->NetRefHandle);

	// Tear off 
	Server->ReplicationSystem->TearOffNextUpdate(ServerObject->NetRefHandle);

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();
	UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 0);

	// Deliver remaining packets
	{
		const SIZE_T PacketCount = Server->GetConnectionInfo(Client->ConnectionIdOnServer).WrittenPackets.Count();
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, DeliverPacket);
		}
	}

	// Send update
	Server->UpdateAndSend({Client});

	// Send update
	Server->UpdateAndSend({Client});

	// Verify all objects are unresolvable on the client
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerSubObject0->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerSubObject1->NetRefHandle));

	// Verify we got the latest state and received the reliable attachment
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
	UE_NET_ASSERT_EQ(ClientSubObject0->IntA, ServerSubObject0->IntA);
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestTearOffOfObjectNotInScopeWithSubObjectPendingDestroy)
{
	// Uncomment the verbosiy override scopes and enable UE_NET_ENABLE_REPLICATIONWRITER_LOG to track down issues if test fails.
	//LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Verbose);

	FTestEnsureScope EnsureScope;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject0 = Server->CreateSubObject(ServerObject->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket, TEXT("Create objects"));

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UTestReplicatedIrisObject* ClientSubObject0 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject0->NetRefHandle));

	UTestReplicatedIrisObject* ServerSubObject1 = Server->CreateSubObject(ServerObject->NetRefHandle);
	// Create reliable attachment to ServerSubObject1
	{
		constexpr uint32 PayloadBitCount = 128U;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerSubObject1->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	Server->DestroyObject(ServerSubObject1);

	ServerObject->IntA++;
	ServerSubObject0->IntA++;

	// Put data in flight
	Server->NetUpdate();
	Server->SendTo(Client, TEXT("SubObject1 creation"));
	Server->PostSendUpdate();

	// Filter out object. This will cause root to end up in WaitOnFlush
	FNetObjectGroupHandle ExclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(ExclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(ExclusionGroupHandle);

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 0);

	// Do not deliver SubObject1 creation
	Server->DeliverTo(Client, DoNotDeliverPacket);

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// At this point we're expecting Root in WaitOnFlush due to going out of scope, SubObject0 in PendingDestroy and SubObject1 in WaitOnCreateConfirmation.
	
	// Tear off while not in scope with unacked data in flight
	Server->ReplicationSystem->TearOffNextUpdate(ServerObject->NetRefHandle);

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();
	UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 0);

	// Do not deliver remaining packets
	{
		const SIZE_T PacketCount = Server->GetConnectionInfo(Client->ConnectionIdOnServer).WrittenPackets.Count();
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, DoNotDeliverPacket);
		}
	}

	// Send and deliver a couple of packets
	Server->UpdateAndSend({Client});
	Server->UpdateAndSend({Client});
	Server->UpdateAndSend({Client});

	// Verify all objects are unresolvable on the client
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerSubObject0->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerSubObject1->NetRefHandle));

	// Once an object starts getting destroyed it will clear dirtiness. A subsequent tear off will not automatically get that latest state over. 
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
	UE_NET_ASSERT_EQ(ClientSubObject0->IntA, ServerSubObject0->IntA);

	// Verify we got the latest state and received the reliable attachment
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestTearOffOfObjectNotInScopeWithSubObjectPendingDestroy2)
{
	// Uncomment the verbosiy override scopes and enable UE_NET_ENABLE_REPLICATIONWRITER_LOG to track down issues if test fails.
	//LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Verbose);
	//LOG_SCOPE_VERBOSITY_OVERRIDE(LogIrisBridge, ELogVerbosity::Verbose);

	FTestEnsureScope EnsureScope;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject0 = Server->CreateSubObject(ServerObject->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket, TEXT("Create objects"));

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UTestReplicatedIrisObject* ClientSubObject0 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject0->NetRefHandle));

	UTestReplicatedIrisObject* ServerSubObject1 = Server->CreateSubObject(ServerObject->NetRefHandle);
	// Create reliable attachment to ServerSubObject1
	{
		constexpr uint32 PayloadBitCount = 128U;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerSubObject1->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	Server->DestroyObject(ServerSubObject1);

	ServerObject->IntA++;
	ServerSubObject0->IntA++;

	// Put data in flight
	Server->NetUpdate();
	Server->SendTo(Client, TEXT("SubObject1 creation"));
	Server->PostSendUpdate();

	// Filter out object. This will cause root to end up in WaitOnFlush
	FNetObjectGroupHandle ExclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(ExclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(ExclusionGroupHandle);

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 0);

	// Do not deliver SubObject1 creation
	Server->DeliverTo(Client, DoNotDeliverPacket);

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Deliver SubObject1 creation
	Server->DeliverTo(Client, DeliverPacket);

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// At this stage we're expecting all objects to be in WaitOnDestroyConfirmation
	
	// Tear off 
	Server->ReplicationSystem->TearOffNextUpdate(ServerObject->NetRefHandle);

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();
	UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 0);

	// Do not deliver remaining packets
	{
		const SIZE_T PacketCount = Server->GetConnectionInfo(Client->ConnectionIdOnServer).WrittenPackets.Count();
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, DoNotDeliverPacket);
		}
	}

	// Send update
	Server->UpdateAndSend({Client});

	// Send update
	Server->UpdateAndSend({Client});

	// Verify all objects are unresolvable on the client
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerSubObject0->NetRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(ServerSubObject1->NetRefHandle));

	// Verify we received the reliable attachment
	UE_NET_ASSERT_EQ(ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts().OnNetBlobReceived, 1U);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestSubObjectIsNotDestroyedBeforeRoot)
{
	// Uncomment the verbosiy override scopes and enable UE_NET_ENABLE_REPLICATIONWRITER_LOG to track down issues if test fails.
	//LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Verbose);
	//LOG_SCOPE_VERBOSITY_OVERRIDE(LogIrisBridge, ELogVerbosity::Verbose);

	FTestEnsureScope EnsureScope;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject0 = Server->CreateSubObject(ServerObject->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket, TEXT("Create objects"));

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UTestReplicatedIrisObject* ClientSubObject0 = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject0->NetRefHandle));

	UTestReplicatedIrisObject* ServerSubObject1 = Server->CreateSubObject(ServerObject->NetRefHandle);
	// Create reliable attachment to ServerSubObject1
	{
		// Create a huge attachment such that it takes more than a couple of frames to deliver
		constexpr uint32 PayloadBitCount = 4000U*8U;
		const TRefCountPtr<FNetObjectAttachment>& Attachment = MockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(PayloadBitCount);
		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerSubObject1->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	Server->DestroyObject(ServerSubObject1);

	// Put data in flight
	Server->NetUpdate();
	Server->SendTo(Client, TEXT("SubObject1 creation"));
	Server->PostSendUpdate();

	// Filter out object. This will cause root to end up in WaitOnFlush
	FNetObjectGroupHandle ExclusionGroupHandle = Server->ReplicationSystem->CreateGroup(NAME_None);
	Server->ReplicationSystem->AddToGroup(ExclusionGroupHandle, ServerObject->NetRefHandle);
	Server->ReplicationSystem->AddExclusionFilterGroup(ExclusionGroupHandle);

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	UE_NET_EXPECT_EQ(EnsureScope.GetCount(), 0);

	// Do not deliver SubObject1 creation
	Server->DeliverTo(Client, DoNotDeliverPacket);

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Update and make sure all objects are created
	Server->DeliverTo(Client, DeliverPacket);

	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerSubObject0->NetRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerSubObject1->NetRefHandle));
}

// The following test can crash if the state update is sent with the destroy
UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestSubObjectStateIsNotSentWithDestroy)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ObjectHandle);
	FNetRefHandle SubObjectHandle = ServerSubObject->NetRefHandle;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that objects is created
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ObjectHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(SubObjectHandle));

	// Modify state on SubObject
	ServerSubObject->IntDWithOnRep += 3;

	// Update but do not ack/nak the packet just yet
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Destroy subobject without flush
	Server->DestroyObject(ServerSubObject);

	// Fail to deliver subobject update
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// Update and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(SubObjectHandle));
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestDynamicStartFlushStartReplicatesAsNewObject)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Verbose);

	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Create and start to replicate object
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle OriginalObjectHandle = ServerObject->NetRefHandle;

	// Stop replicating object using flush
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Should not be resolvable by handle
	UE_NET_ASSERT_EQ(Server->GetReplicationBridge()->GetReplicatedObject(OriginalObjectHandle), nullptr);

	// Since this is a dynamic object, we expect a new handle to be assigned.
	Server->ReplicationBridge->BeginReplication(ServerObject);

	// Should get new handle
	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle != OriginalObjectHandle);

	// New handle should be resolvable 
	UE_NET_ASSERT_NE(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Send update, it should send the data.
	Server->UpdateAndSend({Client});

	// Verify that first instance of object is created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(OriginalObjectHandle), nullptr);
	// Verify that new instance of object is created 
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Deliver a packet
	Server->UpdateAndSend({Client});

	// Verify that object is now destroyed
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(OriginalObjectHandle), nullptr);
	// Verify that new instance of object is not destroyed
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Destroy object
	Server->DestroyObject(ServerObject, EEndReplicationFlags::Destroy);

	// Deliver a packet
	Server->UpdateAndSend({Client});

	// Verify that object now is destroyed
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestDynamicStartStopStartAfterUpdateReplicatesAsNewObject)
{
	//LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Verbose);

	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Create and start to replicate object
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle OriginalServerObjectHandle = ServerObject->NetRefHandle;

	// Stop replicating object using flush and without invalidating the server object.
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Send update, it should send the data.
	Server->UpdateAndSend({Client});

	// Verify that object is created which it should as we requested a flush.
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(OriginalServerObjectHandle), nullptr);

	// Now we start replicating the same object again
	Server->ReplicationBridge->BeginReplication(ServerObject);

	// Which should get a new handle as it is dynamic.
	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle != OriginalServerObjectHandle);

	// Send update, it should send the data.
	Server->UpdateAndSend({Client});

	// Verify that original instance is destroyed on client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(OriginalServerObjectHandle), nullptr);

	// Verify that new instance of object is created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestDynamicStartStopStartAfterUpdateReplicatesAsNewObjectWithPacketLoss)
{
	//LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Verbose);

	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Create and start to replicate object
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle OriginalServerObjectHandle = ServerObject->NetRefHandle;

	// Create reliable attachment to ServerObject to initiate proper flush, can be removed when UE-358165 is fixed.
	EnqueueAttachmentToClients({Client}, ServerObject->NetRefHandle);

	// Stop replicating object that should be flushed due to reliable-rpc and without invalidating the server object.
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Send update, but drop the packet
	Server->UpdateAndSend({Client}, false);

	// Now we start replicating the same object instance again, 
	Server->ReplicationBridge->BeginReplication(ServerObject);

	// Which should get a new handle as it is dynamic.
	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle != OriginalServerObjectHandle);

	Server->UpdateAndSend({Client});

	// This can be surprising, but due to how dynamic objects work (that they get assigned a new NetRefHandle every time we start replication on the server) we will see both instances represented on the client

	// Verify that object is created which it should as we requested a flush.
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(OriginalServerObjectHandle), nullptr);

	// Verify that new instance of object is created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	Server->UpdateAndSend({Client});

	// Now we expect everything to reflect the server state.

	// Verify that original instance is destroyed on client
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(OriginalServerObjectHandle), nullptr);

	// Verify that new instance of object is created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestDynamicSubObjectStartStopStartReplicatesAsNewObject)
{
	FReplicationSystemTestClient* Client = CreateClient();

	// Create and start to replicate object
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ObjectHandle, 0, 0);
	FNetRefHandle OriginalSubObjectHandle = ServerSubObject->NetRefHandle;

	// Stop replicating SubObject but reuse instance indicating that it should be flushed i.e. that the final state should be replicated to all clients scoping the object during the next NetUpdate.
	Server->ReplicationBridge->EndReplication(ServerSubObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Start replicating SubObject again canceling the stop replication request
	Server->ReplicationBridge->BeginReplication(ServerSubObject);

	// Expect new handle as it is a dynamic instance
	UE_NET_ASSERT_TRUE(ServerSubObject->NetRefHandle != OriginalSubObjectHandle);

	Server->UpdateAndSend({Client});

	// Verify that objects are created
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle) != nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(OriginalSubObjectHandle) != nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle) != nullptr);

	Server->UpdateAndSend({Client});

	// Verify that objects still exists
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle) != nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle) != nullptr);

	// But original subobject is destroyed
	UE_NET_ASSERT_FALSE(Client->GetReplicationBridge()->GetReplicatedObject(OriginalSubObjectHandle) != nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestDynamicSubObjectStartStopStartSameFrameReplicatesAsNewObject)
{
	FReplicationSystemTestClient* Client = CreateClient();

	// Create and start to replicate object
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ObjectHandle, 0, 0);
	const FNetRefHandle OriginalSubObjectHandle = ServerSubObject->NetRefHandle;

	// Stop replicating SubObject but keep server instance
	Server->ReplicationBridge->EndReplication(ServerSubObject, EEndReplicationFlags::Destroy);

	// Start replicating SubObject again, will be assigned new Handle as it is dynamic
	Server->ReplicationBridge->BeginReplication(ServerSubObject);
	const FNetRefHandle SubObjectHandle = ServerSubObject->NetRefHandle;

	// Expect new handle
	UE_NET_ASSERT_TRUE(ServerSubObject->NetRefHandle != OriginalSubObjectHandle);

	Server->UpdateAndSend({Client});

	// Verify that objects are created
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle) != nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle) != nullptr);
	// OriginalSubObjectHandle was never relevant for scoping so it will not be created.
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(OriginalSubObjectHandle) == nullptr);

	Server->UpdateAndSend({Client});

	// Verify that objects still exists
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle) != nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectHandle) != nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestDynamicSubObjectStartFlushStartSameFrameReplicatesAsNewObject)
{
	FReplicationSystemTestClient* Client = CreateClient();

	// Create and start to replicate object
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ObjectHandle, 0, 0);
	FNetRefHandle OriginalSubObjectHandle = ServerSubObject->NetRefHandle;

	// Stop replicating SubObject but reuse instance indicating that it should be flushed that is that the final state should be replicated to all clients scoping the object this frame
	Server->ReplicationBridge->EndReplication(ServerSubObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Start replicating SubObject
	Server->ReplicationBridge->BeginReplication(ServerSubObject);

	Server->UpdateAndSend({Client});

	// Verify that objects are created
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle) != nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(OriginalSubObjectHandle) != nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle) != nullptr);

	Server->UpdateAndSend({Client});

	// Verify that original subobject is destroyed
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle) != nullptr);
	UE_NET_ASSERT_FALSE(Client->GetReplicationBridge()->GetReplicatedObject(OriginalSubObjectHandle) != nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle) != nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestStableObjectStartFlushStartAfterNextUpdate)
{
	//LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Verbose);

	const TCHAR* StableObjectName = TEXT("StableObjectName");

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add clients
	FReplicationSystemTestClient* Client = CreateClient();

	// Create stably named object on both server and clients
	UTestReplicatedIrisObject* ServerObject = Server->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName);
	UTestReplicatedIrisObject* ClientObject = Client->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName);

	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsValid());
	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsStatic());

	// Create and start to replicate object
	FNetRefHandle OriginalServerObjectHandle = ServerObject->NetRefHandle;

	ServerObject->IntA = 1;

	// Stop replicating object using flush and without invalidating the server object.
	// Flush means that we send the final state and attachments to the clients.
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Flush);

	// After the update, the server will mark the flushed NetObject/NetRefHandle as destroyed.
	Server->UpdateAndSend({Client});

	// Verify that object is created which it should as we requested a flush.
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(OriginalServerObjectHandle)) == ClientObject);
	UE_NET_ASSERT_EQ(ClientObject->IntA, 1);

	// Now we start replicating the same object again, which will be assigned a new internal index.
	Server->ReplicationBridge->BeginReplication(ServerObject);

	// As this is a static handle we expect the same handle to be used
	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle == OriginalServerObjectHandle);

	ServerObject->IntA = 2;

	// This update will send the actual destroy of the original (flushed) instance.
	Server->UpdateAndSend({Client});

	// Verify that original instance is no longer considered replicated
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(OriginalServerObjectHandle), nullptr);

	// Now that know that the client has destroyed the NetObject for the flushed instance we expect the enw instance to replicate.
	Server->UpdateAndSend({Client});

	// Verify that NetRefHandle is now considered replicated again
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObject);
	UE_NET_ASSERT_EQ(ClientObject->IntA, 2);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestStableObjectCreatedAndDestroyedAndRestartedAfterNextUpdateReplicatesIfFlushedWithDroppedData)
{
	//LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Verbose);

	const TCHAR* StableObjectName = TEXT("StableObjectName");

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add clients
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Create stably named object on both server and clients
	UTestReplicatedIrisObject* ServerObject = Server->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName);
	UTestReplicatedIrisObject* ClientObject = Client->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName);

	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsValid());
	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsStatic());

	// Create and start to replicate object
	FNetRefHandle OriginalServerObjectHandle = ServerObject->NetRefHandle;

	ServerObject->IntA = 1;

	// Create reliable attachment to ServerObject to initiate proper flush, can be removed when UE-358165 is fixed.
	EnqueueAttachmentToClients({Client}, ServerObject->NetRefHandle);
	
	// Stop replicating object using flush and without invalidating the server object.
	// Flush means that we send the final state and attachments to the clients.
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Flush);

	// After the update, the server will mark the flushed NetObject/NetRefHandle as destroyed.
	// Note: that we drop the data
	Server->UpdateAndSend({Client}, false);

	// Now we start replicating the same object again, which will be assigned a new internal index.
	Server->ReplicationBridge->BeginReplication(ServerObject);

	ServerObject->IntA = 2;

	// As this is a static handle we expect the same handle to be used
	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle == OriginalServerObjectHandle);

	// After the update, the server will mark the flushed NetObject/NetRefHandle as destroyed.
	// Note: that we drop the data
	Server->UpdateAndSend({Client});

	// Verify that object is created which it should as we requested a flush, even after dropping the packet containing the original create request
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(OriginalServerObjectHandle)) == ClientObject);
	UE_NET_ASSERT_EQ(ClientObject->IntA, 1);

	// This update will send the actual destroy of the original (flushed) instance.
	Server->UpdateAndSend({Client});

	// Verify that original instance is no longer considered replicated
	UE_NET_ASSERT_EQ(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(OriginalServerObjectHandle)), nullptr);

	// Now that know that the client has destroyed the NetObject for the flushed instance we expect the new instance to replicate.
	Server->UpdateAndSend({Client});

	// Verify that NetRefHandle is now considered replicated again
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObject);
	UE_NET_ASSERT_EQ(ClientObject->IntA, 2);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestStateFlushedDoesNotPickupStateChangesAfterFlush)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;
	
	Server->UpdateAndSend({Client});
	
	// Verify that object is created
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);

	// Modify state
	ServerObject->IntA = 1;

	// Destroy object with flush flag which should flush the state before destroying the object
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Modify state, this should not be sent
	ServerObject->IntA = 2;

	Server->UpdateAndSend({Client});

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle));

	// Verify that object is created
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Verify that we got the expected state
	UE_NET_ASSERT_EQ(ClientObject->IntA, 1);

	// Deliver a packet. Should destroy the object on the client unless that was done
	Server->UpdateAndSend({Client});

	// Verify that object is destroyed
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FTestFlushBeforeDestroyFixture, TestStateFlushedDoesNotAccecptNewSubObjectAfterFlush)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);
	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;

	// Modify state
	ServerObject->IntA = 1;

	// Destroy object with flush flag which should flush the state before destroying the object
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Modify state
	ServerObject->IntA = 2;

	// Add SubObject
	{
		// This should ensure so we add a scope to suppress it
		FTestEnsureScope EnsureScope;

		UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
		FNetRefHandle SubObjectHandle = ServerSubObject->NetRefHandle;

		UE_NET_ASSERT_FALSE(SubObjectHandle.IsValid());
	}
		
	Server->UpdateAndSend({Client});

	// Verify that object is created and that we got the expected state
	{
		UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);
		// Verify that we got the expected state
		UE_NET_ASSERT_EQ(ClientObject->IntA, 1);
	}

	Server->UpdateAndSend({Client});

	// Verify that object is destroyed
	UE_NET_ASSERT_EQ(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle)), nullptr);
}

} // End of namespace
