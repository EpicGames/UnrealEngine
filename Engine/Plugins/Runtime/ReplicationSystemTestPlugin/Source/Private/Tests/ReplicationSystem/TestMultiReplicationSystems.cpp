// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/ReplicationSystem/MultiReplicationSystemsTestFixture.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Net/Core/NetHandle/NetHandleManager.h"
#include "Containers/StaticArray.h"

namespace UE::Net::Private
{

UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, CanCreateMultipleReplicationSystems)
{
	CreateSomeServers();

	TArrayView<FReplicationSystemTestServer*> ServersView = GetAllServers();
	UE_NET_ASSERT_EQ(unsigned(ServersView.Num()), unsigned(DefaultServerCount));

	const unsigned ExpectedReplicationSystemCount = DefaultServerCount;
	unsigned ValidReplicationSystemCount = 0U;
	for (FReplicationSystemTestServer* Server : ServersView)
	{
		if (UReplicationSystem* ReplicationSystem = Server->GetReplicationSystem())
		{
			++ValidReplicationSystemCount;
		}
	}

	UE_NET_ASSERT_EQ(ValidReplicationSystemCount, ExpectedReplicationSystemCount);
}

UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, CanCreateMaxReplicationSystemCount)
{
	// Use a lower max object count for this test to keep memory usage under control
	FReplicationSystemTestNode::FReplicationSystemParamsOverride ParamsOverride;
	ParamsOverride.MaxReplicatedObjectCount = 1024;
	ParamsOverride.InitialNetObjectListCount = 1024;

	CreateServers(FNetRefHandle::MaxReplicationSystemCount, &ParamsOverride);

	TArrayView<FReplicationSystemTestServer*> ServersView = GetAllServers();
	UE_NET_ASSERT_EQ(unsigned(ServersView.Num()), unsigned(FNetRefHandle::MaxReplicationSystemCount));

	const unsigned ExpectedReplicationSystemCount = FNetRefHandle::MaxReplicationSystemCount;
	unsigned ValidReplicationSystemCount = 0U;
	for (FReplicationSystemTestServer* Server : ServersView)
	{
		if (UReplicationSystem* ReplicationSystem = Server->GetReplicationSystem())
		{
			++ValidReplicationSystemCount;
		}
	}

	UE_NET_ASSERT_EQ(ValidReplicationSystemCount, ExpectedReplicationSystemCount);
}

UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, ReplicationSystemIdsAreStable)
{
	constexpr uint32 InitialNumServers = 5;
	TStaticArray<FReplicationSystemTestServer*, InitialNumServers> TestServers;
	
	for (uint32 i = 0; i < InitialNumServers; ++i)
	{
		TestServers[i] = CreateServer();
		UE_NET_ASSERT_NE(TestServers[i]->GetReplicationSystem(), nullptr);
	}

	TArrayView<UReplicationSystem*> ReplicationSystemsView = FReplicationSystemFactory::GetAllReplicationSystems();
	UE_NET_ASSERT_EQ(unsigned(ReplicationSystemsView.Num()), unsigned(InitialNumServers));

	// Destroy some replication systems in the middle of the array
	DestroyServer(TestServers[1]);
	TestServers[1] = nullptr;

	DestroyServer(TestServers[3]);
	TestServers[3] = nullptr;

	// Remaining replication systems should maintain their IDs
	UReplicationSystem* System0 = UE::Net::GetReplicationSystem(TestServers[0]->GetReplicationSystem()->GetId());
	UE_NET_ASSERT_EQ(System0, TestServers[0]->GetReplicationSystem());

	UReplicationSystem* System2 = UE::Net::GetReplicationSystem(TestServers[2]->GetReplicationSystem()->GetId());
	UE_NET_ASSERT_EQ(System2, TestServers[2]->GetReplicationSystem());

	UReplicationSystem* System4 = UE::Net::GetReplicationSystem(TestServers[4]->GetReplicationSystem()->GetId());
	UE_NET_ASSERT_EQ(System4, TestServers[4]->GetReplicationSystem());
}

UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, ReplicationSystemArrayMaintainsMinimumSize)
{
	constexpr uint32 InitialNumServers = 5;
	TStaticArray<FReplicationSystemTestServer*, InitialNumServers> TestServers;

	for (uint32 i = 0; i < InitialNumServers; ++i)
	{
		TestServers[i] = CreateServer();
		UE_NET_ASSERT_NE(TestServers[i]->GetReplicationSystem(), nullptr);
	}

	TArrayView<UReplicationSystem*> ReplicationSystemsView = FReplicationSystemFactory::GetAllReplicationSystems();
	UE_NET_ASSERT_EQ(unsigned(ReplicationSystemsView.Num()), unsigned(InitialNumServers));

	// Destroy some replication systems in the middle of the array
	DestroyServer(TestServers[2]);
	TestServers[2] = nullptr;

	DestroyServer(TestServers[3]);
	TestServers[3] = nullptr;

	// View should still be the same size because the last ReplicationSystem still exists
	TArrayView<UReplicationSystem*> ReplicationSystemsView2 = FReplicationSystemFactory::GetAllReplicationSystems();
	UE_NET_ASSERT_EQ(unsigned(ReplicationSystemsView2.Num()), unsigned(InitialNumServers));

	// Destroy the last replication system, leaving the only valid ones at indices 0 and 1
	DestroyServer(TestServers[4]);
	TestServers[4] = nullptr;

	// View should shrink as all the null entries at the end are removed
	TArrayView<UReplicationSystem*> ReplicationSystemsView3 = FReplicationSystemFactory::GetAllReplicationSystems();
	UE_NET_ASSERT_EQ(unsigned(ReplicationSystemsView3.Num()), unsigned(2U));
}

UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, CanReplicateObjectOnMultipleReplicationSystems)
{
	CreateSomeServers();

	UTestReplicatedIrisObject* Object = CreateObject(UTestReplicatedIrisObject::FComponents{});
	BeginReplication(Object);

	for (FReplicationSystemTestServer* Server : GetAllServers())
	{
		UReplicatedTestObjectBridge* ReplicationBridge = Server->GetReplicationBridge();
		FNetRefHandle RefHandle = ReplicationBridge->GetReplicatedRefHandle(Object);
		UE_NET_ASSERT_TRUE(RefHandle.IsValid());
	}
}

UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, ReplicatedObjectIsAssignedGlobalNetHandle)
{
	CreateSomeServers();

	UTestReplicatedIrisObject* Object = CreateObject(UTestReplicatedIrisObject::FComponents{});
	BeginReplication(Object);

	FNetHandle NetHandle = FNetHandleManager::GetNetHandle(Object);
	UE_NET_ASSERT_TRUE(NetHandle.IsValid());
}

UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, ReplicatedObjectLosesGlobalNetHandleAfterEndReplication)
{
	CreateSomeServers();

	UTestReplicatedIrisObject* Object = CreateObject(UTestReplicatedIrisObject::FComponents{});
	BeginReplication(Object);
	EndReplication(Object);

	// The object should no longer be associated with a NetHandle when ending replication on all systems.
	FNetHandle NetHandle = FNetHandleManager::GetNetHandle(Object);
	UE_NET_ASSERT_FALSE(NetHandle.IsValid());
}

UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, ReplicatedObjectKeepsGlobalNetHandleAfterEndReplicationOnSingleSystem)
{
	CreateSomeServers();

	UTestReplicatedIrisObject* Object = CreateObject(UTestReplicatedIrisObject::FComponents{});
	BeginReplication(Object);

	FNetHandle NetHandlePriorToEndReplication = FNetHandleManager::GetNetHandle(Object);

	// End replication on a single system
	FReplicationSystemTestServer* Server = GetAllServers()[0];
	UReplicatedTestObjectBridge* ReplicationBridge = Server->GetReplicationBridge();
	UE_NET_ASSERT_NE(ReplicationBridge, nullptr);
	ReplicationBridge->EndReplication(Object);

	// Make sure replication was ended on the system.
	FNetRefHandle RefHandle = ReplicationBridge->GetReplicatedRefHandle(Object);
	UE_NET_ASSERT_FALSE(RefHandle.IsValid());

	// Validate there's still a global NetHandle assigned.
	FNetHandle NetHandleAfterSingleEndReplication = FNetHandleManager::GetNetHandle(Object);
	UE_NET_ASSERT_EQ(NetHandlePriorToEndReplication, NetHandleAfterSingleEndReplication);
}

UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, ObjectIsReplicatedToAllClientsOnAllSystems)
{
	CreateSomeServers();

	// Create clients for all systems
	for (FReplicationSystemTestServer* Server : GetAllServers())
	{
		constexpr unsigned ClientCountPerServer = 3U;
		for (unsigned ClientIt = 0, ClientEndIt = ClientCountPerServer; ClientIt != ClientEndIt; ++ClientIt)
		{
			CreateClientForServer(Server);
		}
	}

	UTestReplicatedIrisObject* ServerObject = CreateObject(UTestReplicatedIrisObject::FComponents{});
	BeginReplication(ServerObject);

	FullSendAndDeliverUpdate();

	// Verify the object was created on all clients
	for (FReplicationSystemTestServer* Server : GetAllServers())
	{
		FNetRefHandle RefHandleOnServer = Server->GetReplicationBridge()->GetReplicatedRefHandle(ServerObject);
		for (FReplicationSystemTestClient* Client : GetClients(Server))
		{
			UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(RefHandleOnServer), nullptr);
		}
	}
}

UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, ReplicatedObjectIsDestroyedOnAllClientsAfterEndReplication)
{
	CreateSomeServers();

	// Create clients for all systems
	for (FReplicationSystemTestServer* Server : GetAllServers())
	{
		constexpr unsigned ClientCountPerServer = 3;
		for (unsigned ClientIt = 0, ClientEndIt = ClientCountPerServer; ClientIt != ClientEndIt; ++ClientIt)
		{
			CreateClientForServer(Server);
		}
	}

	UTestReplicatedIrisObject* ServerObject = CreateObject(UTestReplicatedIrisObject::FComponents{});
	BeginReplication(ServerObject);

	TArray<FNetRefHandle> ServerRefHandles;
	ServerRefHandles.Reserve(GetAllServers().Num());
	for (FReplicationSystemTestServer* Server : GetAllServers())
	{
		FNetRefHandle RefHandle = Server->GetReplicationBridge()->GetReplicatedRefHandle(ServerObject);
		ServerRefHandles.Add(RefHandle);
	}

	FullSendAndDeliverUpdate();

	EndReplication(ServerObject);

	FullSendAndDeliverUpdate();

	// Verify the object was destroyed on all clients
	unsigned ServerIt = 0;
	for (FReplicationSystemTestServer* Server : GetAllServers())
	{
		FNetRefHandle RefHandleOnServer = ServerRefHandles[ServerIt++];
		for (FReplicationSystemTestClient* Client : GetClients(Server))
		{
			UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(RefHandleOnServer), nullptr);
		}
	}
}

UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, ReplicatedObjectCanStopReplicatingOnSingleSystem)
{
	CreateSomeServers();

	// Create clients for all systems
	for (FReplicationSystemTestServer* Server : GetAllServers())
	{
		CreateClientForServer(Server);
	}

	UTestReplicatedIrisObject* ServerObject = CreateObject(UTestReplicatedIrisObject::FComponents{});
	BeginReplication(ServerObject);

	FullSendAndDeliverUpdate();

	// End replication on single system
	constexpr SIZE_T SystemIndexToEndReplicationOn = 0;
	FReplicationSystemTestServer* ServerToEndReplicationOn = GetAllServers()[SystemIndexToEndReplicationOn];
	FNetRefHandle RefHandleOnServerToEndReplicationOn;
	{
		UReplicatedTestObjectBridge* ReplicationBridge = ServerToEndReplicationOn->GetReplicationBridge();
		RefHandleOnServerToEndReplicationOn = ReplicationBridge->GetReplicatedRefHandle(ServerObject);
		ReplicationBridge->EndReplication(ServerObject);
	}

	FullSendAndDeliverUpdate();

	// Verify object was destroyed on client connected to system where replication was ended
	for (FReplicationSystemTestServer* Server : GetAllServers())
	{
		if (Server == ServerToEndReplicationOn)
		{
			for (FReplicationSystemTestClient* Client : GetClients(ServerToEndReplicationOn))
			{
				UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(RefHandleOnServerToEndReplicationOn), nullptr);
			}
		}
		else
		{
			FNetRefHandle RefHandleOnServer = Server->GetReplicationBridge()->GetReplicatedRefHandle(ServerObject);
			for (FReplicationSystemTestClient* Client : GetClients(Server))
			{
				UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(RefHandleOnServer), nullptr);
			}
		}
	}
}

UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, ModifiedObjectIsReplicatedToAllClientsOnAllSystems)
{
	CreateSomeServers();

	// Create clients for all systems
	for (FReplicationSystemTestServer* Server : GetAllServers())
	{
		constexpr unsigned ClientCountPerServer = 1U;
		for (unsigned ClientIt = 0, ClientEndIt = ClientCountPerServer; ClientIt != ClientEndIt; ++ClientIt)
		{
			CreateClientForServer(Server);
		}
	}

	UTestReplicatedIrisObject* ServerObject = CreateObject(UTestReplicatedIrisObject::FComponents{});
	BeginReplication(ServerObject);

	FullSendAndDeliverUpdate();

	ServerObject->IntA ^= 4711;
	const int32 ExpectedIntAVAlue = ServerObject->IntA;

	FullSendAndDeliverUpdate();

	// Verify the object has the updated value on all clients
	for (FReplicationSystemTestServer* Server : GetAllServers())
	{
		FNetRefHandle RefHandleOnServer = Server->GetReplicationBridge()->GetReplicatedRefHandle(ServerObject);
		for (FReplicationSystemTestClient* Client : GetClients(Server))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(RefHandleOnServer));
			CA_ASSUME(ClientObject != nullptr);
			UE_NET_ASSERT_EQ(ClientObject->IntA, ExpectedIntAVAlue);
		}
	}
}

/** Test that independent objects in separate systems are dirtied properly */
UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, GlobalDirtyTracker)
{
	constexpr int32 NumServers = 3;
	CreateServers(NumServers);

	TArrayView<FReplicationSystemTestServer*> AllServers = GetAllServers();
	UE_NET_ASSERT_EQ(AllServers.Num(), NumServers);

	TStaticArray<UTestReplicatedIrisObject*, NumServers> ServerObjects = {};

	// Create clients and replicated objects for all systems
	for (uint32 i = 0; i < NumServers; ++i)
	{
		constexpr unsigned ClientCountPerServer = 3U;
		for (unsigned ClientIt = 0, ClientEndIt = ClientCountPerServer; ClientIt != ClientEndIt; ++ClientIt)
		{
			CreateClientForServer(AllServers[i]);
		}

		// Spawn object on server that won't be polled automatically multiple times during this test
		UObjectReplicationBridge::FRootObjectReplicationParams Params;
		Params.PollFrequency = 0.001f;
		Params.bUseClassConfigDynamicFilter = true;
		Params.bNeedsPreUpdate = true;

		ServerObjects[i] = AllServers[i]->CreateObject(Params);
	}

	// Send and deliver packets
	FullSendAndDeliverUpdateTwoPass();

	// Objects should have been created on the clients
	for (int32 i = 0; i < NumServers; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[i]->NetRefHandle));
			UE_NET_ASSERT_NE(ClientObject, nullptr);
		}
	}

	// Set a replicated variable, but don't mark it dirty
	for (UTestReplicatedIrisObject* ServerObject : ServerObjects)
	{
		ServerObject->IntA = 0xFF;
	}

	// Send and deliver packets
	FullSendAndDeliverUpdateTwoPass();

	// Client replicated property should not have changed
	for (int32 i = 0; i < NumServers; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[i]->NetRefHandle));
			UE_NET_ASSERT_NE(ClientObject->IntA, ServerObjects[i]->IntA);
		}
	}

	// Now mark the object dirty
	for (uint32 i = 0; i < NumServers; ++i)
	{
		FGlobalDirtyNetObjectTracker::MarkNetObjectStateDirty(FNetHandleManager::GetOrCreateNetHandle(ServerObjects[i]));
	}

	// Send and deliver packets
	FullSendAndDeliverUpdateTwoPass();

	// Client replicated properties should have changed now
	for (int32 i = 0; i < NumServers; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[i]->NetRefHandle));
			UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObjects[i]->IntA);
		}

		AllServers[i]->DestroyObject(ServerObjects[i]);
	}
}

/** Test the global dirty tracker with multiple systems where one has no clients */
UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, GlobalDirtyTrackerWithNoClients)
{
	constexpr int32 NumServers = 3;
	constexpr int32 NumServersWithClients = NumServers - 1;
	CreateServers(NumServers);

	TArrayView<FReplicationSystemTestServer*> AllServers = GetAllServers();
	UE_NET_ASSERT_EQ(AllServers.Num(), NumServers);

	TStaticArray<UTestReplicatedIrisObject*, NumServersWithClients> ServerObjects = {};

	// Create clients and replicated objects for all but one of the servers
	for (uint32 i = 0; i < NumServersWithClients; ++i)
	{
		constexpr unsigned ClientCountPerServer = 3U;
		for (unsigned ClientIt = 0, ClientEndIt = ClientCountPerServer; ClientIt != ClientEndIt; ++ClientIt)
		{
			CreateClientForServer(AllServers[i]);
		}

		// Spawn object on server that won't be polled automatically multiple times during this test
		UObjectReplicationBridge::FRootObjectReplicationParams Params;
		Params.PollFrequency = 0.001f;
		Params.bUseClassConfigDynamicFilter = true;
		Params.bNeedsPreUpdate = true;

		ServerObjects[i] = AllServers[i]->CreateObject(Params);
	}

	// Send and deliver packets
	FullSendAndDeliverUpdateTwoPass();

	// Objects should have been created on the clients
	for (int32 i = 0; i < NumServersWithClients; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[i]->NetRefHandle));
			UE_NET_ASSERT_NE(ClientObject, nullptr);
		}
	}

	// Set a replicated variable, but don't mark it dirty
	for (UTestReplicatedIrisObject* ServerObject : ServerObjects)
	{
		ServerObject->IntA = 0xFF;
	}

	// Send and deliver packets
	FullSendAndDeliverUpdateTwoPass();

	// Client replicated property should not have changed
	for (int32 i = 0; i < NumServersWithClients; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[i]->NetRefHandle));
			UE_NET_ASSERT_NE(ClientObject->IntA, ServerObjects[i]->IntA);
		}
	}

	// Now mark the object dirty
	for (uint32 i = 0; i < NumServersWithClients; ++i)
	{
		FGlobalDirtyNetObjectTracker::MarkNetObjectStateDirty(FNetHandleManager::GetOrCreateNetHandle(ServerObjects[i]));
	}

	// Send and deliver packets
	FullSendAndDeliverUpdateTwoPass();

	// Client replicated properties should have changed now
	for (int32 i = 0; i < NumServersWithClients; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[i]->NetRefHandle));
			UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObjects[i]->IntA);
		}

		AllServers[i]->DestroyObject(ServerObjects[i]);
	}
}

/** Test that independent objects in separate systems are dirtied properly and independently */
UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, GlobalDirtyTrackerOneObjectDirty)
{
	constexpr int32 NumServers = 3;
	CreateServers(NumServers);

	TArrayView<FReplicationSystemTestServer*> AllServers = GetAllServers();
	UE_NET_ASSERT_EQ(AllServers.Num(), NumServers);

	TStaticArray<UTestReplicatedIrisObject*, NumServers> ServerObjects = {};

	// Create clients and replicated objects for all systems
	for (uint32 i = 0; i < NumServers; ++i)
	{
		constexpr unsigned ClientCountPerServer = 3U;
		for (unsigned ClientIt = 0, ClientEndIt = ClientCountPerServer; ClientIt != ClientEndIt; ++ClientIt)
		{
			CreateClientForServer(AllServers[i]);
		}

		// Spawn object on server that won't be polled automatically multiple times during this test
		UObjectReplicationBridge::FRootObjectReplicationParams Params;
		Params.PollFrequency = 0.001f;
		Params.bUseClassConfigDynamicFilter = true;
		Params.bNeedsPreUpdate = true;

		ServerObjects[i] = AllServers[i]->CreateObject(Params);
	}

	// Send and deliver packets
	FullSendAndDeliverUpdateTwoPass();

	// Objects should have been created on the clients
	for (int32 i = 0; i < NumServers; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[i]->NetRefHandle));
			UE_NET_ASSERT_NE(ClientObject, nullptr);
		}
	}

	// Set a replicated variable, but don't mark it dirty
	for (UTestReplicatedIrisObject* ServerObject : ServerObjects)
	{
		ServerObject->IntA = 0xFF;
	}

	// Send and deliver packets
	FullSendAndDeliverUpdateTwoPass();

	// Client replicated property should not have changed
	for (int32 i = 0; i < NumServers; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[i]->NetRefHandle));
			UE_NET_ASSERT_NE(ClientObject->IntA, ServerObjects[i]->IntA);
		}
	}

	// Now only mark server 1's object dirty
	FGlobalDirtyNetObjectTracker::MarkNetObjectStateDirty(FNetHandleManager::GetOrCreateNetHandle(ServerObjects[1]));

	// Send and deliver packets
	FullSendAndDeliverUpdateTwoPass();

	// Client replicated property should have changed now on server 1...
	for (FReplicationSystemTestClient* Client : GetClients(AllServers[1]))
	{
		UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[1]->NetRefHandle));
		UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObjects[1]->IntA);
	}

	// But not server 0 or 2.
	for (FReplicationSystemTestClient* Client : GetClients(AllServers[0]))
	{
		UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[0]->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject->IntA, ServerObjects[0]->IntA);
	}

	for (FReplicationSystemTestClient* Client : GetClients(AllServers[2]))
	{
		UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[2]->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject->IntA, ServerObjects[2]->IntA);
	}

	for (int32 i = 0; i < NumServers; ++i)
	{
		AllServers[i]->DestroyObject(ServerObjects[i]);
	}
}

/** Test global dirty tracking with multiple repsystems and a late joining client */
UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, GlobalDirtyTrackerLateJoinClient)
{
	constexpr int32 NumServers = 2;
	constexpr int32 NumServersWithClients = NumServers - 1;
	CreateServers(NumServers);

	TArrayView<FReplicationSystemTestServer*> AllServers = GetAllServers();
	UE_NET_ASSERT_EQ(AllServers.Num(), NumServers);

	TStaticArray<UTestReplicatedIrisObject*, NumServers> ServerObjects = {};

	// Create client for one server and replicated objects for all servers
	FReplicationSystemTestClient* Server0Client = CreateClientForServer(AllServers[0]);

	for (uint32 i = 0; i < NumServers; ++i)
	{
		// Spawn object on server that won't be polled automatically multiple times during this test
		UObjectReplicationBridge::FRootObjectReplicationParams Params;
		Params.PollFrequency = 0.001f;
		Params.bUseClassConfigDynamicFilter = true;
		Params.bNeedsPreUpdate = true;

		ServerObjects[i] = AllServers[i]->CreateObject(Params);
	}

	// Send and deliver packets
	FullSendAndDeliverUpdateTwoPass();

	// Object should have been created on the client
	UTestReplicatedIrisObject* Client0Object = Cast<UTestReplicatedIrisObject>(Server0Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[0]->NetRefHandle));
	UE_NET_ASSERT_NE(Client0Object, nullptr);

	// Set a replicated variable, but don't mark it dirty
	for (UTestReplicatedIrisObject* ServerObject : ServerObjects)
	{
		ServerObject->IntA = 0xFF;
	}

	// Send and deliver packets
	FullSendAndDeliverUpdateTwoPass();

	// Client replicated property should not have changed
	UE_NET_ASSERT_NE(Client0Object->IntA, ServerObjects[0]->IntA);

	// Now mark the object dirty
	for (UTestReplicatedIrisObject* ServerObject : ServerObjects)
	{
		FGlobalDirtyNetObjectTracker::MarkNetObjectStateDirty(FNetHandleManager::GetOrCreateNetHandle(ServerObject));
	}

	// Send and deliver packets, and reset global dirty tracker
	FullSendAndDeliverUpdateTwoPass();

	// Add client to second server/repsystem
	FReplicationSystemTestClient* Server1Client = CreateClientForServer(AllServers[1]);

	// Send and deliver packets
	FullSendAndDeliverUpdateTwoPass();

	UTestReplicatedIrisObject* Client1Object = Cast<UTestReplicatedIrisObject>(Server1Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[1]->NetRefHandle));
	UE_NET_ASSERT_NE(Client1Object, nullptr);
	UE_NET_ASSERT_EQ(Client1Object->IntA, ServerObjects[1]->IntA);

	// Client replicated properties should have changed now
	for (int32 i = 0; i < NumServers; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[i]->NetRefHandle));
			UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObjects[i]->IntA);
		}

		AllServers[i]->DestroyObject(ServerObjects[i]);
	}
}

/** Test that validates that a push model enabled object is marked as dirty inside PreUpdate/PreReplication with multiple repsystems */
UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, GlobalDirtyTrackerInsidePreUpdate)
{
	constexpr int32 NumServers = 2;
	CreateServers(NumServers);

	TArrayView<FReplicationSystemTestServer*> AllServers = GetAllServers();
	UE_NET_ASSERT_EQ(AllServers.Num(), NumServers);

	TStaticArray<UTestReplicatedIrisObject*, NumServers> ServerObjects = {};

	// Create a client and an object for each server
	for (uint32 i = 0; i < NumServers; ++i)
	{
		CreateClientForServer(AllServers[i]);

		// Spawn object on server that is polled every frame
		UObjectReplicationBridge::FRootObjectReplicationParams Params;
		Params.bNeedsPreUpdate = true;
		UTestReplicatedIrisObject::FComponents ComponentsToCreate = { .ObjectReferenceComponentCount = 1 };

		ServerObjects[i] = AllServers[i]->CreateObject(Params, &ComponentsToCreate);
	}

	// Send and deliver packet
	FullSendAndDeliverUpdateTwoPass();

	// Objects should have been created on the clients
	for (int32 i = 0; i < NumServers; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[i]->NetRefHandle));
			UE_NET_ASSERT_NE(ClientObject, nullptr);
		}
	}

	auto PreUpdateObjectForServer = [&](int32 ServerIndex)
		{
			return [&ServerObjects, ServerIndex](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge)
				{
					for (UObject* InObject : Instances)
					{
						if (InObject == ServerObjects[ServerIndex])
						{
							// Dirty the object in the global dirty tracker
							ServerObjects[ServerIndex]->ObjectReferenceComponents[0]->ModifyIntA();
						}
					}
				};
		};

	// Mark a property dirty on the first server during PreUpdate. As the object is polled every frame we expect the property to be updated on the client.
	AllServers[0]->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObjectForServer(0));

	// Send and deliver packet
	FullSendAndDeliverUpdateTwoPass();

	// The property should be updated on the client
	for (int32 i = 0; i < NumServers; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[i]->NetRefHandle));
			UE_NET_ASSERT_EQ(ClientObject->ObjectReferenceComponents[0]->IntA, ServerObjects[i]->ObjectReferenceComponents[0]->IntA);
		}
	}

	// Now use the other server/repsystem to mark dirty
	AllServers[0]->GetReplicationBridge()->SetExternalPreUpdateFunctor([](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge) {});
	AllServers[1]->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObjectForServer(1));

	// Send and deliver packet
	FullSendAndDeliverUpdateTwoPass();

	// The property should still be equal on the client of server 0
	FReplicationSystemTestClient* Client0 = GetClients(AllServers[0])[0];
	UTestReplicatedIrisObject* ClientObject0 = Cast<UTestReplicatedIrisObject>(Client0->GetReplicationBridge()->GetReplicatedObject(ServerObjects[0]->NetRefHandle));
	UE_NET_ASSERT_EQ(ClientObject0->ObjectReferenceComponents[0]->IntA, ServerObjects[0]->ObjectReferenceComponents[0]->IntA);

	// The property should be updated on the client of server 1 now since it re-polls dirtiness after the PreUpdate
	FReplicationSystemTestClient* Client1 = GetClients(AllServers[1])[0];
	UTestReplicatedIrisObject* ClientObject1 = Cast<UTestReplicatedIrisObject>(Client1->GetReplicationBridge()->GetReplicatedObject(ServerObjects[1]->NetRefHandle));
	UE_NET_ASSERT_EQ(ClientObject1->ObjectReferenceComponents[0]->IntA, ServerObjects[1]->ObjectReferenceComponents[0]->IntA);
}

/** Test that validates that a push model enabled object is marked as dirty inside PreUpdate/PreReplication of a different replication system */
UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, GlobalDirtyTrackerInsidePreUpdateCrossSystems)
{
	constexpr int32 NumServers = 2;
	CreateServers(NumServers);

	TArrayView<FReplicationSystemTestServer*> AllServers = GetAllServers();
	UE_NET_ASSERT_EQ(AllServers.Num(), NumServers);

	TStaticArray<UTestReplicatedIrisObject*, NumServers> ServerObjects = {};

	// Create a client and an object for each server
	for (uint32 i = 0; i < NumServers; ++i)
	{
		CreateClientForServer(AllServers[i]);

		// Spawn object on server that is polled every frame
		UObjectReplicationBridge::FRootObjectReplicationParams Params;
		Params.bNeedsPreUpdate = true;
		UTestReplicatedIrisObject::FComponents ComponentsToCreate = { .ObjectReferenceComponentCount = 1 };

		ServerObjects[i] = AllServers[i]->CreateObject(Params, &ComponentsToCreate);
	}

	// Send and deliver packet
	FullSendAndDeliverUpdateTwoPass();

	// Objects should have been created on the clients
	for (int32 i = 0; i < NumServers; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[i]->NetRefHandle));
			UE_NET_ASSERT_NE(ClientObject, nullptr);
		}
	}

	auto PreUpdateObjectForServer = [&](int32 ServerIndex)
		{
			return [&ServerObjects, ServerIndex](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge)
				{
					// Dirty the object in the global dirty tracker
					ServerObjects[ServerIndex]->ObjectReferenceComponents[0]->ModifyIntA();
				};
		};

	// On server 0's PreUpdate, mark server 1's object dirty.
	AllServers[0]->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObjectForServer(1));

	// Send and deliver packet
	FullSendAndDeliverUpdateTwoPass();

	// The property should be updated on the client
	for (int32 i = 0; i < NumServers; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[i]->NetRefHandle));
			UE_NET_ASSERT_EQ(ClientObject->ObjectReferenceComponents[0]->IntA, ServerObjects[i]->ObjectReferenceComponents[0]->IntA);
		}
	}

	// This time on server 1's PreUpdate, mark server 0's object dirty.
	AllServers[0]->GetReplicationBridge()->SetExternalPreUpdateFunctor([](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge) {});
	AllServers[1]->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObjectForServer(0));

	// Send and deliver packet
	FullSendAndDeliverUpdateTwoPass();

	// Since ServerObjects[0] was marked dirty after server 0 updated, the client's object should not have changed yet.
	FReplicationSystemTestClient* Client0 = GetClients(AllServers[0])[0];
	UTestReplicatedIrisObject* ClientObject0 = Cast<UTestReplicatedIrisObject>(Client0->GetReplicationBridge()->GetReplicatedObject(ServerObjects[0]->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject0->ObjectReferenceComponents[0]->IntA, ServerObjects[0]->ObjectReferenceComponents[0]->IntA);

	// Client object 1 should still match server 1
	FReplicationSystemTestClient* Client1 = GetClients(AllServers[1])[0];
	UTestReplicatedIrisObject* ClientObject1 = Cast<UTestReplicatedIrisObject>(Client1->GetReplicationBridge()->GetReplicatedObject(ServerObjects[1]->NetRefHandle));
	UE_NET_ASSERT_EQ(ClientObject1->ObjectReferenceComponents[0]->IntA, ServerObjects[1]->ObjectReferenceComponents[0]->IntA);

	// Clear the PreUpdate functors
	AllServers[0]->GetReplicationBridge()->SetExternalPreUpdateFunctor([](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge) {});
	AllServers[1]->GetReplicationBridge()->SetExternalPreUpdateFunctor([](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge) {});

	// Send and deliver packet. This should update client 0 to have the latest server value.
	FullSendAndDeliverUpdateTwoPass();

	// Both clients should match their servers now
	UE_NET_ASSERT_EQ(ClientObject0->ObjectReferenceComponents[0]->IntA, ServerObjects[0]->ObjectReferenceComponents[0]->IntA);
	UE_NET_ASSERT_EQ(ClientObject1->ObjectReferenceComponents[0]->IntA, ServerObjects[1]->ObjectReferenceComponents[0]->IntA);


	// This time both servers mark each other's object dirty.
	AllServers[0]->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObjectForServer(1));
	AllServers[1]->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObjectForServer(0));

	// Send and deliver packet
	FullSendAndDeliverUpdateTwoPass();

	// Since ServerObjects[0] was marked dirty after server 0 updated, the client's object should not have changed yet.
	UE_NET_ASSERT_NE(ClientObject0->ObjectReferenceComponents[0]->IntA, ServerObjects[0]->ObjectReferenceComponents[0]->IntA);

	// Since ServerObjects[1] was marked dirty before server 1 updated, the client object 1 should be updated.
	UE_NET_ASSERT_EQ(ClientObject1->ObjectReferenceComponents[0]->IntA, ServerObjects[1]->ObjectReferenceComponents[0]->IntA);

	// Clear the PreUpdate functors
	AllServers[0]->GetReplicationBridge()->SetExternalPreUpdateFunctor([](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge) {});
	AllServers[1]->GetReplicationBridge()->SetExternalPreUpdateFunctor([](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge) {});

	// Send and deliver packet. This should update client 0 to have the latest server value.
	FullSendAndDeliverUpdateTwoPass();

	// Both clients should match their servers now
	UE_NET_ASSERT_EQ(ClientObject0->ObjectReferenceComponents[0]->IntA, ServerObjects[0]->ObjectReferenceComponents[0]->IntA);
	UE_NET_ASSERT_EQ(ClientObject1->ObjectReferenceComponents[0]->IntA, ServerObjects[1]->ObjectReferenceComponents[0]->IntA);

}

/** Test that the same object in separate systems is dirtied & replicated properly */
UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, GlobalDirtyTracker_SameObject)
{
	constexpr int32 NumServers = 3;
	CreateServers(NumServers);

	TArrayView<FReplicationSystemTestServer*> AllServers = GetAllServers();
	UE_NET_ASSERT_EQ(AllServers.Num(), NumServers);

	TStaticArray<FNetRefHandle, NumServers> RepSystemRefHandles = {};
	
	// Spawn object on server that won't be polled automatically multiple times during this test
	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	Params.PollFrequency = 0.001f;
	UTestReplicatedIrisObject* ServerObject = CreateObject({});

	// Create clients for all systems
	for (uint32 i = 0; i < NumServers; ++i)
	{
		constexpr unsigned ClientCountPerServer = 3U;
		for (unsigned ClientIt = 0, ClientEndIt = ClientCountPerServer; ClientIt != ClientEndIt; ++ClientIt)
		{
			CreateClientForServer(AllServers[i]);
		}

		RepSystemRefHandles[i] = BeginReplication(AllServers[i], ServerObject, Params);
	}

	// Send and deliver packets
	FullSendAndDeliverUpdateTwoPass();

	// Objects should have been created on the clients
	for (int32 i = 0; i < NumServers; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(RepSystemRefHandles[i]));
			UE_NET_ASSERT_NE(ClientObject, nullptr);
		}
	}

	// Set a replicated variable, but don't mark it dirty
	ServerObject->IntA = 0xFF;

	// Send and deliver packets
	FullSendAndDeliverUpdateTwoPass();

	// Client replicated property should not have changed
	for (int32 i = 0; i < NumServers; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(RepSystemRefHandles[i]));
			UE_NET_ASSERT_NE(ClientObject->IntA, ServerObject->IntA);
		}
	}

	// Now mark the object dirty
	FGlobalDirtyNetObjectTracker::MarkNetObjectStateDirty(FNetHandleManager::GetOrCreateNetHandle(ServerObject));

	// Send and deliver packets
	FullSendAndDeliverUpdateTwoPass();

	// Client replicated properties should have changed now
	for (int32 i = 0; i < NumServers; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(RepSystemRefHandles[i]));
			UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
		}
	}
}

/** Test that validates that a single push model enabled object replicated by multiple systems marked dirty inside one PreUpdate/PreReplication replicates properly */
UE_NET_TEST_FIXTURE(FMultiReplicationSystemsTestFixture, GlobalDirtyTrackerInsidePreUpdate_SameObject)
{
	constexpr int32 NumServers = 2;
	CreateServers(NumServers);

	TArrayView<FReplicationSystemTestServer*> AllServers = GetAllServers();
	UE_NET_ASSERT_EQ(AllServers.Num(), NumServers);

	TStaticArray<FNetRefHandle, NumServers> RepSystemRefHandles = {};

	UTestReplicatedIrisObject* ServerObject = CreateObject({ .ObjectReferenceComponentCount = 1 });

	// Create a client for each server
	for (uint32 i = 0; i < NumServers; ++i)
	{
		CreateClientForServer(AllServers[i]);

		// Spawn object on server that is polled every frame
		UObjectReplicationBridge::FRootObjectReplicationParams Params;
		Params.bNeedsPreUpdate = true;

		RepSystemRefHandles[i] = BeginReplication(AllServers[i], ServerObject, Params);
	}

	// Send and deliver packet
	FullSendAndDeliverUpdateTwoPass();

	// Objects should have been created on the clients
	for (int32 i = 0; i < NumServers; ++i)
	{
		for (FReplicationSystemTestClient* Client : GetClients(AllServers[i]))
		{
			UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(RepSystemRefHandles[i]));
			UE_NET_ASSERT_NE(ClientObject, nullptr);
		}
	}

	auto PreUpdateObject = [&](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge)
		{
			for (UObject* InObject : Instances)
			{
				if (InObject == ServerObject)
				{
					// Dirty the object in the global dirty tracker
					ServerObject->ObjectReferenceComponents[0]->ModifyIntA();
				}
			}
		};

	// Mark a property dirty on the first server during PreUpdate. As the object is polled every frame we expect the property to be updated on the client.
	AllServers[0]->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObject);

	// Send and deliver packet
	FullSendAndDeliverUpdateTwoPass();

	// The property should be updated on the clients
	FReplicationSystemTestClient* Client0 = GetClients(AllServers[0])[0];
	UTestReplicatedIrisObject* ClientObject0 = Cast<UTestReplicatedIrisObject>(Client0->GetReplicationBridge()->GetReplicatedObject(RepSystemRefHandles[0]));
	UE_NET_ASSERT_EQ(ClientObject0->ObjectReferenceComponents[0]->IntA, ServerObject->ObjectReferenceComponents[0]->IntA);

	FReplicationSystemTestClient* Client1 = GetClients(AllServers[1])[0];
	UTestReplicatedIrisObject* ClientObject1 = Cast<UTestReplicatedIrisObject>(Client1->GetReplicationBridge()->GetReplicatedObject(RepSystemRefHandles[1]));
	UE_NET_ASSERT_EQ(ClientObject1->ObjectReferenceComponents[0]->IntA, ServerObject->ObjectReferenceComponents[0]->IntA);


	// Now use the other server/repsystem to mark dirty
	AllServers[0]->GetReplicationBridge()->SetExternalPreUpdateFunctor([](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge) {});
	AllServers[1]->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObject);

	// Send and deliver packet
	FullSendAndDeliverUpdateTwoPass();

	// The property on client 0 won't be updated yet because the server only updated the property when rep system 1 updated (after replication to client 0)
	UE_NET_ASSERT_NE(ClientObject0->ObjectReferenceComponents[0]->IntA, ServerObject->ObjectReferenceComponents[0]->IntA);

	// The property should be updated on the client of server 1 now since it re-polls dirtiness after the PreUpdate
	UE_NET_ASSERT_EQ(ClientObject1->ObjectReferenceComponents[0]->IntA, ServerObject->ObjectReferenceComponents[0]->IntA);

	// Clear the PreUpdate functors
	AllServers[0]->GetReplicationBridge()->SetExternalPreUpdateFunctor([](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge) {});
	AllServers[1]->GetReplicationBridge()->SetExternalPreUpdateFunctor([](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge) {});

	// Send and deliver packet. This should update client 0 to have the server value.
	FullSendAndDeliverUpdateTwoPass();

	// The property should be updated on all clients now
	UE_NET_ASSERT_EQ(ClientObject0->ObjectReferenceComponents[0]->IntA, ServerObject->ObjectReferenceComponents[0]->IntA);
	UE_NET_ASSERT_EQ(ClientObject1->ObjectReferenceComponents[0]->IntA, ServerObject->ObjectReferenceComponents[0]->IntA);


	// Now have both systems mark the object dirty in each PreUpdate
	const int32 ServerValuePreUpdate = ServerObject->ObjectReferenceComponents[0]->IntA;
	AllServers[0]->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObject);
	AllServers[1]->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObject);

	// Send and deliver packet
	FullSendAndDeliverUpdateTwoPass();

	// The property on client 0 should be the server value + 1 since one PreUpdate ran before replicating to this client.
	UE_NET_ASSERT_EQ(ClientObject0->ObjectReferenceComponents[0]->IntA, ServerValuePreUpdate + 1);

	// The property on client 1 should be the server since both PreUpdates ran before replicating to this client.
	UE_NET_ASSERT_EQ(ClientObject1->ObjectReferenceComponents[0]->IntA, ServerObject->ObjectReferenceComponents[0]->IntA);

	// Clear the PreUpdate functors
	AllServers[0]->GetReplicationBridge()->SetExternalPreUpdateFunctor([](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge) {});
	AllServers[1]->GetReplicationBridge()->SetExternalPreUpdateFunctor([](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge) {});

	// Send and deliver packet. This should update client 0 to have the latest server value.
	FullSendAndDeliverUpdateTwoPass();

	// The property should be updated on all clients now
	UE_NET_ASSERT_EQ(ClientObject0->ObjectReferenceComponents[0]->IntA, ServerObject->ObjectReferenceComponents[0]->IntA);
	UE_NET_ASSERT_EQ(ClientObject1->ObjectReferenceComponents[0]->IntA, ServerObject->ObjectReferenceComponents[0]->IntA);
}

}
