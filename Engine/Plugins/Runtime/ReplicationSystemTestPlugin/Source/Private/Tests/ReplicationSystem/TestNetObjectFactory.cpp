// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "ReplicationSystemServerClientTestFixture.h"
#include "ReplicatedTestObjectFactory.h"

#include "HAL/IConsoleManager.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/NetObjectFactoryRegistry.h"

//#include "UObject/CoreNetTypes.h"

namespace UE::Net::Private
{

class FNetObjectFactoryTestFixture : public FNetworkAutomationTestSuiteFixture
{
public:

	virtual void SetUp() override
	{
		Super::SetUp();
	}
	virtual void TearDown() override
	{
		Super::TearDown();
	}

private:
	typedef FNetworkAutomationTestSuiteFixture Super;
};

UE_NET_TEST_FIXTURE(FNetObjectFactoryTestFixture, NetFactoryRegistration)
{
	const FName FactoryA = TEXT("NetFactoryA");
	const FName FactoryB = TEXT("NetFactoryB");
	const FName FactoryC = TEXT("NetFactoryC");

	FNetObjectFactoryRegistry::RegisterFactory(UReplicatedTestObjectFactory::StaticClass(), FactoryA);
	FNetObjectFactoryRegistry::RegisterFactory(UReplicatedTestObjectFactory::StaticClass(), FactoryB);
	FNetObjectFactoryRegistry::RegisterFactory(UReplicatedTestObjectFactory::StaticClass(), FactoryC);

	const FNetObjectFactoryId Id_A = FNetObjectFactoryRegistry::GetFactoryIdFromName(FactoryA);
	const FNetObjectFactoryId Id_B = FNetObjectFactoryRegistry::GetFactoryIdFromName(FactoryB);
	const FNetObjectFactoryId Id_C = FNetObjectFactoryRegistry::GetFactoryIdFromName(FactoryC);

	// Valid the initial factories
	{
		UE_NET_ASSERT_TRUE(FNetObjectFactoryRegistry::IsValidFactoryId(Id_A));
		UE_NET_ASSERT_TRUE(FNetObjectFactoryRegistry::IsValidFactoryId(Id_B));
		UE_NET_ASSERT_TRUE(FNetObjectFactoryRegistry::IsValidFactoryId(Id_C));
	}

	// Unregister a factory
	{
		FNetObjectFactoryRegistry::UnregisterFactory(FactoryB);

		UE_NET_ASSERT_TRUE(FNetObjectFactoryRegistry::GetFactoryIdFromName(FactoryB) == InvalidNetObjectFactoryId);

		UE_NET_ASSERT_TRUE(FNetObjectFactoryRegistry::IsValidFactoryId(Id_A));
		UE_NET_ASSERT_FALSE(FNetObjectFactoryRegistry::IsValidFactoryId(Id_B));
		UE_NET_ASSERT_TRUE(FNetObjectFactoryRegistry::IsValidFactoryId(Id_C));

		UE_NET_ASSERT_TRUE(Id_A == FNetObjectFactoryRegistry::GetFactoryIdFromName(FactoryA));
		UE_NET_ASSERT_TRUE(Id_C == FNetObjectFactoryRegistry::GetFactoryIdFromName(FactoryC));
	}

	// Register yet another factory
	const FName FactoryD = TEXT("NetFactoryD");
	FNetObjectFactoryRegistry::RegisterFactory(UReplicatedTestObjectFactory::StaticClass(), FactoryD);
	const FNetObjectFactoryId Id_D = FNetObjectFactoryRegistry::GetFactoryIdFromName(FactoryD);

	// Validate the new factory
	{
		UE_NET_ASSERT_TRUE(Id_D != InvalidNetObjectFactoryId);
		UE_NET_ASSERT_TRUE(FNetObjectFactoryRegistry::IsValidFactoryId(Id_D));
	}

	// Validate the old factories
	{
		UE_NET_ASSERT_TRUE(FNetObjectFactoryRegistry::IsValidFactoryId(Id_A));
		UE_NET_ASSERT_TRUE(FNetObjectFactoryRegistry::IsValidFactoryId(Id_C));

		UE_NET_ASSERT_TRUE(Id_A == FNetObjectFactoryRegistry::GetFactoryIdFromName(FactoryA));
		UE_NET_ASSERT_TRUE(Id_C == FNetObjectFactoryRegistry::GetFactoryIdFromName(FactoryC));
	}

	// These aren't API validatations but simply show that you need to clear your cached ID when unregistering a factory.
	{
		UE_NET_ASSERT_TRUE(Id_B == Id_D);
		UE_NET_ASSERT_TRUE(FNetObjectFactoryRegistry::IsValidFactoryId(Id_B));
		UE_NET_ASSERT_TRUE(FNetObjectFactoryRegistry::GetFactoryIdFromName(FactoryB) == InvalidNetObjectFactoryId);
	}

	// Create a replication system to test the factory spawning path
	TUniquePtr<FReplicationSystemTestServer> Server(new FReplicationSystemTestServer(GetName()));

	// Destroy the server
	Server.Reset();

	// Unregister all the factories registered in this unit test
	FNetObjectFactoryRegistry::UnregisterFactory(FactoryA);
	//B is already unregistered FNetObjectFactoryRegistry::UnregisterFactory(FactoryB);
	FNetObjectFactoryRegistry::UnregisterFactory(FactoryC);
	FNetObjectFactoryRegistry::UnregisterFactory(FactoryD);
}

} // end namespace UE::Net::Private