// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "UObject/CoreNetTypes.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemTypes.h"
#include "Iris/ReplicationSystem/Conditionals/ReplicationCondition.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"

namespace UE::Net::Private
{

class FTestObjectDeltaSerialization : public FReplicationSystemServerClientTestFixture
{
protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	void AllowNewBaselineCreation();
	void DisallowNewBaselineCreation();

private:
	IConsoleVariable* CvarIrisDeltaCompression = nullptr;
	IConsoleVariable* CvarIrisFramesBetweenBaselines = nullptr;

	bool bDeltaCompressionEnable = false;
	int32 MinimuNumberOfFramesBetweenBaselines = -1;
};

UE_NET_TEST_FIXTURE(FTestObjectDeltaSerialization, ClientReceivesLatestValuesWithNewBaseline)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 PropertyComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.PropertyComponentCount = PropertyComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);
	Server->ReplicationSystem->SetDeltaCompressionStatus(ServerObject->NetRefHandle, ENetObjectDeltaCompressionStatus::Allow);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Set value on object and component
	ServerObject->IntA ^= 75;
	ServerObject->Components[0]->IntA ^= 4711;

	AllowNewBaselineCreation();

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Check that the server modified members have their final values.
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
	UE_NET_ASSERT_EQ(ClientObject->Components[0]->IntA, ServerObject->Components[0]->IntA);
}

UE_NET_TEST_FIXTURE(FTestObjectDeltaSerialization, ClientReceivesLatestValuesWithNewBaselineAfterPacketLoss)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 PropertyComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.PropertyComponentCount = PropertyComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);
	Server->ReplicationSystem->SetDeltaCompressionStatus(ServerObject->NetRefHandle, ENetObjectDeltaCompressionStatus::Allow);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Set value on object and component
	ServerObject->IntA ^= 75;
	ServerObject->Components[0]->IntA ^= 4711;

	AllowNewBaselineCreation();

	// Send and do not deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DoNotDeliverPacket);
	Server->PostSendUpdate();

	// Update a property again to force replication
	ServerObject->IntA ^= 1;

	AllowNewBaselineCreation();

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Check that the server modified members have their final values.
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
	UE_NET_ASSERT_EQ(ClientObject->Components[0]->IntA, ServerObject->Components[0]->IntA);
}

UE_NET_TEST_FIXTURE(FTestObjectDeltaSerialization, AllClientsReceivesLatestValuesWithNewBaseline)
{
	// Add clients
	constexpr SIZE_T ClientCount = 3;
	for (SIZE_T ClientIt = 0, ClientEndIt = ClientCount; ClientIt != ClientEndIt; ++ClientIt)
	{
		CreateClient();
	}

	// Spawn object on server
	constexpr uint32 PropertyComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.PropertyComponentCount = PropertyComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);
	Server->ReplicationSystem->SetDeltaCompressionStatus(ServerObject->NetRefHandle, ENetObjectDeltaCompressionStatus::Allow);

	// Send and deliver packets
	Server->NetUpdate();
	for (FReplicationSystemTestClient* Client : Clients)
	{
		Server->SendAndDeliverTo(Client, DeliverPacket);
	}
	Server->PostSendUpdate();

	// Set value on object and component
	ServerObject->IntA ^= 75;
	ServerObject->Components[0]->IntA ^= 4711;

	AllowNewBaselineCreation();

	// Send and deliver packets
	Server->NetUpdate();
	for (FReplicationSystemTestClient* Client : Clients)
	{
		Server->SendAndDeliverTo(Client, DeliverPacket);
	}
	Server->PostSendUpdate();

	// Check that the server modified members have their final values on all clients.
	for (FReplicationSystemTestClient* Client : Clients)
	{
		const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
		UE_NET_ASSERT_EQ(ClientObject->Components[0]->IntA, ServerObject->Components[0]->IntA);
	}
}

UE_NET_TEST_FIXTURE(FTestObjectDeltaSerialization, SimulatedIsProperlyReplicatedAfterBeingAutonomous)
{
	// Add two clients so that a new baseline is not allowed to be created immediately.
	constexpr SIZE_T ClientCount = 2;
	for (SIZE_T ClientIt = 0, ClientEndIt = ClientCount; ClientIt != ClientEndIt; ++ClientIt)
	{
		CreateClient();
	}

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);
	Server->ReplicationSystem->SetDeltaCompressionStatus(ServerObject->NetRefHandle, ENetObjectDeltaCompressionStatus::Allow);

	// Set the client to be autonomous.
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Clients[0]->ConnectionIdOnServer, true);

	// Send and deliver packet
	Server->NetUpdate();
	for (FReplicationSystemTestClient* Client : Clients)
	{
		Server->SendAndDeliverTo(Client, DeliverPacket);
	}
	Server->PostSendUpdate();

	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Set some values that should be replicated for simulated objects
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt ^= 13;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt ^= 37;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt ^= 47;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt ^= 11;

	// Send and deliver packet
	Server->NetUpdate();
	for (FReplicationSystemTestClient* Client : Clients)
	{
		Server->SendAndDeliverTo(Client, DeliverPacket);
	}
	Server->PostSendUpdate();

	// Check that the server modified simulated members are still in the default state
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt, 0);
	
	// Set the client to no longer be autonomous, meaning it should be "simulated".
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Clients[0]->ConnectionIdOnServer, false);

	// Change arbitrary property to trigger replication
	ServerObject->IntA ^= 1;

	// Prevent new baseline creation, in particular for Clients[0]
	DisallowNewBaselineCreation();

	// Send and deliver packet
	Server->NetUpdate();
	for (FReplicationSystemTestClient* Client : Clients)
	{
		Server->SendAndDeliverTo(Client, DeliverPacket);
	}
	Server->PostSendUpdate();

	// Check that the previously server modified simulated members now have the same values on the client
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt);
}

UE_NET_TEST_FIXTURE(FTestObjectDeltaSerialization, SimulatedIsProperlyReplicatedWithNewBaselineAfterBeingAutonomous)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);
	Server->ReplicationSystem->SetDeltaCompressionStatus(ServerObject->NetRefHandle, ENetObjectDeltaCompressionStatus::Allow);

	// Set the client to be autonomous.
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, true);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Set some values that should be replicated for simulated objects
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt ^= 13;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt ^= 37;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt ^= 47;
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt ^= 11;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Check that the server modified simulated members are still in the default state
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt, 0);
	
	// Set the client to no longer be autonomous, meaning it should be "simulated".
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, false);

	// Change arbitrary property to trigger replication
	ServerObject->IntA ^= 1;

	// Allow a new baseline to be created.
	AllowNewBaselineCreation();
	
	// Send and do not deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DoNotDeliverPacket);
	Server->PostSendUpdate();

	// Check that the server modified simulated members are still in the default state
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt, 0);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Check that the previously server modified simulated members now have the same values on the client
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOnlyNoReplayInt);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt, ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsNoReplayInt);
}

UE_NET_TEST_FIXTURE(FTestObjectDeltaSerialization, SubObjectSimulatedIsProperlyReplicatedAfterBeingAutonomous)
{
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with subobject on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, {.ConnectionFilteredComponentCount = 1});
	Server->ReplicationSystem->SetDeltaCompressionStatus(ServerSubObject->NetRefHandle, ENetObjectDeltaCompressionStatus::Allow);

	// Set the root object to be autonomous.
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, true);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Add some values to the simulated only array
	for (int32 Value : {4711, 1337})
	{
		ServerSubObject->ConnectionFilteredComponents[0]->SimulatedOnlyIntArray.Add(Value);
	}

	// In order to actually send the object we need to modify a non-SimulatedOnly property too
	ServerSubObject->ConnectionFilteredComponents[0]->NoneInt += 1;

	// We want a new baseline to be created
	AllowNewBaselineCreation();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));

	// Verify the simulated only array is unaffected by the server changes
	UE_NET_ASSERT_TRUE(ClientSubObject->ConnectionFilteredComponents[0]->SimulatedOnlyIntArray.IsEmpty());

	// Set the client to no longer be autonomous, meaning it should be "simulated".
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, false);

	// Add some values to the simulated only array
	for (int32 Value : {1, -1})
	{
		ServerSubObject->ConnectionFilteredComponents[0]->SimulatedOnlyIntArray.Add(Value);
	}

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Check that the array is properly replicated in its entirety
	UE_NET_ASSERT_EQ(ClientSubObject->ConnectionFilteredComponents[0]->SimulatedOnlyIntArray.Num(), ServerSubObject->ConnectionFilteredComponents[0]->SimulatedOnlyIntArray.Num());
	for (int Index = 0; int32 Value : ServerSubObject->ConnectionFilteredComponents[0]->SimulatedOnlyIntArray)
	{
		UE_NET_ASSERT_EQ(ClientSubObject->ConnectionFilteredComponents[0]->SimulatedOnlyIntArray[Index], Value);
		++Index;
	}
}

UE_NET_TEST_FIXTURE(FTestObjectDeltaSerialization, SubObjectAutonomousIsProperlyReplicatedAfterBeingSimulated)
{
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with subobject on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, {.ConnectionFilteredComponentCount = 1});
	Server->ReplicationSystem->SetDeltaCompressionStatus(ServerSubObject->NetRefHandle, ENetObjectDeltaCompressionStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Add some values to the autonomous only array
	for (int32 Value : {4711, 1337})
	{
		ServerSubObject->ConnectionFilteredComponents[0]->AutonomousOnlyIntArray.Add(Value);
	}

	// In order to actually send the object we need to modify a non-AutonomousOnly property too
	ServerSubObject->ConnectionFilteredComponents[0]->NoneInt += 1;

	// We want a new baseline to be created
	AllowNewBaselineCreation();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));

	// Verify the autonomous only array is unaffected by the server changes
	UE_NET_ASSERT_TRUE(ClientSubObject->ConnectionFilteredComponents[0]->AutonomousOnlyIntArray.IsEmpty());

	// Set the root object to be autonomous.
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, true);

	// Add some values to the autonomous only array
	for (int32 Value : {1, -1})
	{
		ServerSubObject->ConnectionFilteredComponents[0]->AutonomousOnlyIntArray.Add(Value);
	}

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Check that the array is properly replicated in its entirety
	UE_NET_ASSERT_EQ(ClientSubObject->ConnectionFilteredComponents[0]->AutonomousOnlyIntArray.Num(), ServerSubObject->ConnectionFilteredComponents[0]->AutonomousOnlyIntArray.Num());
	for (int Index = 0; int32 Value : ServerSubObject->ConnectionFilteredComponents[0]->AutonomousOnlyIntArray)
	{
		UE_NET_ASSERT_EQ(ClientSubObject->ConnectionFilteredComponents[0]->AutonomousOnlyIntArray[Index], Value);
		++Index;
	}
}

UE_NET_TEST_FIXTURE(FTestObjectDeltaSerialization, SubObjectPhysicsIsProperlyReplicatedWhenEnabled)
{
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with subobject on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, {.ConnectionFilteredComponentCount = 1});
	Server->ReplicationSystem->SetDeltaCompressionStatus(ServerSubObject->NetRefHandle, ENetObjectDeltaCompressionStatus::Allow);

	// Set the root object to be autonomous. This should prevent SimulatedOrPhysics properties to be replicated.
	Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, true);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Add some values to the physics array
	for (int32 Value : {4711, 1337})
	{
		ServerSubObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsIntArray.Add(Value);
	}

	// In order to actually send the object we need to modify a non-physics property too
	ServerSubObject->ConnectionFilteredComponents[0]->NoneInt += 1;

	// We want a new baseline to be created
	AllowNewBaselineCreation();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));

	// Verify the physics array is unaffected by the server changes
	UE_NET_ASSERT_TRUE(ClientSubObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsIntArray.IsEmpty());

	// Enable physics replication
	Server->ReplicationSystem->SetReplicationCondition(ServerObject->NetRefHandle, EReplicationCondition::ReplicatePhysics, true);

	// Add some values to the physics array
	for (int32 Value : {1, -1})
	{
		ServerSubObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsIntArray.Add(Value);
	}

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Check that the array is properly replicated in its entirety
	UE_NET_ASSERT_EQ(ClientSubObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsIntArray.Num(), ServerSubObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsIntArray.Num());
	for (int Index = 0; int32 Value : ServerSubObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsIntArray)
	{
		UE_NET_ASSERT_EQ(ClientSubObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsIntArray[Index], Value);
		++Index;
	}
}

UE_NET_TEST_FIXTURE(FTestObjectDeltaSerialization, ToOwnerStateIsReplicatedToOwnerAfterBeingNonOwner)
{
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);
	Server->ReplicationSystem->SetDeltaCompressionStatus(ServerObject->NetRefHandle, ENetObjectDeltaCompressionStatus::Allow);

	// Set some values in ToOwner only state
	ServerObject->ConnectionFilteredComponents[0]->ToOwnerA = 13;
	ServerObject->ConnectionFilteredComponents[0]->ToOwnerB = 37;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the server modified ToOwner members are still in the default state
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->ToOwnerA, 0);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->ToOwnerB, 0);

	// Set owner
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Change arbitrary property to trigger replication
	ServerObject->IntA ^= 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	// Check that the ToOwner members have the same values as on the sending side.
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->ToOwnerA, ServerObject->ConnectionFilteredComponents[0]->ToOwnerA);
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->ToOwnerB, ServerObject->ConnectionFilteredComponents[0]->ToOwnerB);
}

UE_NET_TEST_FIXTURE(FTestObjectDeltaSerialization, SubObjectToOwnerStateIsReplicatedToOwnerAfterBeingNonOwner)
{
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with subobject on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, {.ConnectionFilteredComponentCount = 1});
	Server->ReplicationSystem->SetDeltaCompressionStatus(ServerSubObject->NetRefHandle, ENetObjectDeltaCompressionStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Add some values to the owner only array
	for (int32 Value : {4711, 1337})
	{
		ServerSubObject->ConnectionFilteredComponents[0]->OwnerOnlyIntArray.Add(Value);
	}

	// In order to actually send the object we need to modify a non-owner property too
	ServerSubObject->ConnectionFilteredComponents[0]->NoneInt += 1;

	// We want a new baseline to be created
	AllowNewBaselineCreation();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));

	// Verify the owner only array is unaffected by the server changes
	UE_NET_ASSERT_TRUE(ClientSubObject->ConnectionFilteredComponents[0]->OwnerOnlyIntArray.IsEmpty());

	// Set owner
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, Client->ConnectionIdOnServer);

	// Add some values to the owner only array
	for (int32 Value : {1, -1})
	{
		ServerSubObject->ConnectionFilteredComponents[0]->OwnerOnlyIntArray.Add(Value);
	}

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Check that the array is properly replicated in its entirety
	UE_NET_ASSERT_EQ(ClientSubObject->ConnectionFilteredComponents[0]->OwnerOnlyIntArray.Num(), ServerSubObject->ConnectionFilteredComponents[0]->OwnerOnlyIntArray.Num());
	for (int Index = 0; int32 Value : ServerSubObject->ConnectionFilteredComponents[0]->OwnerOnlyIntArray)
	{
		UE_NET_ASSERT_EQ(ClientSubObject->ConnectionFilteredComponents[0]->OwnerOnlyIntArray[Index], Value);
		++Index;
	}
}
UE_NET_TEST_FIXTURE(FTestObjectDeltaSerialization, InFlightChangesForDisabledConditionAreNotResent)
{
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	constexpr uint32 ConnectionFilteredComponentCount = 1;
	UTestReplicatedIrisObject::FComponents ObjectComponents;
	ObjectComponents.ConnectionFilteredComponentCount = ConnectionFilteredComponentCount;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(ObjectComponents);
	Server->ReplicationSystem->SetDeltaCompressionStatus(ServerObject->NetRefHandle, ENetObjectDeltaCompressionStatus::Allow);

	// Set autonomous role for our client. This is due to there not being a COND_Physics condition, but there is a COND_SimulatedOrPhysics.
	{
		Server->ReplicationSystem->SetReplicationConditionConnectionFilter(ServerObject->NetRefHandle, EReplicationCondition::RoleAutonomous, Client->ConnectionIdOnServer, true);
	}

	// Enable replicating physics
	{
		constexpr bool bIsPhysicsReplicationAllowed = true;
		Server->ReplicationSystem->SetReplicationCondition(ServerObject->NetRefHandle, EReplicationCondition::ReplicatePhysics, bIsPhysicsReplicationAllowed);
	}

	// Set value for SimulatedOrPhysics condition.
	ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt += 1;
	const int LastReplicatedSimulatedOrPhysicsInt = ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Check that the server modified SimulatedOrPhysics has been replicated properly.
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, LastReplicatedSimulatedOrPhysicsInt);

	// Do not allow new baselines to be created
	DisallowNewBaselineCreation();

	// Send a few packets with modified SimulatedOrPhysicsInt, but do no ack right away.
	for (SIZE_T PacketIt = 0, PacketEndIt = 4; PacketIt != PacketEndIt; ++PacketIt)
	{
		ServerObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt += 1;

		Server->NetUpdate();
		Server->SendTo(Client);
		Server->PostSendUpdate();
	}

	// Disable replicating physics
	{
		constexpr bool bIsPhysicsReplicationAllowed = false;
		Server->ReplicationSystem->SetReplicationCondition(ServerObject->NetRefHandle, EReplicationCondition::ReplicatePhysics, bIsPhysicsReplicationAllowed);
	}

	// Allow a new baseline to be created
	AllowNewBaselineCreation();

	// Change arbitrary property to trigger replication
	ServerObject->IntA ^= 1;

	// Send packet
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();
	
	// Now do the receive logic on the client. For this test assume all but the last packet, with the new baseline, were lost.
	{
		SIZE_T PacketCount = 0;
		const auto& ConnectionInfo = Server->GetConnectionInfo(Client->ConnectionIdOnServer);
		PacketCount = ConnectionInfo.WrittenPackets.Count();
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, (PacketIt + 1 < PacketCount ?  DoNotDeliverPacket : DeliverPacket));
		}
	}

	// Check that the SimulatedOrPhysicsInt has not been updated to the latest value, i.e. it has remained the same.
	UE_NET_ASSERT_EQ(ClientObject->ConnectionFilteredComponents[0]->SimulatedOrPhysicsInt, LastReplicatedSimulatedOrPhysicsInt);

	// Of course the arbitrary property that was changed should have received its final value.
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
}

UE_NET_TEST_FIXTURE(FTestObjectDeltaSerialization, LostChangesDuringPendingDestroyArePartOfBaselineAfterCancellingPendingDestroy)
{
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject({ .DynamicStateComponentCount = 1 });
	Server->ReplicationSystem->SetDeltaCompressionStatus(ServerObject->NetRefHandle, ENetObjectDeltaCompressionStatus::Allow);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	AllowNewBaselineCreation();

	// Modify array
	ServerObject->DynamicStateComponents[0]->IntArray.Add(23456789);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Clear array
	DisallowNewBaselineCreation();
	ServerObject->DynamicStateComponents[0]->IntArray.Empty();

	// Fake latency by acking/naking later
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Filter out object to cause a PendingDestroy
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Drop all packets
	{
		const auto& ConnectionInfo = Server->GetConnectionInfo(Client->ConnectionIdOnServer);
		SIZE_T PacketCount = ConnectionInfo.WrittenPackets.Count();
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, DoNotDeliverPacket);
		}
	}

	// Remove object from filter to cause object to end up in CancelPendingDestroy
	DisallowNewBaselineCreation();
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->UpdateAndSend({ Client });

	// Change other properties to force replication
	ServerObject->IntA += 1;
	ServerObject->DynamicStateComponents[0]->IntStaticArray[0] += 1;

	AllowNewBaselineCreation();
	Server->UpdateAndSend({ Client });

	// Finally modify the IntArray.
	ServerObject->IntA += 1;
	ServerObject->DynamicStateComponents[0]->IntArray.Add(-12345);

	DisallowNewBaselineCreation();
	Server->UpdateAndSend({ Client });

	// Verify IntArray contents
	UE_NET_ASSERT_EQ(ClientObject->DynamicStateComponents[0]->IntArray.Num(), ServerObject->DynamicStateComponents[0]->IntArray.Num());
	UE_NET_ASSERT_EQ(ClientObject->DynamicStateComponents[0]->IntArray[0], ServerObject->DynamicStateComponents[0]->IntArray[0]);
}


// FTestObjectDeltaSerialization implementation
void FTestObjectDeltaSerialization::SetUp()
{
	CvarIrisDeltaCompression = IConsoleManager::Get().FindConsoleVariable(TEXT("net.Iris.EnableDeltaCompression"));
	check(CvarIrisDeltaCompression != nullptr && CvarIrisDeltaCompression->IsVariableBool());
	bDeltaCompressionEnable = CvarIrisDeltaCompression->GetBool();
	CvarIrisDeltaCompression->Set(true, ECVF_SetByCode);

	CvarIrisFramesBetweenBaselines = IConsoleManager::Get().FindConsoleVariable(TEXT("net.Iris.MinimumNumberOfFramesBetweenBaselines"));
	check(CvarIrisFramesBetweenBaselines != nullptr && CvarIrisFramesBetweenBaselines->IsVariableInt());
	MinimuNumberOfFramesBetweenBaselines = CvarIrisFramesBetweenBaselines->GetInt();

	FReplicationSystemServerClientTestFixture::SetUp();
}

void FTestObjectDeltaSerialization::TearDown() 
{
	FReplicationSystemServerClientTestFixture::TearDown();

	CvarIrisDeltaCompression->Set(bDeltaCompressionEnable, ECVF_SetByCode);
	CvarIrisFramesBetweenBaselines->Set(MinimuNumberOfFramesBetweenBaselines, ECVF_SetByCode);
}

void FTestObjectDeltaSerialization::AllowNewBaselineCreation()
{
	CvarIrisFramesBetweenBaselines->Set(0, ECVF_SetByCode);
}

void FTestObjectDeltaSerialization::DisallowNewBaselineCreation()
{
	constexpr int32 ArbitraryLargeFrameCount = 1000000;
	CvarIrisFramesBetweenBaselines->Set(ArbitraryLargeFrameCount, ECVF_SetByCode);
}

}
