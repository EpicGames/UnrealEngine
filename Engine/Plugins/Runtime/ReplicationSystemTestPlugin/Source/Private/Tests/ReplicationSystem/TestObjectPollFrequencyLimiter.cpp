// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSystemServerClientTestFixture.h"

#include "Net/Core/NetBitArray.h"

#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/ObjectPollFrequencyLimiter.h"

#include "Math/UnrealMathUtility.h"

namespace UE::Net::Private
{

UE_NET_TEST(ObjectPollFrequencyLimiter, ObjectsToPollAreSpreadOutWhenCreatingObjectAndUpdatingOncePerFrame)
{
	constexpr float UpdateFrequency = 1.0f;
	constexpr uint32 PollFramesBetweenUpdates = 17U;
	constexpr FInternalNetRefIndex MaxObjectCount = PollFramesBetweenUpdates + 1;
	constexpr FInternalNetRefIndex MaxInternalIndex = MaxObjectCount + 1;

	FObjectPollFrequencyLimiter FrequencyLimiter;
	FrequencyLimiter.Init(MaxInternalIndex);
	FrequencyLimiter.SetUpdateFrequency(UpdateFrequency);

	FNetBitArray EmptyBitArray(MaxInternalIndex);
	FNetBitArray ScopeBitArray(MaxInternalIndex);

	// Create one object at a time and update.
	FNetBitArray OutputArray(MaxInternalIndex);
	FNetBitArrayView OutputBitArrayView = MakeNetBitArrayView(OutputArray);

	for (uint32 It = 0, EndIt = MaxObjectCount; It < EndIt; ++It)
	{
		FInternalNetRefIndex ObjectIndex = It + 1U;
		FrequencyLimiter.SetPollFrequency(ObjectIndex, UpdateFrequency/(PollFramesBetweenUpdates + 1));

		// Add object to the scope
		ScopeBitArray.SetBit(ObjectIndex);

		FrequencyLimiter.Update(MakeNetBitArrayView(ScopeBitArray), MakeNetBitArrayView(EmptyBitArray), OutputBitArrayView);
	}

	// All objects are created. Verify we get one object polled per frame
	for (uint32 It = 0, EndIt = MaxObjectCount; It < EndIt; ++It)
	{
		OutputBitArrayView.ClearAllBits();
		FrequencyLimiter.Update(MakeNetBitArrayView(ScopeBitArray), MakeNetBitArrayView(EmptyBitArray), OutputBitArrayView);

		const uint32 PollCount = OutputArray.CountSetBits();
		const uint32 ExpectedPollCount = 1U;
		UE_NET_ASSERT_EQ(PollCount, ExpectedPollCount);
	}
}

UE_NET_TEST(ObjectPollFrequencyLimiter, ObjectsToPollAreSpreadOutRegardlessOfUpdateCountAndCreatedObjectCount)
{
	constexpr int32 RandSeed = 1192314641;
	FMath::RandInit(RandSeed);

	constexpr float UpdateFrequency = 1.0f;
	constexpr uint32 PollFramesBetweenUpdates = 8U;
	constexpr FInternalNetRefIndex MaxObjectCount = 128;
	constexpr FInternalNetRefIndex MaxInternalIndex = MaxObjectCount + 1;

	FObjectPollFrequencyLimiter FrequencyLimiter;
	FrequencyLimiter.Init(MaxInternalIndex);
	FrequencyLimiter.SetUpdateFrequency(UpdateFrequency);

	// With four times as many objects as the poll frequency we expect four objects to be polled per frame.
	constexpr uint32 PollFrequencyObjectCount = 4U*(PollFramesBetweenUpdates + 1U);
	constexpr uint32 MaxUpdateCountPerIteration = 5U;

	FNetBitArray EmptyBitArray(MaxInternalIndex);
	FNetBitArray ScopeBitArray(MaxInternalIndex);

	// Create an arbitrary number of objects with arbitrary number of frames in between.
	FNetBitArray OutputArray(MaxInternalIndex);
	{
		FNetBitArrayView OutputBitArrayView = MakeNetBitArrayView(OutputArray);

		uint32 CreatedObjectCount = 0;
		do
		{
			int32 UpdateCount = FMath::RandHelper(MaxUpdateCountPerIteration + 1);
			while (UpdateCount-- > 0)
			{
				FrequencyLimiter.Update(MakeNetBitArrayView(ScopeBitArray), MakeNetBitArrayView(EmptyBitArray), OutputBitArrayView);
			}

			int32 CreateCount = FMath::RandHelper(FMath::Min(3U, PollFrequencyObjectCount - CreatedObjectCount) + 1);
			while (CreateCount-- > 0)
			{
				++CreatedObjectCount;
				FInternalNetRefIndex ObjectIndex = CreatedObjectCount;
				// + 1 is to get the exact PollFramesBetweenUpdates when calling SetPollFrequency.
				FrequencyLimiter.SetPollFrequency(ObjectIndex, UpdateFrequency/(PollFramesBetweenUpdates + 1));
			}

			if (CreatedObjectCount > 0)
			{
				ScopeBitArray.SetBits(1, CreatedObjectCount);
			}

		} while (CreatedObjectCount < PollFrequencyObjectCount);
	}

	// Verify the objects get polled evenly.
	for (uint32 PollIt = 0; PollIt <= PollFramesBetweenUpdates; ++PollIt)
	{
		OutputArray.ClearAllBits();
		FNetBitArrayView OutputBitArrayView = MakeNetBitArrayView(OutputArray);
		FrequencyLimiter.Update(MakeNetBitArrayView(ScopeBitArray), MakeNetBitArrayView(EmptyBitArray), OutputBitArrayView);

		uint32 PollCount = OutputArray.CountSetBits();
		const uint32 ExpectedPollCount = PollFrequencyObjectCount/(PollFramesBetweenUpdates + 1U);
		UE_NET_ASSERT_EQ(PollCount, ExpectedPollCount);
	}
}


UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestSubObjectsPollingAndForceNetUpdate)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UReplicatedTestObjectBridge* Bridge = Server->GetReplicationBridge();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server polled every 3 frames
	const uint32 PollPeriod = 3;
	const float PollFrequency = Server->ConvertPollPeriodIntoFrequency(PollPeriod);
	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	Params.PollFrequency = PollFrequency;
	Params.bUseClassConfigDynamicFilter = true;
	Params.bNeedsPreUpdate = true;
	UTestReplicatedIrisObject* ServerRootObject = Server->CreateObject(Params);

	// Spawn subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerRootObject->NetRefHandle, 0, 0);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Root Object should be created on the client
	UTestReplicatedIrisObject* ClientRootObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerRootObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientRootObject, nullptr);

	// Subobject should be created on the client
	UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientSubObject, nullptr);

	// Dirty a root object property
	ServerRootObject->IntA = 0x0A;

	// Update for maximum poll period or when property is received
	auto UpdateUntilPolled = [&](UTestReplicatedIrisObject* InServerObject, UTestReplicatedIrisObject* InClientObject) -> uint32
	{
		uint32 FramesPolled = 0;
		for (; FramesPolled <= PollPeriod; ++FramesPolled)
		{
			// Send and deliver packet
			Server->UpdateAndSend({ Client });

			if (InServerObject->IntA == InClientObject->IntA)
			{
				return FramesPolled+1;
			}
		}
		return FramesPolled;
	};

	// Test polling on root object
	{
		// Update until property is received
		uint32 FramesPolled = UpdateUntilPolled(ServerRootObject, ClientRootObject);

		// Root object should have received property
		UE_NET_ASSERT_EQ(ServerRootObject->IntA, ClientRootObject->IntA);

		// We should have needed the max poll period
		UE_NET_ASSERT_EQ(FramesPolled, PollPeriod + 1);
	}

	// Test polling on sub object
	{
		// Dirty a sub object property
		ServerSubObject->IntA = 0x0B;

		// Update until property is received
		uint32 FramesPolled = UpdateUntilPolled(ServerSubObject, ClientSubObject);

		// Subobject should have received property change
		UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObject->IntA);

		// We should have needed the max poll period
		UE_NET_ASSERT_EQ(FramesPolled, PollPeriod + 1);
	}	

	// Test force update on sub object
	{
		// Dirty a sub object property
		ServerSubObject->IntA = 0x0C;

		// Force update it
		ReplicationSystem->ForceNetUpdate(ServerSubObject->NetRefHandle);

		// Update until property is received
		uint32 FramesPolled = UpdateUntilPolled(ServerSubObject, ClientSubObject);

		// Subobject should have received property change
		UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObject->IntA);

		// We should have needed only 1 update 
		UE_NET_ASSERT_EQ(FramesPolled, 1U);
	}

	// Test force update on root object
	{
		// Dirty a sub object property
		ServerSubObject->IntA = 0x0D;

		// Force update it
		ReplicationSystem->ForceNetUpdate(ServerRootObject->NetRefHandle);

		// Update until property is received
		uint32 FramesPolled = UpdateUntilPolled(ServerSubObject, ClientSubObject);

		// Subobject should have received property change
		UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObject->IntA);

		// We should have needed only 1 update 
		UE_NET_ASSERT_EQ(FramesPolled, 1U);
	}
}


}
