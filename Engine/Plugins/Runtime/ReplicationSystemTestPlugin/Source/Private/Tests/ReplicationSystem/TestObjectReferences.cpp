// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestObjectReferences.h"
#include "Iris/Core/NetObjectReference.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Tests/ReplicationSystem/TestObjectReferences.h"
#include "Iris/Serialization/NetReferenceCollector.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Net/UnrealNetwork.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Iris/Core/IrisDebugging.h"

class UObject;

void UTestObjectReferences_TestClassWithReferences::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

void UTestObjectReferences_TestClassWithReferences::GetLifetimeReplicatedProps(TArray< class FLifetimeProperty > & OutLifetimeProps) const
{
	DOREPLIFETIME_CONDITION(UTestObjectReferences_TestClassWithReferences, ObjectRef, COND_InitialOnly);
	DOREPLIFETIME(UTestObjectReferences_TestClassWithReferences, StructWithRef);
	DOREPLIFETIME(UTestObjectReferences_TestClassWithReferences, StructWithRefCArray);
	DOREPLIFETIME(UTestObjectReferences_TestClassWithReferences, StructWithRefTArray);
	DOREPLIFETIME(UTestObjectReferences_TestClassWithReferences, Ref_CArray);
	DOREPLIFETIME(UTestObjectReferences_TestClassWithReferences, Ref_TArray);
	DOREPLIFETIME(UTestObjectReferences_TestClassWithReferences, StructWithRef_CArray);
	DOREPLIFETIME(UTestObjectReferences_TestClassWithReferences, StructWithRef_TArray);
	DOREPLIFETIME(UTestObjectReferences_TestClassWithReferences, TestStructWithNestedRefTArray_TArray);
	DOREPLIFETIME(UTestObjectReferences_TestClassWithReferences, TestStructWithNestedRefCArray_CArray);
	DOREPLIFETIME(UTestObjectReferences_TestClassWithReferences, Ref_FastArray);
	DOREPLIFETIME(UTestObjectReferences_TestClassWithReferences, Ref_NativeFastArray);
}

UTestObjectReferences_TestClassWithDefaultSubObject::UTestObjectReferences_TestClassWithDefaultSubObject()
: UReplicatedTestObject()
{
	CreatedSubObjectRef = CreateDefaultSubobject<UTestReplicatedIrisObject>("TestDefaultSubObjectRef");
}

void UTestObjectReferences_TestClassWithDefaultSubObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);

	// Include subobject as well in the protocol
	if (CreatedSubObjectRef.Get())
	{
		static_cast<UTestReplicatedIrisObject*>(CreatedSubObjectRef.Get())->RegisterReplicationFragments(Context, RegistrationFlags);
	}
}

void UTestObjectReferences_TestClassWithDefaultSubObject::GetLifetimeReplicatedProps(TArray< class FLifetimeProperty > & OutLifetimeProps) const
{
	DOREPLIFETIME(UTestObjectReferences_TestClassWithDefaultSubObject, ObjectRef);
}

UTestObjectReferences_TestClassWithNestedDefaultSubObjects::UTestObjectReferences_TestClassWithNestedDefaultSubObjects()
: UReplicatedTestObject()
{
	CreatedSubObjectRef = CreateDefaultSubobject<UTestObjectReferences_TestClassWithDefaultSubObject>("TestDefaultSubObjectRef");
}

void UTestObjectReferences_TestClassWithNestedDefaultSubObjects::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);

	// Include subobject as well in the protocol
	if (CreatedSubObjectRef.Get())
	{
		CreatedSubObjectRef.Get()->RegisterReplicationFragments(Context, RegistrationFlags);
	}
}

void UTestObjectReferences_TestClassWithNestedDefaultSubObjects::GetLifetimeReplicatedProps(TArray< class FLifetimeProperty > & OutLifetimeProps) const
{
	DOREPLIFETIME(UTestObjectReferences_TestClassWithNestedDefaultSubObjects, ObjectRef);
}


namespace UE::Net::Private
{

class FTestObjectReferencesFixture : public FReplicationSystemServerClientTestFixture
{
public:
	struct FReferenceCollector
	{
		void Reset() { References.Reset(); }

		bool bCollectAll = true;
		TArray<FNetObjectReference> References;
	};

	struct FCollectValidReferenceCollector
	{
		FCollectValidReferenceCollector(bool bInIncludeInitOnly, const FNetBitArrayView& InChangeMask)
		 : ChangeMask(InChangeMask)
		, bIncludeInitOnly(bInIncludeInitOnly)
		{
			BitArray.Init(InChangeMask.GetNumBits());
		}

		void Reset()
		{
			BitArray.ClearAllBits();
			References.Reset();
		}
		const FNetBitArrayView* GetChangeMask() const { return &ChangeMask; }
		const bool IncludeInitOnly() const { return bIncludeInitOnly; }

		FNetBitArray BitArray;
		TArray<FNetObjectReference> References;
		const FNetBitArrayView& ChangeMask;
		const bool bIncludeInitOnly;
	};

	void VisitReferences(const uint8* RESTRICT SrcInternalBuffer, const FReplicationProtocol* Protocol, FReferenceCollector& ReferenceCollector)
	{
		FNetSerializationContext Context;
		Context.SetIsInitState(true);

		FNetReferenceCollector Collector(ReferenceCollector.bCollectAll ? ENetReferenceCollectorTraits::IncludeInvalidReferences : ENetReferenceCollectorTraits::None);
		FReplicationProtocolOperationsInternal::CollectReferences(Context, Collector, SrcInternalBuffer, Protocol);
	
		for (const FNetReferenceCollector::FReferenceInfo& Info : MakeArrayView(Collector.GetCollectedReferences()))
		{
			ReferenceCollector.References.Add(Info.Reference);
		}
	}

	void VisitReferences(const uint8* RESTRICT SrcInternalBuffer, const FReplicationProtocol* Protocol, FCollectValidReferenceCollector& ReferenceCollector)
	{
		FNetSerializationContext Context;
		Context.SetChangeMask(&ReferenceCollector.ChangeMask);
		Context.SetIsInitState(ReferenceCollector.bIncludeInitOnly);

		FNetReferenceCollector Collector;
		FReplicationProtocolOperationsInternal::CollectReferences(Context, Collector, SrcInternalBuffer, Protocol);
	
		for (const FNetReferenceCollector::FReferenceInfo& Info : MakeArrayView(Collector.GetCollectedReferences()))
		{
			ReferenceCollector.References.Add(Info.Reference);
			if (Info.ChangeMaskInfo.BitCount)
			{
				ReferenceCollector.BitArray.SetBit(Info.ChangeMaskInfo.BitOffset);
			}				
		}
	}

	void SetMemberDirty(FNetBitArrayView ChangeMask, uint32 MemberIndex)
	{
		const FReplicationStateDescriptor* Descriptor = TestObjectProtocol->ReplicationStateDescriptors[1];
		const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskDesciptor = Descriptor->MemberChangeMaskDescriptors[MemberIndex];
		ChangeMask.SetBits(ChangeMaskDesciptor.BitOffset, ChangeMaskDesciptor.BitCount);
	}

	FReferenceCollector CollectAllCollector;
	FReferenceCollector CollectOnlyValidCollector;
	
	TArray<UTestObjectReferences_TestClassWithReferences*> TestReferences;
	UTestObjectReferences_TestClassWithReferences* TestObject;
	uint8* TestObjectStateBuffer;
	const FReplicationProtocol* TestObjectProtocol;

	virtual void SetUp() override
	{
		// These tests rely on full update being executed.
		ServerParamsOverride.bAllowMinimalUpdateIfNoConnections = false;

		FReplicationSystemServerClientTestFixture::SetUp();

		// Create objects
		const uint32 ObjectCount = 30;
		for (uint32 It = 0; It < ObjectCount; ++It)
		{
			TestReferences.Push(Server->CreateObject<UTestObjectReferences_TestClassWithReferences>());
		}

		// Create test object
		TestObject = Server->CreateObject<UTestObjectReferences_TestClassWithReferences>();
		
		uint32 ObjectIndex = Server->GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager().GetInternalIndex(TestObject->NetRefHandle);
		TestObjectStateBuffer = Server->GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager().GetReplicatedObjectStateBufferNoCheck(ObjectIndex);
		TestObjectProtocol = Server->GetReplicationSystem()->GetReplicationProtocol(TestObject->NetRefHandle);

		// InitCollectors
		CollectOnlyValidCollector.bCollectAll = false;
		CollectAllCollector.bCollectAll = true;

		// To trigger copy of data so we have valid data in buffers
		Server->NetUpdate();
		Server->PostSendUpdate();
	}
};

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_NoRef)
{
	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();

	FNetSerializationContext Context;

	// Collect 
	FReferenceCollector& Collector = CollectAllCollector;
	FTestObjectReferencesFixture::VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 38);

	for (const FNetObjectReference& Ref : Collector.References)
	{
		UE_NET_ASSERT_FALSE(Ref.GetRefHandle().IsValid());
	}
}


UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_SingleRef)
{
	// Setup references
	TestObject->ObjectRef = TestReferences[0];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 1);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_ObjectRef)
{
	// Setup references
	TestObject->ObjectRef = TestReferences[0];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();

	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 1);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_StructWithRef)
{
	// Setup references
	TestObject->StructWithRef.ObjectRef = TestReferences[0];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 1);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_StructWithRefCArray)
{
	// Setup references
	TestObject->StructWithRefCArray.Ref_CArray[0] = TestReferences[0];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 1);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_StructWithRefCArrayMulti)
{
	// Setup references
	TestObject->StructWithRefCArray.Ref_CArray[0] = TestReferences[0];
	TestObject->StructWithRefCArray.Ref_CArray[1] = TestReferences[1];
	TestObject->StructWithRefCArray.Ref_CArray[2] = TestReferences[2];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 3);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[1].GetRefHandle() == TestReferences[1]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[2].GetRefHandle() == TestReferences[2]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_StructWithRefTArray)
{
	// Setup references
	TestObject->StructWithRefTArray.Ref_TArray.Push(TestReferences[0]);

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 1);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_StructWithRefTArrayMulti)
{
	// Setup references
	TestObject->StructWithRefTArray.Ref_TArray.Push(TestReferences[0]);
	TestObject->StructWithRefTArray.Ref_TArray.Push(TestReferences[1]);
	TestObject->StructWithRefTArray.Ref_TArray.Push(TestReferences[2]);

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 3);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[1].GetRefHandle() == TestReferences[1]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[2].GetRefHandle() == TestReferences[2]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_RefCArray)
{
	// Setup references
	TestObject->StructWithRefCArray.Ref_CArray[0] = TestReferences[0];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 1);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_RefCArrayMulti)
{
	// Setup references
	TestObject->Ref_CArray[0] = TestReferences[0];
	TestObject->Ref_CArray[1] = TestReferences[1];
	TestObject->Ref_CArray[2] = TestReferences[2];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 3);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[1].GetRefHandle() == TestReferences[1]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[2].GetRefHandle() == TestReferences[2]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_RefTArray)
{
	// Setup references
	TestObject->Ref_TArray.Push(TestReferences[0]);

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 1);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_RefTArrayMulti)
{
	// Setup references
	TestObject->Ref_TArray.Push(TestReferences[0]);
	TestObject->Ref_TArray.Push(TestReferences[1]);
	TestObject->Ref_TArray.Push(TestReferences[2]);

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 3);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[1].GetRefHandle() == TestReferences[1]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[2].GetRefHandle() == TestReferences[2]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_StructWithRef_CArray)
{
	// Setup references
	TestObject->StructWithRef_CArray[0].ObjectRef = TestReferences[0];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 1);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_StructWithRef_CArrayMulti)
{
	// Setup references
	TestObject->StructWithRef_CArray[0].ObjectRef = TestReferences[0];
	TestObject->StructWithRef_CArray[1].ObjectRef = TestReferences[1];
	TestObject->StructWithRef_CArray[2].ObjectRef = TestReferences[2];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();

	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 3);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[1].GetRefHandle() == TestReferences[1]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[2].GetRefHandle() == TestReferences[2]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_StructWithRef_TArray)
{
	// Setup references
	TestObject->StructWithRef_TArray.SetNum(1);
	TestObject->StructWithRef_TArray[0].ObjectRef = TestReferences[0];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 1);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestReferenceVisitor_StructWithRef_TArrayMulti)
{
	// Setup references
	TestObject->StructWithRef_TArray.SetNum(3);
	TestObject->StructWithRef_TArray[0].ObjectRef = TestReferences[0];
	TestObject->StructWithRef_TArray[1].ObjectRef = TestReferences[1];
	TestObject->StructWithRef_TArray[2].ObjectRef = TestReferences[2];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();

	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 3);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[1].GetRefHandle() == TestReferences[1]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[2].GetRefHandle() == TestReferences[2]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestStructWithNestedRefCArray_CArray)
{
	// Setup references
	TestObject->TestStructWithNestedRefCArray_CArray[0].StructWithRef_CArray[0].Ref_CArray[0] = TestReferences[0];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 1);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestStructWithNestedRefCArray_CArrayMulti)
{
	// Setup references
	TestObject->TestStructWithNestedRefCArray_CArray[0].StructWithRef_CArray[0].Ref_CArray[0] = TestReferences[0];
	TestObject->TestStructWithNestedRefCArray_CArray[1].StructWithRef_CArray[0].Ref_CArray[0] = TestReferences[1];
	TestObject->TestStructWithNestedRefCArray_CArray[1].StructWithRef_CArray[1].Ref_CArray[0] = TestReferences[2];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 3);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[1].GetRefHandle() == TestReferences[1]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[2].GetRefHandle() == TestReferences[2]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestStructWithNestedRefTArray_TArray)
{
	// Setup references
	TestObject->TestStructWithNestedRefTArray_TArray.SetNum(1);
	TestObject->TestStructWithNestedRefTArray_TArray[0].StructWithRef_TArray.SetNum(1);
	TestObject->TestStructWithNestedRefTArray_TArray[0].StructWithRef_TArray[0].Ref_TArray.SetNum(1);
	TestObject->TestStructWithNestedRefTArray_TArray[0].StructWithRef_TArray[0].Ref_TArray[0] = TestReferences[0];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 1);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestStructWithNestedRefTArray_TArrayMulti)
{
	// Setup references
	TestObject->TestStructWithNestedRefTArray_TArray.SetNum(2);
	TestObject->TestStructWithNestedRefTArray_TArray[0].StructWithRef_TArray.SetNum(1);
	TestObject->TestStructWithNestedRefTArray_TArray[0].StructWithRef_TArray[0].Ref_TArray.SetNum(1);
	TestObject->TestStructWithNestedRefTArray_TArray[0].StructWithRef_TArray[0].Ref_TArray[0] = TestReferences[0];

	TestObject->TestStructWithNestedRefTArray_TArray[1].StructWithRef_TArray.SetNum(2);
	TestObject->TestStructWithNestedRefTArray_TArray[1].StructWithRef_TArray[0].Ref_TArray.SetNum(1);
	TestObject->TestStructWithNestedRefTArray_TArray[1].StructWithRef_TArray[0].Ref_TArray[0] = TestReferences[1];
	TestObject->TestStructWithNestedRefTArray_TArray[1].StructWithRef_TArray[1].Ref_TArray.SetNum(1);
	TestObject->TestStructWithNestedRefTArray_TArray[1].StructWithRef_TArray[1].Ref_TArray[0] = TestReferences[2];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ(Collector.References.Num(), 3);
	UE_NET_ASSERT_TRUE(Collector.References[0].GetRefHandle() == TestReferences[0]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[1].GetRefHandle() == TestReferences[1]->NetRefHandle);
	UE_NET_ASSERT_TRUE(Collector.References[2].GetRefHandle() == TestReferences[2]->NetRefHandle);
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestStructWithNestedRefTArray_Combo)
{
	// Setup references
	uint32 CurrentReferenceIndex = 0;

	TestObject->ObjectRef = TestReferences[CurrentReferenceIndex++];

	TestObject->StructWithRef.ObjectRef = TestReferences[CurrentReferenceIndex++];

	TestObject->StructWithRefCArray.Ref_CArray[0] = TestReferences[CurrentReferenceIndex++];
	TestObject->StructWithRefCArray.Ref_CArray[1] = TestReferences[CurrentReferenceIndex++];
	TestObject->StructWithRefCArray.Ref_CArray[2] = TestReferences[CurrentReferenceIndex++];

	TestObject->StructWithRefTArray.Ref_TArray.Push(TestReferences[CurrentReferenceIndex++]);
	TestObject->StructWithRefTArray.Ref_TArray.Push(TestReferences[CurrentReferenceIndex++]);
	TestObject->StructWithRefTArray.Ref_TArray.Push(TestReferences[CurrentReferenceIndex++]);

	TestObject->Ref_CArray[0] = TestReferences[CurrentReferenceIndex++];
	TestObject->Ref_CArray[1] = TestReferences[CurrentReferenceIndex++];
	TestObject->Ref_CArray[2] = TestReferences[CurrentReferenceIndex++];

	TestObject->Ref_TArray.Push(TestReferences[CurrentReferenceIndex++]);
	TestObject->Ref_TArray.Push(TestReferences[CurrentReferenceIndex++]);
	TestObject->Ref_TArray.Push(TestReferences[CurrentReferenceIndex++]);

	TestObject->StructWithRef_CArray[0].ObjectRef = TestReferences[CurrentReferenceIndex++];
	TestObject->StructWithRef_CArray[1].ObjectRef = TestReferences[CurrentReferenceIndex++];
	TestObject->StructWithRef_CArray[2].ObjectRef = TestReferences[CurrentReferenceIndex++];

	TestObject->StructWithRef_TArray.SetNum(3);
	TestObject->StructWithRef_TArray[0].ObjectRef = TestReferences[CurrentReferenceIndex++];
	TestObject->StructWithRef_TArray[1].ObjectRef = TestReferences[CurrentReferenceIndex++];
	TestObject->StructWithRef_TArray[2].ObjectRef = TestReferences[CurrentReferenceIndex++];

	TestObject->TestStructWithNestedRefTArray_TArray.SetNum(2);
	TestObject->TestStructWithNestedRefTArray_TArray[0].StructWithRef_TArray.SetNum(1);
	TestObject->TestStructWithNestedRefTArray_TArray[0].StructWithRef_TArray[0].Ref_TArray.SetNum(1);
	TestObject->TestStructWithNestedRefTArray_TArray[0].StructWithRef_TArray[0].Ref_TArray[0] = TestReferences[CurrentReferenceIndex++];

	TestObject->TestStructWithNestedRefTArray_TArray[1].StructWithRef_TArray.SetNum(2);
	TestObject->TestStructWithNestedRefTArray_TArray[1].StructWithRef_TArray[0].Ref_TArray.SetNum(1);
	TestObject->TestStructWithNestedRefTArray_TArray[1].StructWithRef_TArray[0].Ref_TArray[0] = TestReferences[CurrentReferenceIndex++];
	TestObject->TestStructWithNestedRefTArray_TArray[1].StructWithRef_TArray[1].Ref_TArray.SetNum(1);
	TestObject->TestStructWithNestedRefTArray_TArray[1].StructWithRef_TArray[1].Ref_TArray[0] = TestReferences[CurrentReferenceIndex++];

	TestObject->TestStructWithNestedRefCArray_CArray[0].StructWithRef_CArray[0].Ref_CArray[0] = TestReferences[CurrentReferenceIndex++];
	TestObject->TestStructWithNestedRefCArray_CArray[1].StructWithRef_CArray[0].Ref_CArray[0] = TestReferences[CurrentReferenceIndex++];
	TestObject->TestStructWithNestedRefCArray_CArray[1].StructWithRef_CArray[1].Ref_CArray[0] = TestReferences[CurrentReferenceIndex++];

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
	
	FReferenceCollector& Collector = CollectOnlyValidCollector;

	// Collect 
	VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

	UE_NET_ASSERT_EQ((uint32)Collector.References.Num(), CurrentReferenceIndex);
	for (int32 It = 0; It < Collector.References.Num(); ++It)
	{
		UE_NET_ASSERT_TRUE(Collector.References[It].GetRefHandle() == TestReferences[It]->NetRefHandle);
	}	
}

UE_NET_TEST_FIXTURE(FTestObjectReferencesFixture, TestVisitAllDirty)
{
	// Setup references
	uint32 CurrentReferenceIndex = 0;

	TestObject->ObjectRef = TestReferences[CurrentReferenceIndex++]; // Init

	TestObject->StructWithRef.ObjectRef = TestReferences[CurrentReferenceIndex++];  // 0

	TestObject->StructWithRefCArray.Ref_CArray[0] = TestReferences[CurrentReferenceIndex++]; // 1
	TestObject->StructWithRefCArray.Ref_CArray[1] = TestReferences[CurrentReferenceIndex++]; // 1
	TestObject->StructWithRefCArray.Ref_CArray[2] = TestReferences[CurrentReferenceIndex++]; // 1

	TestObject->StructWithRefTArray.Ref_TArray.Push(TestReferences[CurrentReferenceIndex++]); // 2
	TestObject->StructWithRefTArray.Ref_TArray.Push(TestReferences[CurrentReferenceIndex++]); // 2
	TestObject->StructWithRefTArray.Ref_TArray.Push(TestReferences[CurrentReferenceIndex++]); // 2

	TestObject->Ref_CArray[0] = TestReferences[CurrentReferenceIndex++]; // 3
	TestObject->Ref_CArray[1] = TestReferences[CurrentReferenceIndex++]; // 4
	TestObject->Ref_CArray[2] = TestReferences[CurrentReferenceIndex++]; // 5

	TestObject->Ref_TArray.Push(TestReferences[CurrentReferenceIndex++]); // 6
	TestObject->Ref_TArray.Push(TestReferences[CurrentReferenceIndex++]); // 6
	TestObject->Ref_TArray.Push(TestReferences[CurrentReferenceIndex++]); // 6

	TestObject->StructWithRef_CArray[0].ObjectRef = TestReferences[CurrentReferenceIndex++]; // 7
	TestObject->StructWithRef_CArray[1].ObjectRef = TestReferences[CurrentReferenceIndex++]; // 8
	TestObject->StructWithRef_CArray[2].ObjectRef = TestReferences[CurrentReferenceIndex++]; // 9

	TestObject->StructWithRef_TArray.SetNum(3);
	TestObject->StructWithRef_TArray[0].ObjectRef = TestReferences[CurrentReferenceIndex++]; // 10
	TestObject->StructWithRef_TArray[1].ObjectRef = TestReferences[CurrentReferenceIndex++]; // 10
	TestObject->StructWithRef_TArray[2].ObjectRef = TestReferences[CurrentReferenceIndex++]; // 10

	TestObject->TestStructWithNestedRefTArray_TArray.SetNum(2);
	TestObject->TestStructWithNestedRefTArray_TArray[0].StructWithRef_TArray.SetNum(1);
	TestObject->TestStructWithNestedRefTArray_TArray[0].StructWithRef_TArray[0].Ref_TArray.SetNum(1);
	TestObject->TestStructWithNestedRefTArray_TArray[0].StructWithRef_TArray[0].Ref_TArray[0] = TestReferences[CurrentReferenceIndex++]; // 11

	TestObject->TestStructWithNestedRefTArray_TArray[1].StructWithRef_TArray.SetNum(2);
	TestObject->TestStructWithNestedRefTArray_TArray[1].StructWithRef_TArray[0].Ref_TArray.SetNum(1);
	TestObject->TestStructWithNestedRefTArray_TArray[1].StructWithRef_TArray[0].Ref_TArray[0] = TestReferences[CurrentReferenceIndex++]; // 11
	TestObject->TestStructWithNestedRefTArray_TArray[1].StructWithRef_TArray[1].Ref_TArray.SetNum(1);
	TestObject->TestStructWithNestedRefTArray_TArray[1].StructWithRef_TArray[1].Ref_TArray[0] = TestReferences[CurrentReferenceIndex++]; // 11

	TestObject->TestStructWithNestedRefCArray_CArray[0].StructWithRef_CArray[0].Ref_CArray[0] = TestReferences[CurrentReferenceIndex++]; // 12
	TestObject->TestStructWithNestedRefCArray_CArray[1].StructWithRef_CArray[1].Ref_CArray[1] = TestReferences[CurrentReferenceIndex++]; // 13
	TestObject->TestStructWithNestedRefCArray_CArray[2].StructWithRef_CArray[2].Ref_CArray[2] = TestReferences[CurrentReferenceIndex++]; // 14

	FNetBitArray ChangeMaskStorage(TestObjectProtocol->ChangeMaskBitCount);
	FNetBitArrayView ChangeMask = MakeNetBitArrayView(ChangeMaskStorage);

	// To trigger copy of data
	Server->NetUpdate();
	Server->PostSendUpdate();
		
	// Collect with empty changemask
	{
		FCollectValidReferenceCollector Collector(false, ChangeMask);

		VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);
		UE_NET_ASSERT_EQ((uint32)Collector.References.Num(), 0U);
	}	

	// Collect init only
	{
		FCollectValidReferenceCollector Collector(true, ChangeMask);

		VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);
		UE_NET_ASSERT_EQ((uint32)Collector.References.Num(), 1U);
		UE_NET_ASSERT_EQ(Collector.References[0].GetRefHandle(), TestReferences[0]->NetRefHandle);
	}

	// Partial
	{
		FCollectValidReferenceCollector Collector(false, ChangeMask);
		ChangeMask.ClearAllBits();

		// Mark member 11 as dirty. This corresponds to TestStructWithNestedRefTArray_TArray.
		SetMemberDirty(ChangeMask, 11);

		VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

		UE_NET_ASSERT_EQ(Collector.References.Num(), 3);
		for (int32 It = 0; It < Collector.References.Num(); ++It)
		{
			UE_NET_ASSERT_EQ(Collector.References[It].GetRefHandle(), TestReferences[It + 20]->NetRefHandle);
		}
	}

	// Mark dirty
	{
		FCollectValidReferenceCollector Collector(true, ChangeMask);
		ChangeMask.SetAllBits();
		VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

		UE_NET_ASSERT_EQ((uint32)Collector.References.Num(), CurrentReferenceIndex);
		for (int32 It = 0; It < Collector.References.Num(); ++It)
		{
			UE_NET_ASSERT_EQ(Collector.References[It].GetRefHandle(), TestReferences[It]->NetRefHandle);
		}
	}

	// All but no init
	{
		FCollectValidReferenceCollector Collector(false, ChangeMask);
		ChangeMask.SetAllBits();
		VisitReferences(TestObjectStateBuffer, TestObjectProtocol, Collector);

		UE_NET_ASSERT_EQ((uint32)Collector.References.Num(), CurrentReferenceIndex - 1);
		for (int32 It = 0; It < Collector.References.Num(); ++It)
		{
			UE_NET_ASSERT_EQ(Collector.References[It].GetRefHandle(), TestReferences[It + 1]->NetRefHandle);
		}
	}
}

// Test that reference to inlined subobject works as expected
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestInlinedSubObjectReference)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestObjectReferences_TestClassWithDefaultSubObject* Object = Server->CreateObject<UTestObjectReferences_TestClassWithDefaultSubObject>();

	// Verify subobject
	UE_NET_ASSERT_TRUE(Object->CreatedSubObjectRef != nullptr);
	UE_NET_ASSERT_TRUE((Object->CreatedSubObjectRef->HasAnyFlags(RF_DefaultSubObject) || Object->IsDefaultSubobject()));

	// Replicate object
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that object has been spawned on client
	auto ClientObject = Client->GetObjectAs<UTestObjectReferences_TestClassWithDefaultSubObject>(Object->NetRefHandle);

	// Verify that default subobject is created
	UE_NET_ASSERT_TRUE(ClientObject->CreatedSubObjectRef != nullptr);
	UE_NET_ASSERT_TRUE((ClientObject->CreatedSubObjectRef->HasAnyFlags(RF_DefaultSubObject) || ClientObject->IsDefaultSubobject()));
	
	// Set a reference to subobject
	Object->ObjectRef = Object->CreatedSubObjectRef;

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that we could resolve the subobject ref and that we got the expected reference
	UE_NET_ASSERT_TRUE(ClientObject->ObjectRef != nullptr);
	UE_NET_ASSERT_EQ(ClientObject->ObjectRef.Get(), ClientObject->CreatedSubObjectRef.Get());
}

// Test that reference to separate subobject works as expected
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestExternalSubObjectReference)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestObjectReferences_TestClassWithDefaultSubObject* Object = Server->CreateObject<UTestObjectReferences_TestClassWithDefaultSubObject>();

	// Verify internal subobject
	UE_NET_ASSERT_TRUE(Object->CreatedSubObjectRef != nullptr);
	UE_NET_ASSERT_TRUE((Object->CreatedSubObjectRef->HasAnyFlags(RF_DefaultSubObject) || Object->IsDefaultSubobject()));

	// Create external subobject
	UTestObjectReferences_TestClassWithDefaultSubObject* ExternalSubObject = Server->CreateSubObject<UTestObjectReferences_TestClassWithDefaultSubObject>(Object->NetRefHandle);

	// Replicate object
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that object has been spawned on client
	UTestObjectReferences_TestClassWithDefaultSubObject* ClientObject = Client->GetObjectAs<UTestObjectReferences_TestClassWithDefaultSubObject>(Object->NetRefHandle);
	UTestObjectReferences_TestClassWithDefaultSubObject* ClientExternalSubObject = Client->GetObjectAs<UTestObjectReferences_TestClassWithDefaultSubObject>(ExternalSubObject->NetRefHandle);

	// Verify that default subobject is created
	UE_NET_ASSERT_TRUE(ClientObject->CreatedSubObjectRef != nullptr);
	UE_NET_ASSERT_TRUE(ClientExternalSubObject != nullptr);
	UE_NET_ASSERT_TRUE((ClientObject->CreatedSubObjectRef->HasAnyFlags(RF_DefaultSubObject) || ClientObject->IsDefaultSubobject()));
		
	// Set a reference to external subobject
	Object->ObjectRef = ExternalSubObject;

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that we could resolve the subobject ref and that we got the expected reference
	UE_NET_ASSERT_TRUE(ClientObject->ObjectRef != nullptr);
	UE_NET_ASSERT_EQ(static_cast<UTestObjectReferences_TestClassWithDefaultSubObject*>(ClientObject->ObjectRef.Get()), ClientExternalSubObject);
}

// Test partial resolve of references in array
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestPartialResolveOfArrayReferences)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestObjectReferences_TestClassWithReferences* ServerObject = Server->CreateObject<UTestObjectReferences_TestClassWithReferences>();

	// Spawn objects to reference
	UReplicatedTestObject* ServerReferencedObjectA = Server->CreateObject(0 ,0);
	UReplicatedTestObject* ServerReferencedObjectB = Server->CreateObject(0 ,0);
	UReplicatedTestObject* ServerReferencedObjectC = Server->CreateObject(0 ,0);

	// Setup references
	ServerObject->Ref_TArray.Add(ServerReferencedObjectA);
	ServerObject->Ref_TArray.Add(ServerReferencedObjectB);
	ServerObject->Ref_TArray.Add(ServerReferencedObjectC);

	// Make sure that references are not replicated to client
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectB->NetRefHandle);
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectC->NetRefHandle);
	
	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that object has been spawned on client
	auto ClientObject = Client->GetObjectAs<UTestObjectReferences_TestClassWithReferences>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(ClientObject != nullptr);

	// Verify that array has been replicated
	UE_NET_ASSERT_EQ(3, ClientObject->Ref_TArray.Num());

	// Verify that objects could not be resolved
	UE_NET_ASSERT_TRUE(ClientObject->Ref_TArray[0] == nullptr);
	UE_NET_ASSERT_TRUE(ClientObject->Ref_TArray[1] == nullptr);
	UE_NET_ASSERT_TRUE(ClientObject->Ref_TArray[2] == nullptr);

	// Enable replication for ServerReferenceObjectA
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that we managed to resolve objectA even though we have other unresolved objects in the array
	UE_NET_ASSERT_TRUE(ClientObject->Ref_TArray[0] != nullptr);
	UE_NET_ASSERT_TRUE(ClientObject->Ref_TArray[1] == nullptr);
	UE_NET_ASSERT_TRUE(ClientObject->Ref_TArray[2] == nullptr);

	// Enable replication for ServerReferenceObjectB 
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectB->NetRefHandle);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that we managed to resolve objectA even though we have other unresolved objects in the array
	UE_NET_ASSERT_TRUE(ClientObject->Ref_TArray[0] != nullptr);
	UE_NET_ASSERT_TRUE(ClientObject->Ref_TArray[1] != nullptr);
	UE_NET_ASSERT_TRUE(ClientObject->Ref_TArray[2] == nullptr);

	// Enable replication for ServerReferenceObjectB 
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectC->NetRefHandle);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();	

	// Verify that we managed to resolve objectA even though we have other unresolved objects in the array
	UE_NET_ASSERT_TRUE(ClientObject->Ref_TArray[0] != nullptr);
	UE_NET_ASSERT_TRUE(ClientObject->Ref_TArray[1] != nullptr);
	UE_NET_ASSERT_TRUE(ClientObject->Ref_TArray[2] != nullptr);
}

// Test partial out of order resolve of references in  fast array
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestOutOfOrderResolveInFastArray)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestObjectReferences_TestClassWithReferences* ServerObject = Server->CreateObject<UTestObjectReferences_TestClassWithReferences>();

	// Spawn objects to reference
	UReplicatedTestObject* ServerReferencedObjectA = Server->CreateObject(0 ,0);
	UReplicatedTestObject* ServerReferencedObjectB = Server->CreateObject(0 ,0);
	UReplicatedTestObject* ServerReferencedObjectC = Server->CreateObject(0 ,0);

	// Setup references
	auto& ServerFastArray = ServerObject->Ref_FastArray;

	FTestObjectReferences_TestObjectRefFastArrayItem Item;
	Item.ObjectRef = ServerReferencedObjectA;
	ServerFastArray.Items.Add(Item);
	ServerFastArray.MarkItemDirty(ServerFastArray.Items[0]);

	Item.ObjectRef = ServerReferencedObjectB;
	ServerFastArray.Items.Add(Item);
	ServerFastArray.MarkItemDirty(ServerFastArray.Items[1]);

	Item.ObjectRef = ServerReferencedObjectC;
	ServerFastArray.Items.Add(Item);
	ServerFastArray.MarkItemDirty(ServerFastArray.Items[2]);
#if !UE_BUILD_SHIPPING
	UE::Net::IrisDebugHelper::DebugOutputNetObjectState(ServerObject->NetRefHandle.GetId(), ServerObject->NetRefHandle.GetReplicationSystemId());
#endif

	// Make sure that references are not replicated to client
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectB->NetRefHandle);
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectC->NetRefHandle);
	
	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that object has been spawned on client
	auto ClientObject = Client->GetObjectAs<UTestObjectReferences_TestClassWithReferences>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(ClientObject != nullptr);

	auto& ClientFastArray = ClientObject->Ref_FastArray;

	// Verify that array has been replicated
	UE_NET_ASSERT_EQ(3, ClientFastArray.Items.Num());

	// Verify that objects could not be resolved
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[0].ObjectRef == nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[1].ObjectRef == nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[2].ObjectRef == nullptr);

	// Enable replication for ServerReferenceObjectA
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

#if !UE_BUILD_SHIPPING
	UE::Net::IrisDebugHelper::DebugOutputNetObjectState(ClientObject->NetRefHandle.GetId(), ClientObject->NetRefHandle.GetReplicationSystemId());
#endif

	// Verify that we managed to resolve objectA even though we have other unresolved objects in the array
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[0].ObjectRef != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[1].ObjectRef == nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[2].ObjectRef == nullptr);

	// Enable replication for ServerReferenceObjectB 
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectB->NetRefHandle);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that we managed to resolve objectB even though we have other unresolved objects in the array
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[0].ObjectRef != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[1].ObjectRef != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[2].ObjectRef == nullptr);

	// Enable replication for ServerReferenceObjectC
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectC->NetRefHandle);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();	

	// Verify that we managed to resolve all references
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[0].ObjectRef != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[1].ObjectRef != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[2].ObjectRef != nullptr);
}

// Test partial out of order resolve of references in  fast array
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestOutOfOrderResolveInNativeFastArray)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestObjectReferences_TestClassWithReferences* ServerObject = Server->CreateObject<UTestObjectReferences_TestClassWithReferences>();

	// Spawn objects to reference
	UReplicatedTestObject* ServerReferencedObjectA = Server->CreateObject(0 ,0);
	UReplicatedTestObject* ServerReferencedObjectB = Server->CreateObject(0 ,0);
	UReplicatedTestObject* ServerReferencedObjectC = Server->CreateObject(0 ,0);

	// Setup references
	auto ServerFastArrayEditor = ServerObject->Ref_NativeFastArray.Edit();

	FTestObjectReferences_TestObjectRefFastArrayItem Item;
	Item.ObjectRef = ServerReferencedObjectA;
	ServerFastArrayEditor.Add(Item);

	Item.ObjectRef = ServerReferencedObjectB;
	ServerFastArrayEditor.Add(Item);

	Item.ObjectRef = ServerReferencedObjectC;
	ServerFastArrayEditor.Add(Item);

	// Make sure that references are not replicated to client
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectB->NetRefHandle);
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectC->NetRefHandle);
	
	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that object has been spawned on client
	auto ClientObject = Client->GetObjectAs<UTestObjectReferences_TestClassWithReferences>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(ClientObject != nullptr);

	auto& ClientFastArray = ClientObject->Ref_NativeFastArray;

	// Verify that array has been replicated
	UE_NET_ASSERT_EQ(3, ClientFastArray.Items.Num());

	// Verify that objects could not be resolved
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[0].ObjectRef == nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[1].ObjectRef == nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[2].ObjectRef == nullptr);

	// Enable replication for ServerReferenceObjectA
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that we managed to resolve objectA even though we have other unresolved objects in the array
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[0].ObjectRef != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[1].ObjectRef == nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[2].ObjectRef == nullptr);

	// Enable replication for ServerReferenceObjectB 
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectB->NetRefHandle);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that we managed to resolve objectB even though we have other unresolved objects in the array
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[0].ObjectRef != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[1].ObjectRef != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[2].ObjectRef == nullptr);

	// Enable replication for ServerReferenceObjectC
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectC->NetRefHandle);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();	

	// Verify that we managed to resolve all references
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[0].ObjectRef != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[1].ObjectRef != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[2].ObjectRef != nullptr);

}

// Test partial out of order resolve of references in fast array
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestOutOfOrderResolveOfArrayInNativeFastArray)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestObjectReferences_TestClassWithReferences* ServerObject = Server->CreateObject<UTestObjectReferences_TestClassWithReferences>();

	// Spawn objects to reference
	UReplicatedTestObject* ServerReferencedObjectA = Server->CreateObject(0 ,0);
	UReplicatedTestObject* ServerReferencedObjectB = Server->CreateObject(0 ,0);
	UReplicatedTestObject* ServerReferencedObjectC = Server->CreateObject(0 ,0);

	// Setup references
	auto ServerFastArrayEditor = ServerObject->Ref_NativeFastArray.Edit();

	FTestObjectReferences_TestObjectRefFastArrayItem Item;
	Item.Ref_TArray.SetNum(1);
	Item.Ref_TArray[0] = ServerReferencedObjectA;
	Item.ObjectRef = nullptr;

	ServerFastArrayEditor.Add(Item);

	Item.Ref_TArray[0] = ServerReferencedObjectB;
	ServerFastArrayEditor.Add(Item);
	
	Item.Ref_TArray[0] = ServerReferencedObjectC;
	ServerFastArrayEditor.Add(Item);

	// Make sure that references are not replicated to client
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectB->NetRefHandle);
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectC->NetRefHandle);
	
	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Modify the array
	//Item.Ref_TArray[0] = ServerReferencedObjectC;
	//ServerFastArrayEditor.Add(Item);

	// Verify that object has been spawned on client
	auto ClientObject = Client->GetObjectAs<UTestObjectReferences_TestClassWithReferences>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(ClientObject != nullptr);

	auto& ClientFastArray = ClientObject->Ref_NativeFastArray;

	// Verify that array has been replicated
	UE_NET_ASSERT_EQ(3, ClientFastArray.Items.Num());

	// Verify that objects could not be resolved
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[0].Ref_TArray.Num() == 1);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[0].Ref_TArray[0] == nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[1].Ref_TArray.Num() == 1);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[1].Ref_TArray[0] == nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[2].Ref_TArray.Num() == 1);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[2].Ref_TArray[0] == nullptr);

	// Enable replication for ServerReferenceObjectA
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that we managed to resolve objectA even though we have other unresolved objects in the array
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[0].Ref_TArray[0] != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[1].Ref_TArray[0] == nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[2].Ref_TArray[0] == nullptr);

	// Enable replication for ServerReferenceObjectB 
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectB->NetRefHandle);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that we managed to resolve objectB even though we have other unresolved objects in the array
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[0].Ref_TArray[0] != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[1].Ref_TArray[0] != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[2].Ref_TArray[0] == nullptr);

	// Enable replication for ServerReferenceObjectC
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectC->NetRefHandle);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();	

	// Verify that we managed to resolve all references
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[0].Ref_TArray[0] != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[1].Ref_TArray[0] != nullptr);
	UE_NET_ASSERT_TRUE(ClientFastArray.Items[2].Ref_TArray[0] != nullptr);
}

// Replicate an object with a reference to another object which is later filtered out and then replicated again.
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestRemovedReferenceIsRemapped)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestObjectReferences_TestClassWithReferences* ServerObject = Server->CreateObject<UTestObjectReferences_TestClassWithReferences>();

	// Spawn object to be referenced
	UReplicatedTestObject* ServerReferencedObjectA = Server->CreateObject(0 ,0);

	// Setup references
	ServerObject->StructWithRef.ObjectRef = ServerReferencedObjectA;

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that object has been spawned on client
	auto ClientObject = Client->GetObjectAs<UTestObjectReferences_TestClassWithReferences>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(ClientObject != nullptr);

	// Verify that object can be resolved
	UE_NET_ASSERT_NE(ClientObject->StructWithRef.ObjectRef, TObjectPtr<UObject>(nullptr));

	// Disable replication for ServerReferenceObjectA
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);

	// Replicate to make sure object is destroyed on client
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Change the int value to force the reference in the struct to be re-resolved (normally GC would null them out).
	ServerObject->StructWithRef.IntValue ^= 1;

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that object can no longer be resolved
	UE_NET_ASSERT_EQ(ClientObject->StructWithRef.ObjectRef, TObjectPtr<UObject>(nullptr));

	// Enable replication for ServerReferenceObjectA
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Client will not send anything, but need object reference tracking update.
	Client->NetUpdate();
	Client->PostSendUpdate();

	// Verify that we can now resolve the referenced object again
	UE_NET_ASSERT_NE(ClientObject->StructWithRef.ObjectRef, TObjectPtr<UObject>(nullptr));
}

// Replicate an object with a reference to another object, in two different changemasks, which is later filtered out and then replicated again.
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestRemovedReferenceIsRemappedEverywhere)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestObjectReferences_TestClassWithReferences* ServerObject = Server->CreateObject<UTestObjectReferences_TestClassWithReferences>();

	// Spawn object to be referenced
	UReplicatedTestObject* ServerReferencedObjectA = Server->CreateObject(0 ,0);

	// Setup references
	ServerObject->StructWithRef.ObjectRef = ServerReferencedObjectA;
	ServerObject->StructWithRef_CArray[0].ObjectRef = ServerReferencedObjectA;

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Client will not send anything, but need object reference tracking update.
	Client->NetUpdate();
	Client->PostSendUpdate();

	// Verify that object has been spawned on client
	auto ClientObject = Client->GetObjectAs<UTestObjectReferences_TestClassWithReferences>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(ClientObject != nullptr);

	// Verify that object can be resolved
	UE_NET_ASSERT_NE(ClientObject->StructWithRef.ObjectRef, TObjectPtr<UObject>(nullptr));
	UE_NET_ASSERT_NE(ClientObject->StructWithRef_CArray[0].ObjectRef, TObjectPtr<UObject>(nullptr));

	// Disable replication for ServerReferenceObjectA
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);

	// Replicate to make sure object is destroyed on client
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Change the int values of the structs to force the object references to be re-resolved (normally GC would null them out).
	ServerObject->StructWithRef.IntValue ^= 1;
	ServerObject->StructWithRef_CArray[0].IntValue ^= 1;

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that object can no longer be resolved
	UE_NET_ASSERT_EQ(ClientObject->StructWithRef.ObjectRef, TObjectPtr<UObject>(nullptr));
	UE_NET_ASSERT_EQ(ClientObject->StructWithRef_CArray[0].ObjectRef, TObjectPtr<UObject>(nullptr));

	// Enable replication for ServerReferenceObjectA
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Client will not send anything, but need object reference tracking update.
	Client->NetUpdate();
	Client->PostSendUpdate();

	// Verify that we can now resolve the referenced object again
	UE_NET_ASSERT_NE(ClientObject->StructWithRef.ObjectRef, TObjectPtr<UObject>(nullptr));
	UE_NET_ASSERT_NE(ClientObject->StructWithRef_CArray[0].ObjectRef, TObjectPtr<UObject>(nullptr));
}

// Replicate an object with references to multiple objects, in a single changemask, which are later filtered out and then replicated again.
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestRemovingReferencesFromArrayRemapsEverywhere)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestObjectReferences_TestClassWithReferences* ServerObject = Server->CreateObject<UTestObjectReferences_TestClassWithReferences>();

	// Spawn objects to be referenced
	UReplicatedTestObject* ServerReferencedObjectA = Server->CreateObject(0, 0);
	UReplicatedTestObject* ServerReferencedObjectB = Server->CreateObject(0, 0);
	UReplicatedTestObject* ServerReferencedObjectC = Server->CreateObject(0, 0);
	UReplicatedTestObject* ServerReferencedObjectD = Server->CreateObject(0, 0);

	// Setup references
	auto& ServerTArray = ServerObject->StructWithRef_TArray;

	FTestObjectReferences_TestStructWithRef Item;

	Item.ObjectRef = ServerReferencedObjectA;
	ServerTArray.Add(Item);

	Item.ObjectRef = ServerReferencedObjectB;
	ServerTArray.Add(Item);

	Item.ObjectRef = ServerReferencedObjectC;
	ServerTArray.Add(Item);

	Item.ObjectRef = ServerReferencedObjectD;
	ServerTArray.Add(Item);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that object has been spawned on client
	auto ClientObject = Client->GetObjectAs<UTestObjectReferences_TestClassWithReferences>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	auto& ClientTArray = ClientObject->StructWithRef_TArray;

	// Verify that objects can be resolved
	UE_NET_ASSERT_NE(ClientTArray[0].ObjectRef, TObjectPtr<UObject>(nullptr));
	UE_NET_ASSERT_NE(ClientTArray[1].ObjectRef, TObjectPtr<UObject>(nullptr));
	UE_NET_ASSERT_NE(ClientTArray[2].ObjectRef, TObjectPtr<UObject>(nullptr));
	UE_NET_ASSERT_NE(ClientTArray[3].ObjectRef, TObjectPtr<UObject>(nullptr));

	// Disable replication for ServerReferenceObjectA, B and D.
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectB->NetRefHandle);
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectD->NetRefHandle);

	// Replicate to make sure objects are destroyed on client
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Client will not send anything, but need object reference tracking update.
	Client->NetUpdate();
	Client->PostSendUpdate();

	// Change the int values of the structs to force the object references to be re-resolved (normally GC would null them out).
	ServerTArray[0].IntValue ^= 1;
	ServerTArray[1].IntValue ^= 1;
	ServerTArray[2].IntValue ^= 1;
	ServerTArray[3].IntValue ^= 1;

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Client will not send anything, but need object reference tracking update.
	Client->NetUpdate();
	Client->PostSendUpdate();

	// Verify that objects A, B and D can no longer be resolved, while C still can be resolved.
	UE_NET_ASSERT_EQ(ClientTArray[0].ObjectRef, TObjectPtr<UObject>(nullptr));
	UE_NET_ASSERT_EQ(ClientTArray[1].ObjectRef, TObjectPtr<UObject>(nullptr));
	UE_NET_ASSERT_NE(ClientTArray[2].ObjectRef, TObjectPtr<UObject>(nullptr));
	UE_NET_ASSERT_EQ(ClientTArray[3].ObjectRef, TObjectPtr<UObject>(nullptr));

	// Enable replication for ServerReferenceObjectA, B and D.
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectA->NetRefHandle);
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectB->NetRefHandle);
	Server->ReplicationSystem->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerReferencedObjectD->NetRefHandle);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Client will not send anything, but need object reference tracking update.
	Client->NetUpdate();
	Client->PostSendUpdate();

	// Verify that we can now resolve all the referenced objects again
	UE_NET_ASSERT_NE(ClientTArray[0].ObjectRef, TObjectPtr<UObject>(nullptr));
	UE_NET_ASSERT_NE(ClientTArray[1].ObjectRef, TObjectPtr<UObject>(nullptr));
	UE_NET_ASSERT_NE(ClientTArray[2].ObjectRef, TObjectPtr<UObject>(nullptr));
	UE_NET_ASSERT_NE(ClientTArray[3].ObjectRef, TObjectPtr<UObject>(nullptr));
}

// Replicate an object with reference to an object which is later torn off.
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestTearOffDoesInvalidateReference)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestObjectReferences_TestClassWithReferences* ServerObject = Server->CreateObject<UTestObjectReferences_TestClassWithReferences>();

	// Spawn object to be referenced
	UReplicatedTestObject* ServerReferencedObjectA = Server->CreateObject(0 ,0);

	// Setup references
	ServerObject->StructWithRef.ObjectRef = ServerReferencedObjectA;

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Client will not send anything, but need object reference tracking update.
	Client->NetUpdate();
	Client->PostSendUpdate();

	// Verify that object has been spawned on client
	auto ClientObject = Client->GetObjectAs<UTestObjectReferences_TestClassWithReferences>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Verify that object can be resolved
	UE_NET_ASSERT_NE(ClientObject->StructWithRef.ObjectRef, TObjectPtr<UObject>(nullptr));

	// Tear off ServerReferenceObjectA
	Server->ReplicationBridge->EndReplication(ServerReferencedObjectA, EEndReplicationFlags::TearOff);

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Client will not send anything, but need object reference tracking update.
	Client->NetUpdate();
	Client->PostSendUpdate();

	// Change the int value to force the reference in the struct to be re-resolved
	ServerObject->StructWithRef.IntValue ^= 1;

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Client will not send anything, but need object reference tracking update.
	Client->NetUpdate();
	Client->PostSendUpdate();

	// Verify that object cannot be resolved
	UE_NET_ASSERT_EQ(ClientObject->StructWithRef.ObjectRef, TObjectPtr<UObject>(nullptr));
}

// Test that references to subobjects of stale dynamic objects works as epxected
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestStaleDefaultSubObjectReferences)
{
	TGuardConsoleVariable<bool> CVarOverride(TEXT("net.iris.VerifyOuterForStaleDynamicReferences"), true);

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestObjectReferences_TestClassWithDefaultSubObject* Object1 = Server->CreateObject<UTestObjectReferences_TestClassWithDefaultSubObject>();
	UTestObjectReferences_TestClassWithDefaultSubObject* Object2 = Server->CreateObject<UTestObjectReferences_TestClassWithDefaultSubObject>();
	// Have object2 reference the default subobject of object1
	Object2->ObjectRef = Object1->CreatedSubObjectRef;

	Server->UpdateAndSend({Client});

	// Store original references
	const FNetObjectReference OriginalRefToReplicatedObject1 = Server->ReplicationBridge->GetOrCreateObjectReference(Object1);
	const FNetObjectReference OriginalRefToSubObject = Server->ReplicationBridge->GetOrCreateObjectReference(Object1->CreatedSubObjectRef);

	// Verify that objects has been spawned on client and that we can resolve the replicated subobject via ref
	{
		UTestObjectReferences_TestClassWithDefaultSubObject* ClientObject1 = Client->GetObjectAs<UTestObjectReferences_TestClassWithDefaultSubObject>(Object1->NetRefHandle);
		UTestObjectReferences_TestClassWithDefaultSubObject* ClientObject2 = Client->GetObjectAs<UTestObjectReferences_TestClassWithDefaultSubObject>(Object2->NetRefHandle);
	
		UE_NET_ASSERT_NE(ClientObject1, nullptr);
		UE_NET_ASSERT_NE(ClientObject2, nullptr);
		UE_NET_ASSERT_NE(ClientObject2->ObjectRef.Get(), nullptr);
	}

	// Stop replicating object1, Note: Object is not destroyed on server, but we tell server to destroy it on client
	Server->ReplicationBridge->EndReplication(Object1, EEndReplicationFlags::Destroy);

	Server->UpdateAndSend({Client});

	constexpr bool bPerformFullPurge = true;
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, bPerformFullPurge);

	// Verify new status
	{
		// This will assign a new reference
		const FNetObjectReference NewRefToNotReplicatingObject1 = Server->ReplicationBridge->GetOrCreateObjectReference(Object1);
		UE_NET_ASSERT_FALSE(NewRefToNotReplicatingObject1 == OriginalRefToReplicatedObject1);

		// This will assign a new reference as the outer has changed.
		const FNetObjectReference NewRefToNotReplicatingSubObject = Server->ReplicationBridge->GetOrCreateObjectReference(Object1->CreatedSubObjectRef);
		UE_NET_ASSERT_FALSE(NewRefToNotReplicatingSubObject == OriginalRefToSubObject);

		UTestObjectReferences_TestClassWithDefaultSubObject* ClientObject1 = Client->GetObjectAs<UTestObjectReferences_TestClassWithDefaultSubObject>(Object1->NetRefHandle);
		UTestObjectReferences_TestClassWithDefaultSubObject* ClientObject2 = Client->GetObjectAs<UTestObjectReferences_TestClassWithDefaultSubObject>(Object2->NetRefHandle);
	
		UE_NET_ASSERT_EQ(ClientObject1, nullptr);
		UE_NET_ASSERT_NE(ClientObject2, nullptr);
		UE_NET_ASSERT_EQ(ClientObject2->ObjectRef.Get(), nullptr);
	}

	// Restart replication of object1, Note: since it is dynamic, it will get a new NetRefHandle
	Server->ReplicationBridge->BeginReplication(Object1);

	// Spawn objects on server
	UTestObjectReferences_TestClassWithDefaultSubObject* Object3 = Server->CreateObject<UTestObjectReferences_TestClassWithDefaultSubObject>();

	// Have object3 reference object1
	Object3->ObjectRef = Object1->CreatedSubObjectRef;

	Server->UpdateAndSend({Client});

	// Verify that objects has been spawned on client and that we can resolve the new replicated subobject for client object 3
	{
		UTestObjectReferences_TestClassWithDefaultSubObject* ClientObject1 = Client->GetObjectAs<UTestObjectReferences_TestClassWithDefaultSubObject>(Object1->NetRefHandle);
		UTestObjectReferences_TestClassWithDefaultSubObject* ClientObject2 = Client->GetObjectAs<UTestObjectReferences_TestClassWithDefaultSubObject>(Object2->NetRefHandle);
		UTestObjectReferences_TestClassWithDefaultSubObject* ClientObject3 = Client->GetObjectAs<UTestObjectReferences_TestClassWithDefaultSubObject>(Object3->NetRefHandle);
	
		UE_NET_ASSERT_NE(ClientObject1, nullptr);
		UE_NET_ASSERT_NE(ClientObject2, nullptr);
		// This will not resolve as it references a subobject of the old no longer existing instance of the dynamic object
		UE_NET_ASSERT_EQ(ClientObject2->ObjectRef.Get(), nullptr);
		UE_NET_ASSERT_NE(ClientObject3->ObjectRef.Get(), nullptr);
	}
}

// Test that references to nested default subobjects of stale replicated dynamic objects works as expected
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestStaleNestedDefaultSubObjectReferences)
{
	TGuardConsoleVariable<bool> CVarOverride(TEXT("net.iris.VerifyOuterForStaleDynamicReferences"), true);

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestObjectReferences_TestClassWithNestedDefaultSubObjects* Object1 = Server->CreateObject<UTestObjectReferences_TestClassWithNestedDefaultSubObjects>();
	UTestObjectReferences_TestClassWithNestedDefaultSubObjects* Object2 = Server->CreateObject<UTestObjectReferences_TestClassWithNestedDefaultSubObjects>();

	// Have object2 reference nested default subobject of object1->
	Object2->ObjectRef = Object1->CreatedSubObjectRef->CreatedSubObjectRef;

	Server->UpdateAndSend({Client});

	// Store original references
	const FNetObjectReference OriginalRefToReplicatedObject1 = Server->ReplicationBridge->GetOrCreateObjectReference(Object1);
	const FNetObjectReference OriginalRefToSubObject = Server->ReplicationBridge->GetOrCreateObjectReference(Object1->CreatedSubObjectRef->CreatedSubObjectRef);

	// Verify that objects has been spawned on client and that we can resolve the replicated subobject via ref
	{
		UTestObjectReferences_TestClassWithNestedDefaultSubObjects* ClientObject1 = Client->GetObjectAs<UTestObjectReferences_TestClassWithNestedDefaultSubObjects>(Object1->NetRefHandle);
		UTestObjectReferences_TestClassWithNestedDefaultSubObjects* ClientObject2 = Client->GetObjectAs<UTestObjectReferences_TestClassWithNestedDefaultSubObjects>(Object2->NetRefHandle);
	
		UE_NET_ASSERT_NE(ClientObject1, nullptr);
		UE_NET_ASSERT_NE(ClientObject2, nullptr);
		UE_NET_ASSERT_NE(ClientObject2->ObjectRef.Get(), nullptr);
	}

	// Stop replicating object1, Note: Object is not destroyed on server, but we tell server to destroy it on client
	Server->ReplicationBridge->EndReplication(Object1, EEndReplicationFlags::Destroy);

	Server->UpdateAndSend({Client});

	constexpr bool bPerformFullPurge = true;
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, bPerformFullPurge);

	// Verify new status
	{
		// This will assign a new reference
		const FNetObjectReference NewRefToNotReplicatingObject1 = Server->ReplicationBridge->GetOrCreateObjectReference(Object1);
		UE_NET_ASSERT_FALSE(NewRefToNotReplicatingObject1 == OriginalRefToReplicatedObject1);

		// This will assign a new reference as the outer has changed.
		const FNetObjectReference NewRefToNotReplicatingSubObject = Server->ReplicationBridge->GetOrCreateObjectReference(Object1->CreatedSubObjectRef->CreatedSubObjectRef);
		UE_NET_ASSERT_FALSE(NewRefToNotReplicatingSubObject == OriginalRefToSubObject);

		UTestObjectReferences_TestClassWithNestedDefaultSubObjects* ClientObject1 = Client->GetObjectAs<UTestObjectReferences_TestClassWithNestedDefaultSubObjects>(Object1->NetRefHandle);
		UTestObjectReferences_TestClassWithNestedDefaultSubObjects* ClientObject2 = Client->GetObjectAs<UTestObjectReferences_TestClassWithNestedDefaultSubObjects>(Object2->NetRefHandle);
	
		UE_NET_ASSERT_EQ(ClientObject1, nullptr);
		UE_NET_ASSERT_NE(ClientObject2, nullptr);
		UE_NET_ASSERT_EQ(ClientObject2->ObjectRef.Get(), nullptr);
	}

	// Restart replication of object1, Note: since it is dynamic, it will get a new NetRefHandle
	Server->ReplicationBridge->BeginReplication(Object1);

	// Spawn objects on server
	UTestObjectReferences_TestClassWithNestedDefaultSubObjects* Object3 = Server->CreateObject<UTestObjectReferences_TestClassWithNestedDefaultSubObjects>();

	// Have object3 reference object1
	Object3->ObjectRef = Object1->CreatedSubObjectRef->CreatedSubObjectRef;

	Server->UpdateAndSend({Client});

	// Verify that objects has been spawned on client and that we can resolve the new replicated subobject for client object 3
	{
		UTestObjectReferences_TestClassWithNestedDefaultSubObjects* ClientObject1 = Client->GetObjectAs<UTestObjectReferences_TestClassWithNestedDefaultSubObjects>(Object1->NetRefHandle);
		UTestObjectReferences_TestClassWithNestedDefaultSubObjects* ClientObject2 = Client->GetObjectAs<UTestObjectReferences_TestClassWithNestedDefaultSubObjects>(Object2->NetRefHandle);
		UTestObjectReferences_TestClassWithNestedDefaultSubObjects* ClientObject3 = Client->GetObjectAs<UTestObjectReferences_TestClassWithNestedDefaultSubObjects>(Object3->NetRefHandle);
	
		UE_NET_ASSERT_NE(ClientObject1, nullptr);
		UE_NET_ASSERT_NE(ClientObject2, nullptr);
		UE_NET_ASSERT_NE(ClientObject3, nullptr);

		// This will not resolve as it references a subobject of the old no longer existing instance of the dynamic object
		UE_NET_ASSERT_EQ(ClientObject2->ObjectRef.Get(), nullptr);

		UE_NET_ASSERT_NE(ClientObject3->ObjectRef.Get(), nullptr);
	}
}

}
