// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkAutomationTest.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Iris/ReplicationState/IrisFastArraySerializer.h"
#include "Iris/ReplicationState/Private/IrisFastArraySerializerInternal.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "Tests/Serialization/EnumTestTypes.h"
#include "TestObjectReferences.generated.h"

USTRUCT()
struct FTestObjectReferences_TestStructWithRef
{
	GENERATED_BODY();

	UPROPERTY()
	TObjectPtr<UObject> ObjectRef;

	UPROPERTY()
	int32 IntValue;
};

USTRUCT()
struct FTestObjectReferences_TestStructWithRefCArray
{
	enum Constants : uint32 { ArrayElementCount = 3, };

	GENERATED_BODY();

	UPROPERTY()
	TObjectPtr<UObject> Ref_CArray[ArrayElementCount];
};

USTRUCT()
struct FTestObjectReferences_TestStructWithRefTArray
{
	GENERATED_BODY();

	UPROPERTY()
	TArray<TObjectPtr<UObject>> Ref_TArray;
};

USTRUCT()
struct FTestObjectReferences_TestObjectRefFastArrayItem : public FFastArraySerializerItem
{
	GENERATED_BODY();

	UPROPERTY()
	TObjectPtr<UObject> ObjectRef;

	UPROPERTY()
	TArray<TObjectPtr<UObject>> Ref_TArray;
};

USTRUCT()
struct FTestObjectReferences_TestObjectRefNativeFastArray : public FIrisFastArraySerializer
{
	GENERATED_BODY()

	// Iris native interface
	typedef TArray<FTestObjectReferences_TestObjectRefFastArrayItem> ItemArrayType;
	const ItemArrayType& GetItemArray() const { return Items; }
	ItemArrayType& GetItemArray() { return Items; }
	
	typedef UE::Net::TIrisFastArrayEditor<FTestObjectReferences_TestObjectRefNativeFastArray> FFastArrayEditor;
	FFastArrayEditor Edit() { return FFastArrayEditor(*this); }	
	
	UPROPERTY()
	TArray<FTestObjectReferences_TestObjectRefFastArrayItem> Items;
};

USTRUCT()
struct FTestObjectReferences_TestObjectRefFastArray : public FFastArraySerializer
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FTestObjectReferences_TestObjectRefFastArrayItem> Items;
};

USTRUCT()
struct FTestObjectReferences_TestStructWithNestedRefCArray
{
	enum Constants : uint32 { ArrayElementCount = 3, };

	GENERATED_BODY();

	UPROPERTY()
	FTestObjectReferences_TestStructWithRefCArray StructWithRef_CArray[ArrayElementCount];
};

USTRUCT()
struct FTestObjectReferences_TestStructWithNestedRefTArray
{
	GENERATED_BODY();

	UPROPERTY()
	TArray<FTestObjectReferences_TestStructWithRefTArray> StructWithRef_TArray;
};

UCLASS()
class UTestObjectReferences_TestClassWithReferences : public UReplicatedTestObject
{
	enum Constants : uint32 { ArrayElementCount = 3, };

	GENERATED_BODY()

public:
	UTestObjectReferences_TestClassWithReferences() : UReplicatedTestObject() {}

	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	UPROPERTY(Transient, Replicated)
	TObjectPtr<UObject> ObjectRef;

	UPROPERTY(Transient, Replicated)
	FTestObjectReferences_TestStructWithRef StructWithRef;

	UPROPERTY(Transient, Replicated)
	FTestObjectReferences_TestStructWithRefCArray StructWithRefCArray;

	UPROPERTY(Transient, Replicated)
	FTestObjectReferences_TestStructWithRefTArray StructWithRefTArray;

	UPROPERTY(Transient, Replicated)
	TObjectPtr<UObject> Ref_CArray[ArrayElementCount];

	UPROPERTY(Transient, Replicated)
	TArray<TObjectPtr<UObject>> Ref_TArray;

	UPROPERTY(Transient, Replicated)
	FTestObjectReferences_TestStructWithRef StructWithRef_CArray[ArrayElementCount];

	UPROPERTY(Transient, Replicated)
	TArray<FTestObjectReferences_TestStructWithRef> StructWithRef_TArray;

	UPROPERTY(Transient, Replicated)
	TArray<FTestObjectReferences_TestStructWithNestedRefTArray> TestStructWithNestedRefTArray_TArray;

	UPROPERTY(Transient, Replicated)
	FTestObjectReferences_TestStructWithNestedRefCArray TestStructWithNestedRefCArray_CArray[ArrayElementCount];

	UPROPERTY(Transient, Replicated)
	FTestObjectReferences_TestObjectRefFastArray Ref_FastArray;

	UPROPERTY(Transient, Replicated)
	FTestObjectReferences_TestObjectRefNativeFastArray Ref_NativeFastArray;
};

UCLASS()
class UTestObjectReferences_TestClassWithDefaultSubObject : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UTestObjectReferences_TestClassWithDefaultSubObject(); /* : UReplicatedTestObject() {} */

	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	UPROPERTY(Transient, Replicated)
	TWeakObjectPtr<UObject> ObjectRef;

	// Used to store subobject created before we start replication
	UPROPERTY(Transient)
	TObjectPtr<UObject> CreatedSubObjectRef;
};

UCLASS()
class UTestObjectReferences_TestClassWithNestedDefaultSubObjects : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UTestObjectReferences_TestClassWithNestedDefaultSubObjects(); /* : UReplicatedTestObject() {} */

	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	UPROPERTY(Transient, Replicated)
	TWeakObjectPtr<UObject> ObjectRef;

	// Used to store subobject created before we start replication
	UPROPERTY(Transient)
	TObjectPtr<UTestObjectReferences_TestClassWithDefaultSubObject> CreatedSubObjectRef;
};


