// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/IConsoleManager.h"
#include "Iris/ReplicationSystem/NetBlob/NetObjectBlobHandler.h"
#include "Iris/ReplicationSystem/ReplicationRecord.h"
#include "Misc/ScopeExit.h"
#include "NetBlob/NetBlobTestFixture.h"
#include "NetBlob/MockNetBlob.h"
#include "NetBlob/MockNetObjectAttachment.h"

namespace UE::Net::Private
{

class FSplitObjectTestFixture : public FNetBlobTestFixture
{
	typedef FNetBlobTestFixture Super;

public:
	enum : uint32
	{
		HugeObjectPayloadByteCount = 16384,
		HugeObjectMaxNetTickCountToArrive = 32U,
	};

public:
	FSplitObjectTestFixture()
	{
	}

protected:
	virtual void SetUp() override
	{
		AddNetBlobHandlerDefinitions();
		Super::SetUp();
		RegisterNetBlobHandlers(Server);
	}

	virtual void TearDown() override
	{
		Super::TearDown();
	}

	void RegisterNetBlobHandlers(FReplicationSystemTestNode* Node)
	{
		UReplicationSystem* RepSys = Node->GetReplicationSystem();
		const bool bIsServer = RepSys->IsServer();

		FNetBlobHandlerManager* NetBlobHandlerManager = &RepSys->GetReplicationSystemInternal()->GetNetBlobHandlerManager();

		{
			UMockNetObjectAttachmentHandler* BlobHandler = NewObject<UMockNetObjectAttachmentHandler>();
			const bool bMockNetObjectAttachmentHandlerWasRegistered = RegisterNetBlobHandler(RepSys, BlobHandler);
			check(bMockNetObjectAttachmentHandlerWasRegistered);

			if (bIsServer)
			{
				ServerMockNetObjectAttachmentHandler = TStrongObjectPtr<UMockNetObjectAttachmentHandler>(BlobHandler);
			}
			else
			{
				ClientMockNetObjectAttachmentHandler = TStrongObjectPtr<UMockNetObjectAttachmentHandler>(BlobHandler);
			}
		}
	}

	void SetObjectPayloadByteCount(UTestReplicatedIrisObject* Object, uint32 ByteCount)
	{
		UTestReplicatedIrisDynamicStatePropertyComponent* Component = Object->DynamicStateComponents[0].Get();
		Component->IntArray.SetNumZeroed(ByteCount/4U);
	}

	UTestReplicatedIrisObject* CreateObject(FReplicationSystemTestNode* Node)
	{
		UTestReplicatedIrisObject::FComponents Components;
		Components.DynamicStateComponentCount = 1;
		UTestReplicatedIrisObject* Object = Node->CreateObject(Components);

		return Object;
	}

	UTestReplicatedIrisObject* CreateSubObject(FReplicationSystemTestNode* Node, FNetRefHandle Parent)
	{
		UTestReplicatedIrisObject::FComponents Components;
		Components.DynamicStateComponentCount = 1;
		UTestReplicatedIrisObject* Object = Node->CreateSubObject(Parent, Components);

		return Object;
	}

	UTestReplicatedIrisObject* CreateHugeObject(FReplicationSystemTestNode* Node)
	{
		UTestReplicatedIrisObject* Object = CreateObject(Node);
		SetObjectPayloadByteCount(Object, HugeObjectPayloadByteCount);
		return Object;
	}

private:
	void AddNetBlobHandlerDefinitions()
	{
		AddMockNetBlobHandlerDefinition();
		const FNetBlobHandlerDefinition NetBlobHandlerDefinitions[] = 
		{
			{TEXT("MockNetObjectAttachmentHandler"),},
			// The proper partial attachment and net object blob handlers are needed for splitting huge objects and attachments.
			{TEXT("PartialNetObjectAttachmentHandler"),}, 
			{TEXT("NetObjectBlobHandler"),}, 
		};
		Super::AddNetBlobHandlerDefinitions(NetBlobHandlerDefinitions, UE_ARRAY_COUNT(NetBlobHandlerDefinitions));
	}


protected:
	TStrongObjectPtr<UMockNetObjectAttachmentHandler> ServerMockNetObjectAttachmentHandler;
	TStrongObjectPtr<UMockNetObjectAttachmentHandler> ClientMockNetObjectAttachmentHandler;
};

// Test that huge object state can be replicated on creation.
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, SplitHugeObjectOnCreation)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestReplicatedIrisObject* ServerObject = CreateHugeObject(Server);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// As the payload is huge we don't expect the whole payload to arrive the first frame
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_EXPECT_EQ(ClientObject, nullptr);

	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive && ClientObject == nullptr; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
		ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	}
	UE_NET_ASSERT_NE(ClientObject, nullptr);
}

// Test that huge object state can be replicated after an object has been created.
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, SplitHugeObjectAfterCreation)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestReplicatedIrisObject* ServerObject = CreateObject(Server);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	SetObjectPayloadByteCount(ServerObject, HugeObjectPayloadByteCount);

	// Clear function call status so we can easily verify we get the huge payload.
	UTestReplicatedIrisDynamicStatePropertyComponent* Component = ClientObject->DynamicStateComponents[0].Get();
	Component->CallCounts = {};

	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive && Component->CallCounts.IntArrayRepNotifyCounter == 0; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
	}
	UE_NET_ASSERT_GT(Component->CallCounts.IntArrayRepNotifyCounter, 0U);
}

// Test that object with huge subobjects can be replicated on creation.
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, SplitObjectWithHugeSubObjectsOnCreation)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestReplicatedIrisObject* ServerObject = CreateObject(Server);
	constexpr uint32 SubObjectCount = 3;
	UTestReplicatedIrisObject* ServerSubObjects[SubObjectCount];
	for (uint32 SubObjectIt=0; SubObjectIt != SubObjectCount; ++SubObjectIt)
	{
		ServerSubObjects[SubObjectIt] = CreateSubObject(Server, ServerObject->NetRefHandle);
		SetObjectPayloadByteCount(ServerSubObjects[SubObjectIt], 4096U);
	}

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// As the payload is huge we don't expect the whole payload to arrive the first frame
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_EXPECT_EQ(ClientObject, nullptr);

	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive && ClientObject == nullptr; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
		ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	}
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Verify the subobjects made it through as well.
	for (uint32 SubObjectIt=0; SubObjectIt != SubObjectCount; ++SubObjectIt)
	{
		const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObjects[SubObjectIt]->NetRefHandle));
		UE_NET_ASSERT_NE(ClientSubObject, nullptr);
	}
}

// Test that object with lots of subobjects with attachments can be sent on creation.
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, SplitObjectWithSubObjectsWithHugeAttachmentsOnCreation)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	constexpr uint32 SubObjectCount = 16;
	constexpr uint32 SubObjectPayloadByteCount = 128U;
	constexpr uint32 SubObjectAttachmentPayloadByteCount = 128U;

	UTestReplicatedIrisObject* ServerObject = CreateObject(Server);
	UTestReplicatedIrisObject* ServerSubObjects[SubObjectCount];

	for (uint32 SubObjectIt=0; SubObjectIt != SubObjectCount; ++SubObjectIt)
	{
		UTestReplicatedIrisObject* ServerSubObject = CreateSubObject(Server, ServerObject->NetRefHandle);
		ServerSubObjects[SubObjectIt] = ServerSubObject;
		SetObjectPayloadByteCount(ServerSubObject, SubObjectPayloadByteCount);

		TRefCountPtr<FNetObjectAttachment> Attachment;
		// Alternate between reliable and unreliable attachments
		if ((SubObjectIt & 1U) != 0)
		{
			Attachment = ServerMockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(SubObjectPayloadByteCount*8U);
		}
		else
		{
			Attachment = ServerMockNetObjectAttachmentHandler->CreateUnreliableNetObjectAttachment(SubObjectPayloadByteCount*8U);
		}

		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerSubObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// As the payload is huge we don't expect the whole payload to arrive the first frame
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_EXPECT_EQ(ClientObject, nullptr);

	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive && ClientObject == nullptr; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
		ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	}
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Verify the subobjects made it through.
	for (uint32 SubObjectIt=0; SubObjectIt != SubObjectCount; ++SubObjectIt)
	{
		const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObjects[SubObjectIt]->NetRefHandle));
		UE_NET_ASSERT_NE(ClientSubObject, nullptr);
	}

	// Wait for attachments
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
	}

	// Verify the attachments made it through.
	UMockNetObjectAttachmentHandler::FCallCounts AttachmentCallCounts = ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts();
	UE_NET_ASSERT_EQ(AttachmentCallCounts.OnNetBlobReceived, SubObjectCount);
}

// Test that object with lots of subobjects with attachments can be sent after creation.
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, SplitObjectWithSubObjectsWithHugeAttachmentsAfterCreation)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	constexpr uint32 SubObjectCount = 16;
	constexpr uint32 SubObjectPayloadByteCount = 128U;
	constexpr uint32 SubObjectAttachmentPayloadByteCount = 128U;

	UTestReplicatedIrisObject* ServerObject = CreateObject(Server);
	UTestReplicatedIrisObject* ServerSubObjects[SubObjectCount];

	for (uint32 SubObjectIt=0; SubObjectIt != SubObjectCount; ++SubObjectIt)
	{
		UTestReplicatedIrisObject* ServerSubObject = CreateSubObject(Server, ServerObject->NetRefHandle);
		ServerSubObjects[SubObjectIt] = ServerSubObject;
	}

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// As the payload is huge we don't expect the whole payload to arrive the first frame
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Verify the subobjects made it through.
	for (uint32 SubObjectIt=0; SubObjectIt != SubObjectCount; ++SubObjectIt)
	{
		const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObjects[SubObjectIt]->NetRefHandle));
		UE_NET_ASSERT_NE(ClientSubObject, nullptr);
	}

	// Now create huge payload and attachments for each subobject.
	for (uint32 SubObjectIt=0; SubObjectIt != SubObjectCount; ++SubObjectIt)
	{
		UTestReplicatedIrisObject* ServerSubObject = ServerSubObjects[SubObjectIt];
		SetObjectPayloadByteCount(ServerSubObject, SubObjectPayloadByteCount);

		TRefCountPtr<FNetObjectAttachment> Attachment;
		// Alternate between reliable and unreliable attachments
		if ((SubObjectIt & 1U) != 0)
		{
			Attachment = ServerMockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(SubObjectPayloadByteCount*8U);
		}
		else
		{
			Attachment = ServerMockNetObjectAttachmentHandler->CreateUnreliableNetObjectAttachment(SubObjectPayloadByteCount*8U);
		}

		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerSubObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	bool bHasReceivedHugeState = false;
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive && !bHasReceivedHugeState; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();

		// Assume that if one subobject has received its huge state then all of them have
		const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObjects[SubObjectCount - 1U]->NetRefHandle));
		if (ClientSubObject->DynamicStateComponents[0]->IntArray.Num() > 0)
		{
			bHasReceivedHugeState = true;
		}
	}
	UE_NET_ASSERT_TRUE(bHasReceivedHugeState);

	// Verify the attachments made it through.
	UMockNetObjectAttachmentHandler::FCallCounts AttachmentCallCounts = ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts();
	UE_NET_ASSERT_EQ(AttachmentCallCounts.OnNetBlobReceived, SubObjectCount);
}

// Test that object can have consecutive huge states.
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, HugeObjectStateCanBeSentBackToBack)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	constexpr uint32 SubObjectCount = 16;
	constexpr uint32 SubObjectPayloadByteCount = 128U;
	constexpr uint32 SubObjectAttachmentPayloadByteCount = 128U;

	UTestReplicatedIrisObject* ServerObject = CreateObject(Server);
	UTestReplicatedIrisObject* ServerSubObjects[SubObjectCount];

	for (uint32 SubObjectIt=0; SubObjectIt != SubObjectCount; ++SubObjectIt)
	{
		UTestReplicatedIrisObject* ServerSubObject = CreateSubObject(Server, ServerObject->NetRefHandle);
		ServerSubObjects[SubObjectIt] = ServerSubObject;
	}

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// As the payload is huge we don't expect the whole payload to arrive the first frame
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Verify the subobjects made it through.
	for (uint32 SubObjectIt=0; SubObjectIt != SubObjectCount; ++SubObjectIt)
	{
		const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObjects[SubObjectIt]->NetRefHandle));
		UE_NET_ASSERT_NE(ClientSubObject, nullptr);
	}

	// Now create huge payload and attachments for each subobject.
	for (uint32 SubObjectIt=0; SubObjectIt != SubObjectCount; ++SubObjectIt)
	{
		UTestReplicatedIrisObject* ServerSubObject = ServerSubObjects[SubObjectIt];
		SetObjectPayloadByteCount(ServerSubObject, SubObjectPayloadByteCount);

		TRefCountPtr<FNetObjectAttachment> Attachment;
		// Alternate between reliable and unreliable attachments
		if ((SubObjectIt & 1U) != 0)
		{
			Attachment = ServerMockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(SubObjectPayloadByteCount*8U);
		}
		else
		{
			Attachment = ServerMockNetObjectAttachmentHandler->CreateUnreliableNetObjectAttachment(SubObjectPayloadByteCount*8U);
		}

		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerSubObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	bool bHasReceivedHugeState = false;
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive && !bHasReceivedHugeState; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();

		// Assume that if one subobject has received its huge state then all of them have
		const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObjects[SubObjectCount - 1U]->NetRefHandle));
		if (ClientSubObject->DynamicStateComponents[0]->IntArray.Num() > 0)
		{
			bHasReceivedHugeState = true;
		}
	}
	UE_NET_ASSERT_TRUE(bHasReceivedHugeState);

	// Verify the attachments made it through.
	UMockNetObjectAttachmentHandler::FCallCounts AttachmentCallCounts = ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts();
	UE_NET_ASSERT_EQ(AttachmentCallCounts.OnNetBlobReceived, SubObjectCount);
}

// Test that we can send one huge object after another.
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, SplitObjectCanBeSentBackToBack)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestReplicatedIrisObject* ServerObject = CreateHugeObject(Server);

	// Send and deliver packet. This will initiate huge object transfer.
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// As the payload is huge we don't expect the whole payload to arrive the first frame
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_EQ(ClientObject, nullptr);

	const int OriginalArrayCount = ServerObject->DynamicStateComponents[0]->IntArray.Num();

	// Modify the payload which will cause the same object to require a huge object transfer again.
	ServerObject->DynamicStateComponents[0]->IntArray.Add(1);

	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive && ClientObject == nullptr; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
		ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	}
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_EQ(ClientObject->DynamicStateComponents[0]->IntArray.Num(), OriginalArrayCount);

	bool bHasReceivedSecondHugeState = false;
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive && !bHasReceivedSecondHugeState; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();

		if (ClientObject->DynamicStateComponents[0]->IntArray.Num() == OriginalArrayCount + 1)
		{
			bHasReceivedSecondHugeState = true;
		}
	}
	UE_NET_ASSERT_TRUE(bHasReceivedSecondHugeState);
}

// Test that a huge object can be deleted. Currently we assume the object must be created before deleted.
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, SplitObjectIsDeletedAfterBeingCreated)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestReplicatedIrisObject* ServerObject = CreateHugeObject(Server);
	const FNetRefHandle ServerNetRefHandle = ServerObject->NetRefHandle;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// As the payload is huge we don't expect the whole payload to arrive the first frame
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerNetRefHandle));
	UE_NET_ASSERT_EQ(ClientObject, nullptr);

	Server->DestroyObject(ServerObject);

	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive && ClientObject == nullptr; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();
		ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerNetRefHandle));
	}
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// The object should be destroyed after the next net update.
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerNetRefHandle));
	UE_NET_ASSERT_EQ(ClientObject, nullptr);
}

// Test that a subobject to a huge object can be deleted properly. Currently we assume the huge object payload must have been received before the subobject can be deleted.
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, SubObjectToHugeObjectCanBeDeleted)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestReplicatedIrisObject* ServerObject = CreateObject(Server);
	constexpr uint32 SubObjectCount = 3;
	UTestReplicatedIrisObject* ServerSubObjects[SubObjectCount];
	for (uint32 SubObjectIt=0; SubObjectIt != SubObjectCount; ++SubObjectIt)
	{
		ServerSubObjects[SubObjectIt] = CreateSubObject(Server, ServerObject->NetRefHandle);
	}

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	const FNetRefHandle SubObjectNetRefHandle = ServerSubObjects[0]->NetRefHandle;
	const UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientSubObject, nullptr);

	// Make subobject payloads huge
	for (uint32 SubObjectIt=0; SubObjectIt != SubObjectCount; ++SubObjectIt)
	{
		UTestReplicatedIrisObject* ServerSubObject = ServerSubObjects[SubObjectIt];
		SetObjectPayloadByteCount(ServerSubObjects[SubObjectIt], 4096U);
	}

	// Initiate sending so that we have huge data in flight with the subobject we are going to destroy.
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	Server->DestroyObject(ServerSubObjects[0]);

	bool bHasReceivedHugeState = false;
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive && !bHasReceivedHugeState; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();

		// Assume that if one subobject has received its huge state then all of them have
		ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectNetRefHandle));
		UE_NET_ASSERT_NE(ClientSubObject, nullptr);
		if (ClientSubObject->DynamicStateComponents[0]->IntArray.Num() > 0)
		{
			bHasReceivedHugeState = true;
		}
	}
	UE_NET_ASSERT_TRUE(bHasReceivedHugeState);

	// Now the subobject can safely be destroyed
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(SubObjectNetRefHandle));
	UE_NET_ASSERT_EQ(ClientSubObject, nullptr);
}

// Test TearOff for new huge object
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, TearOffOnCreation)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestReplicatedIrisObject* ServerObject = CreateHugeObject(Server);

	// TearOff the object
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	const int32 NumObjectsCreatedOnClientBeforeReplication = Client->CreatedObjects.Num();

	bool bHasHugeObjectBeenCreated = false;
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive && !bHasHugeObjectBeenCreated; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();

		if (Client->CreatedObjects.Num() > NumObjectsCreatedOnClientBeforeReplication)
		{
			bHasHugeObjectBeenCreated = true;
		}
	}
	UE_NET_ASSERT_EQ(Client->CreatedObjects.Num(), NumObjectsCreatedOnClientBeforeReplication + 1);

	// Verify that ClientObject is torn-off and that the final state was applied
	UTestReplicatedIrisObject* ClientObjectThatWasTornOff = Cast<UTestReplicatedIrisObject>(Client->CreatedObjects[NumObjectsCreatedOnClientBeforeReplication].Get());
	UE_NET_ASSERT_EQ(ClientObjectThatWasTornOff->DynamicStateComponents[0]->IntArray.Num(), ServerObject->DynamicStateComponents[0]->IntArray.Num());
	UE_NET_ASSERT_EQ(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)), nullptr);
}

// Test TearOff for existing confirmed object during huge object state send
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, TearOffCreatedObjectWithHugePayload)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestReplicatedIrisObject* ServerObject = CreateObject(Server);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Store client object while it can still be found using the server net handle.
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObjectThatWillBeTornOff, nullptr);

	// Set huge payload and TearOff the object
	SetObjectPayloadByteCount(ServerObject, HugeObjectPayloadByteCount);
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	bool bHasReceivedHugeState = false;
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive && !bHasReceivedHugeState; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();

		if (ClientObjectThatWillBeTornOff->DynamicStateComponents[0]->IntArray.Num() > 0)
		{
			bHasReceivedHugeState = true;
		}
	}

	// Verify that ClientObject is torn-off and that the final state was applied
	UE_NET_ASSERT_EQ(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)), nullptr);
	UE_NET_ASSERT_EQ(ClientObjectThatWillBeTornOff->DynamicStateComponents[0]->IntArray.Num(), ServerObject->DynamicStateComponents[0]->IntArray.Num());
}

// Test TearOff while huge object state is still sending.
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, TearOffWhileHugeObjectStateIsSending)
{
	FReplicationSystemTestClient* Client = CreateClient();

	UTestReplicatedIrisObject* ServerObject = CreateHugeObject(Server);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Tear off object before it has been created on the client.
	ServerObject->IntA ^= 1;
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	const int32 NumObjectsCreatedOnClientBeforeReplication = Client->CreatedObjects.Num();

	bool bHasHugeObjectBeenCreated = false;
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive && !bHasHugeObjectBeenCreated; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();

		if (Client->CreatedObjects.Num() > NumObjectsCreatedOnClientBeforeReplication)
		{
			bHasHugeObjectBeenCreated = true;
		}
	}
	UE_NET_ASSERT_EQ(Client->CreatedObjects.Num(), NumObjectsCreatedOnClientBeforeReplication + 1);

	// Verify we have the previous state
	const UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_EQ((ClientObjectThatWillBeTornOff->IntA ^ 1), ServerObject->IntA);

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that ClientObject is torn-off and that the final state was applied
	UE_NET_ASSERT_EQ(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)), nullptr);
	UE_NET_ASSERT_EQ(ClientObjectThatWillBeTornOff->IntA, ServerObject->IntA);
}

UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, TestCancelPendingDestroyOfHugeObjectDuringWaitOnCreateConfirmationWithoutPacketLoss)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = CreateHugeObject(Server);

	// Write packets
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendUpdate(Client->ConnectionIdOnServer);
		Server->PostSendUpdate();
	}

	// Filter out object to cause a PendingDestroy
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Remove object from filter to cause object to end up in CancelPendingDestroy
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Deliver object creation packets
	{
		SIZE_T PacketCount = 0;
		const auto& ConnectionInfo = Server->GetConnectionInfo(Client->ConnectionIdOnServer);
		PacketCount = ConnectionInfo.WrittenPackets.Count();
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, DeliverPacket);
		}
	}

	// Verify that the object now exists on client
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Modify a property on the object and make sure it's replicated as the object should now be confirmed created
	ServerObject->IntA ^= 1;

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
}


UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, TestCancelPendingDestroyOfHugeObjectDuringWaitOnCreateConfirmationWithPacketLoss)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = CreateHugeObject(Server);

	// Write packets
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendUpdate(Client->ConnectionIdOnServer);
		Server->PostSendUpdate();
	}

	// Filter out object to cause a PendingDestroy
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Remove object from filter to cause object to end up in CancelPendingDestroy
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Cause packet loss on object creation
	{
		SIZE_T PacketCount = 0;
		const auto& ConnectionInfo = Server->GetConnectionInfo(Client->ConnectionIdOnServer);
		PacketCount = ConnectionInfo.WrittenPackets.Count();
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, DoNotDeliverPacket);
		}
	}

	// Write and send packets and verify object is created
	{
		const int32 NumObjectsCreatedOnClientBeforeReplication = Client->CreatedObjects.Num();

		for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive; ++RetryIt)
		{
			Server->NetUpdate();
			Server->SendAndDeliverTo(Client, DeliverPacket);
			Server->PostSendUpdate();

			if (Client->CreatedObjects.Num() > NumObjectsCreatedOnClientBeforeReplication)
			{
				break;
			}
		}
	}

	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, TestCancelPendingDestroyDuringHugeObjectStateUpdate)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = CreateObject(Server);

	// Write and send packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Force huge object state
	SetObjectPayloadByteCount(ServerObject, HugeObjectPayloadByteCount);

	// Write packets
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendUpdate(Client->ConnectionIdOnServer);
		Server->PostSendUpdate();
	}

	// Filter out object to cause a PendingDestroy
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Remove object from filter to cause object to end up in CancelPendingDestroy
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->PostSendUpdate();

	// Modify a property on the object and make sure it's replicated as the object should still be created
	ServerObject->IntA ^= 1;

	// Deliver huge state
	{
		SIZE_T PacketCount = 0;
		const auto& ConnectionInfo = Server->GetConnectionInfo(Client->ConnectionIdOnServer);
		PacketCount = ConnectionInfo.WrittenPackets.Count();
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, DeliverPacket);
		}
	}

	// Deliver latest state
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
}

UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, TestReliableAttachmentIsDeliveredDespiteHugeObjectBeingDestroyed)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = CreateObject(Server);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Force huge object state
	SetObjectPayloadByteCount(ServerObject, HugeObjectPayloadByteCount);

	// Add reliable attachment
	{
		TRefCountPtr<FNetObjectAttachment> Attachment = ServerMockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(1U);

		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Write packets
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendUpdate(Client->ConnectionIdOnServer);
		Server->PostSendUpdate();
	}

	// Filter out object to cause object to be set in state WaitOnFlush
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Deliver huge state
	{
		SIZE_T PacketCount = 0;
		const auto& ConnectionInfo = Server->GetConnectionInfo(Client->ConnectionIdOnServer);
		PacketCount = ConnectionInfo.WrittenPackets.Count();
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, DeliverPacket);
		}
	}

	// Deliver latest state
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify the attachment made it through, despite the wish to destroy the object.
	const UMockNetObjectAttachmentHandler::FCallCounts AttachmentCallCounts = ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts();
	UE_NET_ASSERT_EQ(AttachmentCallCounts.OnNetBlobReceived, 1U);

	// The object should not exist
	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_EQ(ClientObject, nullptr);
}

UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, TestHugeObjectIsFlushedAndNotDestroyedWhenFilteredOutAndThenIn)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = CreateObject(Server);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Force huge object state
	SetObjectPayloadByteCount(ServerObject, HugeObjectPayloadByteCount);

	// Add reliable attachment
	{
		TRefCountPtr<FNetObjectAttachment> Attachment = ServerMockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(1U);

		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Write packets
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendUpdate(Client->ConnectionIdOnServer);
		Server->PostSendUpdate();
	}

	// Filter out object to cause object to be set in state WaitOnFlush
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Remove object from filter to cause object to be set in state Created
	Server->GetReplicationSystem()->RemoveFromGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Deliver huge state
	{
		SIZE_T PacketCount = 0;
		const auto& ConnectionInfo = Server->GetConnectionInfo(Client->ConnectionIdOnServer);
		PacketCount = ConnectionInfo.WrittenPackets.Count();
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, DeliverPacket);
		}
	}

	// Modify a property on the object and make sure it's replicated as the object should still be created.
	ServerObject->IntA += 1;

	// Deliver latest state
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify the attachment made it through
	const UMockNetObjectAttachmentHandler::FCallCounts AttachmentCallCounts = ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts();
	UE_NET_ASSERT_EQ(AttachmentCallCounts.OnNetBlobReceived, 1U);

	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
}

// Test that huge object state can be replicated on creation.
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, SplitManyHugeObjectsOnCreation)
{
	FReplicationSystemTestClient* Client = CreateClient();

	constexpr uint32 HugeObjectCount = 37U;
	UTestReplicatedIrisObject* ServerObjects[HugeObjectCount];
	for (UTestReplicatedIrisObject*& ServerObject : ServerObjects)
	{
		ServerObject = CreateHugeObject(Server);
	}

	// Send and deliver packets until all huge objects have arrived.
	UTestReplicatedIrisObject* ClientObjects[HugeObjectCount] = {};
	uint32 ClientObjectCount = 0;
	for (uint32 RetryIt = 0; RetryIt != (HugeObjectMaxNetTickCountToArrive*HugeObjectCount) && ClientObjectCount < HugeObjectCount; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();

		for (UTestReplicatedIrisObject*& ClientObject : ClientObjects)
		{
			if (ClientObject)
			{
				continue;
			}

			const SIZE_T ObjectIndex = &ClientObject - ClientObjects;
			ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[ObjectIndex]->NetRefHandle));
			ClientObjectCount += (ClientObject != nullptr);
		}
	}

	UE_NET_ASSERT_EQ(ClientObjectCount, HugeObjectCount);
}

// Test that huge object state can be replicated after an object has been created.
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, SplitManyHugeObjectsAfterCreation)
{
	FReplicationSystemTestClient* Client = CreateClient();

	constexpr uint32 HugeObjectCount = 37U;
	UTestReplicatedIrisObject* ServerObjects[HugeObjectCount];
	for (UTestReplicatedIrisObject*& ServerObject : ServerObjects)
	{
		ServerObject = CreateObject(Server);
	}

	// Send and deliver packets until all huge objects have been received on the client.
	UTestReplicatedIrisObject* ClientObjects[HugeObjectCount] = {};
	uint32 ClientObjectCount = 0;
	for (uint32 RetryIt = 0; RetryIt != HugeObjectCount && ClientObjectCount < HugeObjectCount; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();

		for (UTestReplicatedIrisObject*& ClientObject : ClientObjects)
		{
			if (ClientObject)
			{
				continue;
			}

			const SIZE_T ObjectIndex = &ClientObject - ClientObjects;
			ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjects[ObjectIndex]->NetRefHandle));
			ClientObjectCount += (ClientObject != nullptr);
		}
	}

	UE_NET_ASSERT_EQ(ClientObjectCount, HugeObjectCount);

	// Make all objects huge.
	for (UTestReplicatedIrisObject*& ServerObject : ServerObjects)
	{
		SetObjectPayloadByteCount(ServerObject, HugeObjectPayloadByteCount);
	}

	// Clear function call status so we can easily verify we get the huge payload.
	for (UTestReplicatedIrisObject* ClientObject : ClientObjects)
	{
		UTestReplicatedIrisDynamicStatePropertyComponent* Component = ClientObject->DynamicStateComponents[0].Get();
		Component->CallCounts = {};
	}

	uint32 ClientObjectsWithHugeArraysCount = 0;
	for (uint32 RetryIt = 0; RetryIt != (HugeObjectMaxNetTickCountToArrive*HugeObjectCount) && ClientObjectsWithHugeArraysCount < HugeObjectCount; ++RetryIt)
	{
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, DeliverPacket);
		Server->PostSendUpdate();

		ClientObjectsWithHugeArraysCount = 0;
		for (UTestReplicatedIrisObject* ClientObject : ClientObjects)
		{
			UTestReplicatedIrisDynamicStatePropertyComponent* Component = ClientObject->DynamicStateComponents[0].Get();
			ClientObjectsWithHugeArraysCount += (Component->CallCounts.IntArrayRepNotifyCounter > 0);
		}
	}

	UE_NET_ASSERT_EQ(ClientObjectsWithHugeArraysCount, HugeObjectCount);
}

UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, TestDependentObjectCannotBeDestroyedWhileWaitingForCreation)
{
	UReplicatedTestObjectBridge* ServerBridge = Server->GetReplicationBridge();
	FReplicationSystemTestClient* Client = CreateClient();

	UTestReplicatedIrisObject* ServerObject = CreateHugeObject(Server);
	UTestReplicatedIrisObject* ServerDependentObject = Server->CreateObject(UObjectReplicationBridge::FRootObjectReplicationParams{});

	ServerBridge->AddDependentObject(ServerObject->NetRefHandle, ServerDependentObject->NetRefHandle);

	// Introduce latency by not immediately delivering packets.
	Server->NetUpdate();
	Server->SendTo(Client, TEXT("Create HugeObject + Dependent"));
	Server->PostSendUpdate();

	// Filter out dependent object to cause it to end up being destroyed
	Server->ReplicationSystem->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedNetObjectGroup(), ServerDependentObject->NetRefHandle);

	Server->NetUpdate();
	Server->SendTo(Client, TEXT("Try destroy Dependent"));
	Server->PostSendUpdate();

	// Make sure at least one of the packets required for object creation is lost.
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// Deliver all pending packets
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive; ++RetryIt)
	{
		Server->DeliverTo(Client, DeliverPacket);
	}

	// Make sure we replicate the full state of all objects
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive; ++RetryIt)
	{
		Server->UpdateAndSend({ Client }, DeliverPacket);
	}

	// Make sure the dependent object was destroyed.
	const UTestReplicatedIrisObject* ClientDependentObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerDependentObject->NetRefHandle));
	UE_NET_ASSERT_EQ(ClientDependentObject, nullptr);
}

UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, TestHugeObjectCanBeSentDuringReplicationRecordStarvation)
{
	IConsoleVariable* CVarReplicationRecordStarvationThreshold = IConsoleManager::Get().FindConsoleVariable(TEXT("net.Iris.ReplicationWriterReplicationRecordStarvationThreshold"));
	UE_NET_ASSERT_NE(CVarReplicationRecordStarvationThreshold, nullptr);
	UE_NET_ASSERT_TRUE(CVarReplicationRecordStarvationThreshold->IsVariableInt());
	const int32 PrevReplicationRecordStarvationThreshold = CVarReplicationRecordStarvationThreshold->GetInt();
	ON_SCOPE_EXIT
	{
		CVarReplicationRecordStarvationThreshold->Set(PrevReplicationRecordStarvationThreshold, ECVF_SetByCode);
	};
	
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Set starvation threshold to highest possible
	CVarReplicationRecordStarvationThreshold->Set(FReplicationRecord::MaxReplicationRecordCount, ECVF_SetByCode);

	// Consume a few ReplicationRecords to enter starvation
	for (int It : {0, 1, 2})
	{
		Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	}

	// Create huge object so that the ReplicationWriter will use the huge object path.
	UTestReplicatedIrisObject* ServerHugeObject = CreateHugeObject(Server);

	// Write packet
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive; ++RetryIt)
	{
		Server->NetUpdate();
		const bool bPacketWasWritten = Server->SendUpdate(Client->ConnectionIdOnServer, TEXT("Create HugeObject"));
		Server->PostSendUpdate();
		if (!bPacketWasWritten)
		{
			break;
		}
	}

	// Deliver object creation packets
	{
		const auto& ConnectionInfo = Server->GetConnectionInfo(Client->ConnectionIdOnServer);
		const SIZE_T PacketCount = ConnectionInfo.WrittenPackets.Count();
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, DeliverPacket);
		}
	}

	// The huge object should have been delivered despite the ReplicationRecord starvation
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerHugeObject->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, TestHugeObjectCreatedAndDestroyedSameFrameReplicatesIfFlushedWithPacketLoss)
{
	FReplicationSystemTestClient* Client = CreateClient();

	// Create huge object so that the ReplicationWriter will use the huge object path.
	UTestReplicatedIrisObject* ServerObject = CreateHugeObject(Server);

	FNetRefHandle ObjectHandle = ServerObject->NetRefHandle;

	// Destroy object indicating that it should be flushed that is that the final state should be replicated to all clients with the object in scope, this invalidates the creationinfo which has to be cached in order for this to work.
	Server->DestroyObject(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Send update, dropping data
	Server->UpdateAndSend({Client}, false);

	UTestReplicatedIrisObject* ClientObject = nullptr;
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive && ClientObject == nullptr; ++RetryIt)
	{
		Server->UpdateAndSend({Client});
		ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	}
	// Verify that object is created
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Deliver a packet, make sure that object is destroyed on the client.
	Server->UpdateAndSend({Client});

	// Verify that object is destroyed
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ObjectHandle), nullptr);
}

// Below test will fail as we only have a special path for reliable attachments for objects that stopped replicating, not for being filtered out.
#if 0
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, TestReliableAttachmentAddedAfterSplittingHugeObjectIsDeliveredBeforeObjectIsFilteredOut)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = CreateObject(Server);

	// Send and deliver packet
	Server->PreSendUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Force huge object state
	SetObjectPayloadByteCount(ServerObject, HugeObjectPayloadByteCount);

	// Write packets
	for (uint32 RetryIt = 0; RetryIt != HugeObjectMaxNetTickCountToArrive; ++RetryIt)
	{
		Server->PreSendUpdate();
		Server->SendUpdate(Client->ConnectionIdOnServer);
		Server->PostSendUpdate();
	}

	// Add reliable attachment
	{
		TRefCountPtr<FNetObjectAttachment> Attachment = ServerMockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(1U);

		FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
		Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);
	}

	// Filter out object to cause object to be set in state WaitOnFlush
	Server->GetReplicationSystem()->AddToGroup(Server->GetReplicationSystem()->GetNotReplicatedGroup(), ServerObject->NetRefHandle);
	Server->PreSendUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Deliver huge state
	{
		SIZE_T PacketCount = 0;
		const auto& ConnectionInfo = Server->GetConnectionInfo(Client->ConnectionIdOnServer);
		PacketCount = ConnectionInfo.WrittenPackets.Count();
		for (SIZE_T PacketIt = 0; PacketIt != PacketCount; ++PacketIt)
		{
			Server->DeliverTo(Client, DeliverPacket);
		}
	}

	// Verify the attachment made it through, despite the wish to destroy the object.
	const UMockNetObjectAttachmentHandler::FCallCounts AttachmentCallCounts = ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts();
	UE_NET_ASSERT_EQ(AttachmentCallCounts.OnNetBlobReceived, 1U);

	// The object should not exist
	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_EQ(ClientObject, nullptr);
}
#endif

// Special test for network connection loss due to bitstream error on receiving end. 
// Basically we need to craft data so that it hits the hugeobject path due to combined size of exports and attachments.
// Attachments are then dropped reducing the size of written data to fit within the hugeobject split threshold but close enough for it to not fit when including the overhead of the raw blob. 
// Prior to fix this would count the object as written even if it was just enqueued for next packet which resulted in bitstream error.
// NOTE: This test is compiled out due to being very sensitive to code changes affects bits written/config settings etc.
#if 0
UE_NET_TEST_FIXTURE(FSplitObjectTestFixture, SplitHugeObjectNoPartsInPacket)
{
	FReplicationSystemTestClient* Client = CreateClient();
	RegisterNetBlobHandlers(Client);

	// Limit packet size to setup this case
	Server->SetMaxSendPacketSize(200);

	UTestReplicatedIrisObject::FComponents Components;
	Components.DynamicStateComponentCount = 1;
	Components.ObjectReferenceComponentCount = 1;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(Components);
	
	// Add another object to send packet even if split object does not fit in packet.
	UTestReplicatedIrisObject* DummyObject = Server->CreateObject(Components);

	// Give the object we expect to be split high priority
	Server->ReplicationSystem->SetStaticPriority(ServerObject->NetRefHandle, 100.f);

	TRefCountPtr<FNetObjectAttachment> Attachment;
	Attachment = ServerMockNetObjectAttachmentHandler->CreateReliableNetObjectAttachment(82);
	FNetObjectReference AttachmentTarget = FObjectReferenceCache::MakeNetObjectReference(ServerObject->NetRefHandle);
	Server->GetReplicationSystem()->QueueNetObjectAttachment(Client->ConnectionIdOnServer, AttachmentTarget, Attachment);

	static uint32 PayloadByteCount = 112U;
	SetObjectPayloadByteCount(ServerObject, PayloadByteCount);

	// Exports to balance the size of the written attachments.
	UTestReplicatedIrisPushModelComponentWithObjectReference* Component = ServerObject->ObjectReferenceComponents[0].Get();
	Component->RawObjectPtrRef = UReplicationBridge::StaticClass();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// The object should not exist as we expect it to be split in the first packet but not fit
	{
		UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_EQ(ClientObject, nullptr);
	}

	// Bump the packet size, so that the unsplit rpc will fit.
	Server->SetMaxSendPacketSize(400);
	Server->UpdateAndSend({Client});

	// The object should now exist as we expect it to have been sent as an un-split hugeobject, but attachments should not have been delivered yet.
	{
		UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		UE_NET_ASSERT_NE(ClientObject, nullptr);

		const UMockNetObjectAttachmentHandler::FCallCounts AttachmentCallCounts = ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts();
		UE_NET_ASSERT_EQ(AttachmentCallCounts.OnNetBlobReceived, 0U);
	}

	// Send another packet to verify that we now got the attachement
	Server->UpdateAndSend({Client});

	// Verify the attachment made it through
	{
		const UMockNetObjectAttachmentHandler::FCallCounts AttachmentCallCounts = ClientMockNetObjectAttachmentHandler->GetFunctionCallCounts();
		UE_NET_ASSERT_EQ(AttachmentCallCounts.OnNetBlobReceived, 1U);
	}
}
#endif

}
