// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/ReplicationSystem/DirtyNetObjectTracker.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Tests/ReplicationSystem/ReplicationSystemTestFixture.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Iris/ReplicationSystem/Filtering/ReplicationFiltering.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectGroups.h"

namespace UE::Net::Private
{

/** Test that validates behavior when creating replicated sub objects inside PreUpdate/PreReplication */
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, CreateObjectsInsidePreUpdateTest)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server that is polled late in order to test ForceNetUpdate
	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	const uint32 PollPeriod = 100;
	const float PollFrequency = Server->ConvertPollPeriodIntoFrequency(PollPeriod);
	Params.PollFrequency = PollFrequency;
	Params.bUseClassConfigDynamicFilter = true;
	Params.bNeedsPreUpdate = true;
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(Params);
	UTestReplicatedIrisObject* ServerSubObject(nullptr);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Object should have been created on the client
	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Add a PreUpdate where we create a new subobject
	auto PreUpdateObject = [&](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge)
	{
		for (UObject* ReplicatedObject : Instances)
		{
			if (ServerObject == ReplicatedObject)
			{
				if (ServerSubObject == nullptr)
				{
					// Dirty a property
					ServerObject->IntA = 0xBB;

					// Create a subobject
					ServerSubObject = Server->CreateSubObject<UTestReplicatedIrisObject>(ServerObject->NetRefHandle);

					// Dirty this subobject
					ServerSubObject->IntA = 0xBB;

				}
			}
		}
	};
	Server->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObject);

	Server->ReplicationSystem->ForceNetUpdate(ServerObject->NetRefHandle);
	Server->UpdateAndSend({ Client });
	
	// Was the property changed in PreUpdate replicated ?
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);

	// And in the same Send was the subobject replicated ?
	UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientSubObject, nullptr);

	// The dirty property of the subobject should have replicated too.
	UE_NET_ASSERT_EQ(ClientSubObject->IntA, ServerSubObject->IntA);
}

/** Test that validates behavior when creating replicated sub objects inside PreUpdate/PreReplication when using fully pushbased objects */
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, CreateObjectsInsidePreUpdateTestIsPolledIfFullyPushbased)
{
	// Add client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server that is polled late in order to test ForceNetUpdate
	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	Params.bUseClassConfigDynamicFilter = true;
	Params.bNeedsPreUpdate = true;

	UTestReplicatedIrisPushModelObject* ServerObject = Server->CreateObjectWithParams<UTestReplicatedIrisPushModelObject>(Params);
	UTestReplicatedIrisPushModelObject* ServerSubObject(nullptr);
	
	// Send and deliver packet
	Server->UpdateAndSend({ Client });
	
	// Object should have been created on the client
	UTestReplicatedIrisPushModelObject* ClientObject = Cast<UTestReplicatedIrisPushModelObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Dirty a property
	ServerObject->SetIntA(0xBB);

	// Add a PreUpdate where we create a new subobject
	auto PreUpdateObject = [&](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge)
	{
		for (UObject* ReplicatedObject : Instances)
		{
			if (ServerObject == ReplicatedObject)
			{
				if (ServerSubObject == nullptr)
				{
					// Create a subobject
					ServerSubObject = Server->CreateSubObject<UTestReplicatedIrisPushModelObject>(ServerObject->NetRefHandle);

					// Dirty this subobject
					ServerSubObject->SetIntA(0xBB);
				}
			}
		}
	};
	Server->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObject);
	Server->UpdateAndSend({ Client });
	
	// Was the property changed in PreUpdate replicated ?
	UE_NET_ASSERT_EQ(ClientObject->GetIntA(), ServerObject->GetIntA());

	// And in the same Send was the subobject replicated ?
	UTestReplicatedIrisPushModelObject* ClientSubObject = Cast<UTestReplicatedIrisPushModelObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientSubObject, nullptr);

	// The dirty property of the subobject should have replicated too.
	UE_NET_ASSERT_EQ(ClientSubObject->GetIntA(), ServerSubObject->GetIntA());
}



}