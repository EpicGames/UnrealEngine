// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "ReplicatedTestObject.h"
#include "ReplicationSystemTestFixture.h"
#include "Iris/ReplicationSystem/RepTag.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"

namespace UE::Net::Private
{

class FTestRepTagFixture : public FReplicationSystemTestFixture
{
public:
	FTestRepTagFixture() : ReplicationProtocol(nullptr), StateDescriptorWithTag(~0U) {}

protected:
	virtual void SetUp() override
	{
		FReplicationSystemTestFixture::SetUp();

		{
			constexpr uint32 PropertyComponentCount = 1;
			constexpr uint32 IrisComponentCount = 2;
			UTestReplicatedIrisObject* TestObject = CreateObject(PropertyComponentCount, IrisComponentCount);
			const FNetRefHandle NetRefHandle = ReplicationBridge->BeginReplication(TestObject);
			ReplicationProtocol = ReplicationSystem->GetReplicationProtocol(NetRefHandle);

			StateDescriptorWithTag = PropertyComponentCount + 2U;
			RepTagToFind = ReplicationProtocol->ReplicationStateDescriptors[StateDescriptorWithTag]->MemberTagDescriptors[0].Tag;
		}
	}

	virtual void TearDown() override
	{
		RepTagToFind = GetInvalidRepTag();
		ReplicationProtocol = nullptr;
		FReplicationSystemTestFixture::TearDown();
	}

protected:
	FRepTag RepTagToFind;
	const FRepTag RepTagNotToFind = GetInvalidRepTag();

	const FReplicationProtocol* ReplicationProtocol;
	uint32 StateDescriptorWithTag;
};

UE_NET_TEST_FIXTURE(FTestRepTagFixture, HasRepTagReturnsFalseWhenLookingForNonExistingTag)
{
	UE_NET_ASSERT_FALSE(HasRepTag(ReplicationProtocol, RepTagNotToFind));
}

UE_NET_TEST_FIXTURE(FTestRepTagFixture, HasRepTagReturnsTrueWhenLookingForExistingTag)
{
	UE_NET_ASSERT_TRUE(HasRepTag(ReplicationProtocol, RepTagToFind));
}

UE_NET_TEST_FIXTURE(FTestRepTagFixture, FindRepTagInProtocolReturnsFalseWhenLookingForNonExistingTag)
{
	FRepTagFindInfo FindInfo;
	UE_NET_ASSERT_FALSE(FindRepTag(ReplicationProtocol, RepTagNotToFind, FindInfo));
}

UE_NET_TEST_FIXTURE(FTestRepTagFixture, FindRepTagInProtocolReturnsTrueWhenLookingForAnExistingTag)
{
	FRepTagFindInfo FindInfo;
	UE_NET_ASSERT_TRUE(FindRepTag(ReplicationProtocol, RepTagToFind, FindInfo));

	// Make sure the returned offset is the expected one. 
	uint32 ExpectedExternalOffset = 0;
	uint32 ExpectedInternalAbsoluteOffset = 0;

	for (const FReplicationStateDescriptor* StateDescriptor : MakeArrayView(ReplicationProtocol->ReplicationStateDescriptors, StateDescriptorWithTag))
	{
		ExpectedInternalAbsoluteOffset = Align(ExpectedInternalAbsoluteOffset, StateDescriptor->InternalAlignment);
		ExpectedInternalAbsoluteOffset += StateDescriptor->InternalSize;
	}

	const FReplicationStateDescriptor* StateDescriptor = ReplicationProtocol->ReplicationStateDescriptors[StateDescriptorWithTag];
	const FReplicationStateMemberDescriptor& MemberDescriptor = StateDescriptor->MemberDescriptors[StateDescriptor->MemberTagDescriptors[0].MemberIndex];

	ExpectedExternalOffset = MemberDescriptor.ExternalMemberOffset;
	ExpectedInternalAbsoluteOffset = Align(ExpectedInternalAbsoluteOffset, StateDescriptor->InternalAlignment) + MemberDescriptor.InternalMemberOffset;

	UE_NET_ASSERT_EQ(FindInfo.ExternalStateOffset, ExpectedExternalOffset);
	UE_NET_ASSERT_EQ(FindInfo.InternalStateAbsoluteOffset, ExpectedInternalAbsoluteOffset);
	UE_NET_ASSERT_EQ(FindInfo.StateIndex, StateDescriptorWithTag);
}

UE_NET_TEST_FIXTURE(FTestRepTagFixture, FindRepTagInReplicationStateReturnsFalseWhenLookingForNonExistingTag)
{
	FRepTagFindInfo FindInfo;
	UE_NET_ASSERT_FALSE(FindRepTag(ReplicationProtocol->ReplicationStateDescriptors[StateDescriptorWithTag], RepTagNotToFind, FindInfo));
}

UE_NET_TEST_FIXTURE(FTestRepTagFixture, FindRepTagInReplicationStateReturnsTrueWhenLookingForAnExistingTag)
{
	FRepTagFindInfo FindInfo;
	UE_NET_ASSERT_TRUE(FindRepTag(ReplicationProtocol->ReplicationStateDescriptors[StateDescriptorWithTag], RepTagToFind, FindInfo));

	// Make sure the returned offset is the expected one. We know we're searching for the first tag in the second descriptor.
	const FReplicationStateDescriptor* StateDescriptor = ReplicationProtocol->ReplicationStateDescriptors[StateDescriptorWithTag];
	const FReplicationStateMemberDescriptor& MemberDescriptor = StateDescriptor->MemberDescriptors[StateDescriptor->MemberTagDescriptors[0].MemberIndex];

	const uint32 ExpectedExternalOffset = MemberDescriptor.ExternalMemberOffset;
	const uint32 ExpectedInternalAbsoluteOffset = MemberDescriptor.InternalMemberOffset;

	UE_NET_ASSERT_EQ(FindInfo.ExternalStateOffset, ExpectedExternalOffset);
	UE_NET_ASSERT_EQ(FindInfo.InternalStateAbsoluteOffset, ExpectedInternalAbsoluteOffset);
	UE_NET_ASSERT_EQ(FindInfo.StateIndex, 0U);
}

UE_NET_TEST_FIXTURE(FTestRepTagFixture, HasRepTagReturnsTrueWhenLookingForNestedTag)
{
	UE_NET_ASSERT_TRUE(HasRepTag(ReplicationProtocol, MakeRepTag("NetTest_TestVector")));
}

}
