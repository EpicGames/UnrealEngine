// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSystemTestFixture.h"

namespace UE::Net::Private
{

class FReplicationBridgeTestFixture : public FReplicationSystemTestFixture
{
protected:
	enum : uint32
	{
		RegularStatePropertyCountInPropertyComponent = 2,
	};

	static constexpr uint32 PropertyComponentSize = 40U;
};

UE_NET_TEST_FIXTURE(FReplicationBridgeTestFixture, CreateObjectWithSingleFragment)
{
	UTestReplicatedIrisObject* TestObject = CreateObject(0, 0);

	// Start replicating the object.
	FNetRefHandle CreatedHandle = ReplicationBridge->BeginReplication(TestObject);

	// Verify that we managed to create a valid handle	
	UE_NET_ASSERT_TRUE(CreatedHandle.IsValid());
	UE_NET_ASSERT_TRUE(CreatedHandle == TestObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(CreatedHandle == ReplicationBridge->GetReplicatedRefHandle(TestObject));

	// Verify ReplicationProtocol
	const FReplicationProtocol* Protocol = ReplicationSystem->GetReplicationProtocol(TestObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(Protocol != nullptr);
	UE_NET_ASSERT_TRUE(Protocol->ProtocolIdentifier != 0);
	UE_NET_ASSERT_EQ(1u, Protocol->ReplicationStateCount);
	UE_NET_ASSERT_EQ(4u, Protocol->InternalTotalAlignment);
	UE_NET_ASSERT_EQ(20u, Protocol->InternalTotalSize);
	UE_NET_ASSERT_EQ(Align(32u + static_cast<uint32>(sizeof(FReplicationStateHeader)), 8U), Protocol->MaxExternalStateSize);
	UE_NET_ASSERT_EQ(5u, Protocol->ChangeMaskBitCount);
	UE_NET_ASSERT_EQ(EReplicationProtocolTraits::SupportsDeltaCompression, Protocol->ProtocolTraits);

	// Destroy the handle
	ReplicationBridge->EndReplication(TestObject);
}

UE_NET_TEST_FIXTURE(FReplicationBridgeTestFixture, CreateObjectWithMultipleFragments)
{
	constexpr uint32 PropertyComponentCount = 12;
	constexpr uint32 IrisComponentCount = 0;
	UTestReplicatedIrisObject* TestObject = CreateObject(PropertyComponentCount, IrisComponentCount);

	// Start replicating the object.
	FNetRefHandle CreatedHandle = ReplicationBridge->BeginReplication(TestObject);

	// Verify that we managed to create a valid handle	
	UE_NET_ASSERT_TRUE(CreatedHandle.IsValid());
	UE_NET_ASSERT_TRUE(CreatedHandle == TestObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(CreatedHandle == ReplicationBridge->GetReplicatedRefHandle(TestObject));

	// Verify ReplicationProtocol
	const FReplicationProtocol* Protocol = ReplicationSystem->GetReplicationProtocol(TestObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(Protocol != nullptr);
	UE_NET_ASSERT_EQ(25u, Protocol->ReplicationStateCount);
	UE_NET_ASSERT_EQ(8u, Protocol->InternalTotalAlignment);
	UE_NET_ASSERT_EQ(16u + PropertyComponentCount*PropertyComponentSize, Protocol->InternalTotalSize);
	UE_NET_ASSERT_EQ(Align(36u + static_cast<uint32>(sizeof(FReplicationStateHeader)), 8U), Protocol->MaxExternalStateSize);
	UE_NET_ASSERT_EQ(5U + PropertyComponentCount*RegularStatePropertyCountInPropertyComponent, Protocol->ChangeMaskBitCount);
	UE_NET_ASSERT_EQ(EReplicationProtocolTraits::SupportsDeltaCompression, Protocol->ProtocolTraits);

	// Destroy the handle
	ReplicationBridge->EndReplication(TestObject);
}

UE_NET_TEST_FIXTURE(FReplicationBridgeTestFixture, CreateObjectWithMixedFragments)
{
	constexpr uint32 PropertyComponentCount = 12;
	constexpr uint32 IrisComponentCount = 3;
	UTestReplicatedIrisObject* TestObject = CreateObject(PropertyComponentCount, IrisComponentCount);

	// Start replicating the object.
	FNetRefHandle CreatedHandle = ReplicationBridge->BeginReplication(TestObject);

	// Verify that we managed to create a valid handle	
	UE_NET_ASSERT_TRUE(CreatedHandle.IsValid());
	UE_NET_ASSERT_TRUE(CreatedHandle == TestObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(CreatedHandle == ReplicationBridge->GetReplicatedRefHandle(TestObject));

	// Verify ReplicationProtocol
	const FReplicationProtocol* Protocol = ReplicationSystem->GetReplicationProtocol(TestObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(Protocol != nullptr);
	UE_NET_ASSERT_EQ(28u, Protocol->ReplicationStateCount);
	UE_NET_ASSERT_EQ(8u, Protocol->InternalTotalAlignment);
	UE_NET_ASSERT_EQ(16u + PropertyComponentCount*PropertyComponentSize + IrisComponentCount*12U, Protocol->InternalTotalSize);
	UE_NET_ASSERT_EQ(Align(40u + static_cast<uint32>(sizeof(FReplicationStateHeader)), 8U), Protocol->MaxExternalStateSize);
	UE_NET_ASSERT_EQ(5U + PropertyComponentCount*RegularStatePropertyCountInPropertyComponent + IrisComponentCount*3U, Protocol->ChangeMaskBitCount);
	UE_NET_ASSERT_EQ(EReplicationProtocolTraits::SupportsDeltaCompression, Protocol->ProtocolTraits);

	// Destroy the handle
	ReplicationBridge->EndReplication(TestObject);
}

UE_NET_TEST_FIXTURE(FReplicationBridgeTestFixture, CreateObjectWithDynamicStateComponent)
{
	constexpr uint32 DynamicStateComponentCount = 1U;
	UTestReplicatedIrisObject* TestObject = CreateObjectWithDynamicState(0, 0, DynamicStateComponentCount);

	// Start replicating the object.
	FNetRefHandle CreatedHandle = ReplicationBridge->BeginReplication(TestObject);

	// Verify that we managed to create a valid handle	
	UE_NET_ASSERT_TRUE(CreatedHandle.IsValid());
	UE_NET_ASSERT_TRUE(CreatedHandle == TestObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(CreatedHandle == ReplicationBridge->GetReplicatedRefHandle(TestObject));

	// Verify ReplicationProtocol
	const FReplicationProtocol* Protocol = ReplicationSystem->GetReplicationProtocol(TestObject->NetRefHandle);
	UE_NET_ASSERT_NE(Protocol, nullptr);
	UE_NET_ASSERT_NE(Protocol->ProtocolIdentifier, FReplicationProtocolIdentifier(0));
	UE_NET_ASSERT_EQ(Protocol->ReplicationStateCount, 1U + DynamicStateComponentCount);
	UE_NET_ASSERT_EQ(Protocol->ProtocolTraits, (EReplicationProtocolTraits::HasDynamicState | EReplicationProtocolTraits::SupportsDeltaCompression));

	// Destroy the handle
	ReplicationBridge->EndReplication(TestObject);
}

}

namespace UE::Net
{

FTestMessage& operator<<(FTestMessage& TestMessage, const EReplicationProtocolTraits Traits)
{
	if (Traits == EReplicationProtocolTraits::None)
	{
		return TestMessage << TEXT("EReplicationProtocolTraits::None");
	}

	// Horribly slow
	uint32 Flags = uint32(Traits);
	const uint32 FlagCount = FMath::CountBits(Flags);
	if (FlagCount > 1U)
	{
		TestMessage << TEXT('(');
	}

	for (uint32 FlagIt = 0, FlagEndIt = FlagCount; FlagIt != FlagEndIt; ++FlagIt)
	{
		const uint32 Flag = Flags & uint32(-int32(Flags));
		Flags ^= Flag;
		if (FlagIt > 0)
		{
			TestMessage << TEXT(" | ");
		}

		switch (EReplicationProtocolTraits(Flag))
		{
		case EReplicationProtocolTraits::HasDynamicState:
			TestMessage << TEXT("EReplicationProtocolTraits::HasDynamicState");
			break;
		case EReplicationProtocolTraits::HasLifetimeConditionals:
			TestMessage << TEXT("EReplicationProtocolTraits::HasLifetimeConditionals");
			break;
		case EReplicationProtocolTraits::HasConditionalChangeMask:
			TestMessage << TEXT("EReplicationProtocolTraits::HasConditionalChangeMask");
			break;
		case EReplicationProtocolTraits::HasConnectionSpecificSerialization:
			TestMessage << TEXT("EReplicationProtocolTraits::HasConnectionSpecificSerialization");
			break;
		case EReplicationProtocolTraits::HasObjectReference:
			TestMessage << TEXT("EReplicationProtocolTraits::HasObjectReference");
			break;
		case EReplicationProtocolTraits::SupportsDeltaCompression:
			TestMessage << TEXT("EReplicationProtocolTraits::SupportsDeltaCompression");
			break;
		default:
			TestMessage << TEXT("EReplicationProtocolTraits::UnknownFlagWithValue_") << Flag;
			break;
		}
	}

	if (FlagCount > 1U)
	{
		TestMessage << TEXT(')');
	}

	return TestMessage;
}

}

