// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSystemServerClientTestFixture.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/Core/IrisLog.h"

namespace UE::Net
{

class FReplicationConfigTestFixture : public FNetworkAutomationTestSuiteFixture
{
protected:
	virtual void SetUp() override
	{
		// Delay the setup until the test is ready
	}

	virtual void TearDown() override
	{
		delete Server;
		delete Client;
		DataStreamUtil.TearDown();
		FNetworkAutomationTestSuiteFixture::TearDown();
	}

	void StartReplicationSystem()
	{
		FNetworkAutomationTestSuiteFixture::SetUp();

		// Fake what we normally get from config
		DataStreamUtil.SetUp();
		DataStreamUtil.AddDataStreamDefinition(TEXT("NetToken"), TEXT("/Script/IrisCore.NetTokenDataStream"));
		DataStreamUtil.AddDataStreamDefinition(TEXT("Replication"), TEXT("/Script/IrisCore.ReplicationDataStream"));
		DataStreamUtil.FixupDefinitions();

		Server = new FReplicationSystemTestServer(FReplicationSystemTestNode::DelaySetup);
		Server->Setup(true, GetName(), &OverrideServerConfig);

		Client = new FReplicationSystemTestClient(FReplicationSystemTestNode::DelaySetup);
		Client->Setup(false, GetName(), &OverrideClientConfig);

		// The client needs a connection
		Client->LocalConnectionId = Client->AddConnection();

		// Auto connect to server
		Client->ConnectionIdOnServer = Server->AddConnection();
	}

	void CreateReplicatedObjects(int32 NumObjects)
	{
		const int32 StartingIndex = ServerObjects.Num();

		ServerObjects.AddZeroed(NumObjects);
		ClientObjects.AddZeroed(NumObjects);

		for (int32 i = 0; i < NumObjects; ++i)
		{
			const int32 Index = StartingIndex + i;

			UTestReplicatedIrisObject* ServerObject = Server->CreateObject<UTestReplicatedIrisObject>();
			const FNetRefHandle ServerHandle = ServerObject->NetRefHandle;
			UE_NET_ASSERT_TRUE(ServerHandle.IsValid());

			ServerObjects[Index] = ServerObject;
		}

		bool bAllObjectsCreated = false;

		constexpr uint32 MaxTickCount = 100;

		for (uint32 SendCount = 0; SendCount < MaxTickCount; ++SendCount)
		{
			// Send and deliver packet
			Server->UpdateAndSend({ Client });

			bAllObjectsCreated = true;
			for (int32 i = 0; i < NumObjects; ++i)
			{
				const int32 Index = StartingIndex + i;
				if (ClientObjects[Index] == nullptr)
				{
					const FNetRefHandle ServerHandle = ServerObjects[Index]->NetRefHandle;
					UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerHandle));
					ClientObjects[Index] = ClientObject;

					bAllObjectsCreated &= (ClientObject != nullptr);
				}
			}

			if (bAllObjectsCreated)
			{
				break;
			}
		}

		UE_NET_ASSERT_TRUE(bAllObjectsCreated);
	}

	TArray<UTestReplicatedIrisObject*> ServerObjects;
	TArray<UTestReplicatedIrisObject*> ClientObjects;

	FDataStreamTestUtil DataStreamUtil;
	FReplicationSystemTestServer* Server = nullptr;
	FReplicationSystemTestClient* Client = nullptr;

	FReplicationSystemTestNode::FReplicationSystemParamsOverride OverrideServerConfig;
	FReplicationSystemTestNode::FReplicationSystemParamsOverride OverrideClientConfig;
};

} // end namespace UE::Net

namespace UE::Net::Private
{
UE_NET_TEST_FIXTURE(FReplicationConfigTestFixture, TestNetObjectListGrowEvent)
{
	const uint32 MaxNumObjects = 96;
	const uint32 InitNumObjects = 32;
	const uint32 GrowCount = 32;

	OverrideServerConfig.MaxReplicatedObjectCount = MaxNumObjects;
	OverrideServerConfig.InitialNetObjectListCount = InitNumObjects;
	OverrideServerConfig.NetObjectListGrowCount = GrowCount;

	OverrideClientConfig.MaxReplicatedObjectCount = MaxNumObjects;
	OverrideClientConfig.InitialNetObjectListCount = InitNumObjects;
	OverrideClientConfig.NetObjectListGrowCount = GrowCount;

	StartReplicationSystem();

	FDelegateHandle ServerDelegate;
	FDelegateHandle ClientDelegate;

	FNetRefHandleManager* ServerNetRefHandleManager = &Server->ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
	FNetRefHandleManager* ClientNetRefHandleManager = &Client->ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();

	ON_SCOPE_EXIT
	{
		ServerNetRefHandleManager->GetOnMaxInternalNetRefIndexIncreasedDelegate().Remove(ServerDelegate);
		ClientNetRefHandleManager->GetOnMaxInternalNetRefIndexIncreasedDelegate().Remove(ClientDelegate);
	};


	bool bHasServerRealloc = false;
	ServerDelegate = ServerNetRefHandleManager->GetOnMaxInternalNetRefIndexIncreasedDelegate().AddLambda([&bHasServerRealloc](uint32 NewMaxIndex)
	{
		bHasServerRealloc = true;
	});

	bool bHasClientRealloc = false;
	ClientDelegate = ClientNetRefHandleManager->GetOnMaxInternalNetRefIndexIncreasedDelegate().AddLambda([&bHasClientRealloc](uint32 NewMaxIndex)
	{
		bHasClientRealloc = true;
	});

	// Create enough objects to fill the init list (entry[0] is already reserved for invalid object)
	CreateReplicatedObjects(InitNumObjects - 1);
	UE_NET_ASSERT_EQ(bHasServerRealloc, false);
	UE_NET_ASSERT_EQ(bHasClientRealloc, false);

	// Create one more to cause a realloc to trigger
	CreateReplicatedObjects(1);
	UE_NET_ASSERT_EQ(bHasServerRealloc, true);
	UE_NET_ASSERT_EQ(bHasClientRealloc, true);

	// Reset the test conditions
	bHasServerRealloc = false;
	bHasClientRealloc = false;

	// Maximimize the objects allowed to exist without reallocating
	CreateReplicatedObjects(GrowCount-1);
	UE_NET_ASSERT_EQ(bHasServerRealloc, false);
	UE_NET_ASSERT_EQ(bHasClientRealloc, false);

	// Create one more to cause a realloc to trigger
	CreateReplicatedObjects(1);
	UE_NET_ASSERT_EQ(bHasServerRealloc, true);
	UE_NET_ASSERT_EQ(bHasClientRealloc, true);

	// Reset the test conditions
	bHasServerRealloc = false;
	bHasClientRealloc = false;
	
	// Create the maximum amount of held objects
	CreateReplicatedObjects(MaxNumObjects - ServerObjects.Num() - 1); //Remove 1 to account for invalid entry[0]
	UE_NET_ASSERT_EQ(bHasServerRealloc, false);
	UE_NET_ASSERT_EQ(bHasClientRealloc, false);

	// Adding 1 more should cause a Fatal error
	//CreateReplicatedObjects(1);
}


UE_NET_TEST_FIXTURE(FReplicationConfigTestFixture, TestNetObjectListGrow_DuringPollPhase)
{
	const uint32 MaxNumObjects = 96;
	const uint32 InitNumObjects = 32;
	const uint32 GrowCount = 32;

	OverrideServerConfig.MaxReplicatedObjectCount = MaxNumObjects;
	OverrideServerConfig.InitialNetObjectListCount = InitNumObjects;
	OverrideServerConfig.NetObjectListGrowCount = GrowCount;

	StartReplicationSystem();

	FDelegateHandle ServerDelegate;

	FNetRefHandleManager* ServerNetRefHandleManager = &Server->ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
	ON_SCOPE_EXIT
	{
		ServerNetRefHandleManager->GetOnMaxInternalNetRefIndexIncreasedDelegate().Remove(ServerDelegate);
	};


	bool bHasServerRealloc = false;
	ServerDelegate = ServerNetRefHandleManager->GetOnMaxInternalNetRefIndexIncreasedDelegate().AddLambda([&bHasServerRealloc](uint32 NewMaxIndex)
	{
		bHasServerRealloc = true;
	});

	// Create enough objects to fill the init list (entry[0] is already reserved for invalid object)
	for (uint32 i = 0; i < (InitNumObjects - 1); ++i)
	{
		UObjectReplicationBridge::FRootObjectReplicationParams Params;
		Params.bNeedsPreUpdate = true;
		ServerObjects.Emplace(Server->CreateObject(Params));
	}

	// Send and deliver packet
	Server->UpdateAndSend({ Client });
	
	UTestReplicatedIrisObject* ServerObject = ServerObjects[0];
	UE_NET_ASSERT_NE(ServerObject, nullptr);

	UTestReplicatedIrisObject* ServerSubObject(nullptr);

	// Add a PreUpdate callback where we create a new subobject
	auto PreUpdateObject = [&](TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge)
	{
		for (UObject* ReplicatedObject : Instances)
		{
			if (ServerObject == ReplicatedObject)
			{
				if (ServerSubObject == nullptr)
				{
					// Create a subobject
					ServerSubObject = Server->CreateSubObject<UTestReplicatedIrisObject>(ServerObject->NetRefHandle);
				}
			}
		}
	};
	Server->GetReplicationBridge()->SetExternalPreUpdateFunctor(PreUpdateObject);

	Server->UpdateAndSend( { Client });

	// Check that the server realloc happened.
	UE_NET_ASSERT_EQ(bHasServerRealloc, true);

	// Make sure the subobject was spawned on the client
	UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientSubObject, nullptr);
}


} // end namespace UE::Net::Private

