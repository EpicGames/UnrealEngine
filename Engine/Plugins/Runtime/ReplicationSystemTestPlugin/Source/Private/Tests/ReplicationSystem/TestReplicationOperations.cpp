// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSystemServerClientTestFixture.h"
#include "ReplicationSystemTestFixture.h"

#include "Iris/ReplicationSystem/ChangeMaskUtil.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationSystem/ReplicationTypes.h"
#include "Misc/MemStack.h"
#include "Misc/ScopeExit.h"

namespace UE::Net::Private
{

struct FTestReplicationOperationsFixture : public FReplicationSystemTestFixture
{
	FTestReplicationOperationsFixture() : FReplicationSystemTestFixture()
	, TempLinearAllocator(nullptr)
	{
	}

	template<typename T>
	void VerifyAs(const T& ExpectedValue, const FPropertyReplicationState& State, uint32 StateIndex)
	{
		T TempValue;
		State.GetPropertyValue(StateIndex, &TempValue);
		UE_NET_ASSERT_EQ(ExpectedValue, TempValue);
	}

	virtual void SetUp() override
	{
		FReplicationSystemTestFixture::SetUp();
		TempLinearAllocator = new FMemStackBase();
	}

	virtual void TearDown() override
	{
		delete TempLinearAllocator;
		FReplicationSystemTestFixture::TearDown();
	}

	FMemStackBase& GetTempAllocator() { return *TempLinearAllocator; }

	alignas(16) uint8 ChangeMaskBuffer[32];
	FNetBitStreamWriter ChangeMaskWriter;
	FMemStackBase* TempLinearAllocator;

	ChangeMaskStorageType* GetChangeMaskStorage() { return reinterpret_cast<ChangeMaskStorageType*>(ChangeMaskBuffer); }

	FNetBitStreamWriter* GetChangeMaskWriter(uint32 BitCount)
	{ 
		check((BitCount >> 3) < sizeof(ChangeMaskBuffer));
		FMemory::Memset(ChangeMaskBuffer, 0);
		ChangeMaskWriter.InitBytes(ChangeMaskBuffer, sizeof(ChangeMaskBuffer));

		return &ChangeMaskWriter;
	}
};

UE_NET_TEST_FIXTURE(FTestReplicationOperationsFixture, CanPollAndCopyPropertyData)
{
	constexpr uint32 PropertyComponentCount = 5U;
	constexpr uint32 IrisComponentCount = 3U;
	constexpr uint32 DynamicStateComponentCount = 3U;

	UTestReplicatedIrisObject* TestObject = CreateObjectWithDynamicState(PropertyComponentCount, IrisComponentCount, DynamicStateComponentCount);

	// Create NetRefHandle for the CreatedHandle,
	FNetRefHandle CreatedHandle = ReplicationBridge->BeginReplication(TestObject);

	auto&& GetPropertyReplicationStateChecked = [](const FReplicationFragment* Fragment)
	{ 
		check(Fragment && EnumHasAnyFlags(Fragment->GetTraits(), EReplicationFragmentTraits::HasPropertyReplicationState));
		return static_cast<const FPropertyReplicationFragment*>(Fragment)->GetPropertyReplicationState();
	};

	// Src state for root
	const FPropertyReplicationState& State = *GetPropertyReplicationStateChecked(TestObject->ReplicationFragments[0]);
	const FPropertyReplicationState& ComponentState = *GetPropertyReplicationStateChecked(TestObject->Components[0]->ReplicationFragments[1]);

	// Verify that we got default values in ReplicationStates
	VerifyAs<int32>(0, State, 0);
	VerifyAs<int32>(0, State, 1);
	VerifyAs<int8>(0, State, 2);

	// Verify components
	VerifyAs<int32>(0, ComponentState, 0);

	// Verify iris native component
	UE_NET_ASSERT_EQ(0, TestObject->IrisComponents[0]->ReplicationState.GetIntA());
	UE_NET_ASSERT_EQ(0, TestObject->IrisComponents[0]->ReplicationState.GetIntB());
	UE_NET_ASSERT_EQ(0, TestObject->IrisComponents[0]->ReplicationState.GetIntC());

	// Verify dynamic state component
	constexpr uint32 DynamicStateComponentIndex = DynamicStateComponentCount/2;
	UE_NET_ASSERT_EQ(TestObject->DynamicStateComponents[DynamicStateComponentIndex]->IntArray.Num(), 0);
	UE_NET_ASSERT_EQ(TestObject->DynamicStateComponents[DynamicStateComponentIndex]->IntStaticArray[6], int8(0));

	// Set some values in the root and in each type of component
	TestObject->IntA = 1;
	TestObject->Components[0]->IntA = 2;
	TestObject->IrisComponents[0]->ReplicationState.SetIntA(3);
	TestObject->DynamicStateComponents[DynamicStateComponentIndex]->IntArray.SetNum(1);
	TestObject->DynamicStateComponents[DynamicStateComponentIndex]->IntArray[0] = 4711;
	TestObject->DynamicStateComponents[DynamicStateComponentIndex]->IntStaticArray[5] = 43;

	// Poll data to update property replication state
	FReplicationInstanceOperations::PollAndCopyPropertyData(ReplicationBridge->GetReplicationInstanceProtocol(TestObject->NetRefHandle));

	// Verify values after poll
	VerifyAs<int32>(1, State, 0);
	VerifyAs<int32>(0, State, 1);
	VerifyAs<int8>(0, State, 2);

	// Verify components
	VerifyAs<int32>(2, ComponentState, 0);

	UE_NET_ASSERT_EQ(3, TestObject->IrisComponents[0]->ReplicationState.GetIntA());

	UE_NET_ASSERT_EQ(TestObject->DynamicStateComponents[DynamicStateComponentIndex]->IntArray.Num(), 1);
	UE_NET_ASSERT_EQ(TestObject->DynamicStateComponents[DynamicStateComponentIndex]->IntArray[0], 4711);
	UE_NET_ASSERT_EQ(TestObject->DynamicStateComponents[DynamicStateComponentIndex]->IntStaticArray[5], int8(43));

	// Destroy the handle
	ReplicationBridge->EndReplication(TestObject);
}

UE_NET_TEST_FIXTURE(FTestReplicationOperationsFixture, CanQuantizeAndDequantize)
{
	constexpr uint32 PropertyComponentCount = 12U;
	constexpr uint32 IrisComponentCount = 3U;
	constexpr uint32 DynamicStateComponentCount = 3U;

	UTestReplicatedIrisObject* TestObject = CreateObjectWithDynamicState(PropertyComponentCount, IrisComponentCount, DynamicStateComponentCount);
	
	// Create NetRefHandle for the CreatedHandle,
	FNetRefHandle CreatedHandle = ReplicationBridge->BeginReplication(TestObject);
	const FReplicationProtocol* ProtocolA = ReplicationSystem->GetReplicationProtocol(CreatedHandle);

	// Set some values
	TestObject->IntA = 1;
	TestObject->IntC = 2;
	TestObject->StructD.NotReplicatedIntA = 'A';
	TestObject->StructD.IntB = 'B';
	TestObject->Components[0]->IntA = 3;
	TestObject->Components[11]->IntA = 4;
	TestObject->IrisComponents[0]->ReplicationState.SetIntA(5);
	TestObject->IrisComponents[2]->ReplicationState.SetIntA(6);
	TestObject->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray.SetNumUninitialized(3);
	TestObject->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray[0] = 1;
	TestObject->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray[1] = 2;
	TestObject->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray[2] = 3;
	TestObject->DynamicStateComponents[DynamicStateComponentCount - 1]->IntStaticArray[4] = 127;

	// Poll data to update property replication state
	FReplicationInstanceOperations::PollAndCopyPropertyData(ReplicationBridge->GetReplicationInstanceProtocol(TestObject->NetRefHandle));

	// Quantize to buffer
	const uint32 BufferSize = 1024;
	const uint8 ValueToSet = 0xCD;

	// Need to be able to handle worst case alignment
	UE_NET_ASSERT_GE(BufferSize, ProtocolA->InternalTotalSize + ProtocolA->InternalTotalAlignment);

	uint8 Buffer[BufferSize];	
	FMemory::Memset(Buffer, ValueToSet, BufferSize);
	uint8* AlignedBuffer = Align(Buffer, ProtocolA->InternalTotalAlignment);
	// The part of the buffer that will hold the internal state needs to be cleared due to the presence of dynamic state
	FMemory::Memzero(AlignedBuffer, ProtocolA->InternalTotalSize);

	FNetSerializationContext SerializationContext;
	FInternalNetSerializationContext InternalContext;
	SerializationContext.SetInternalContext(&InternalContext);
	SerializationContext.SetIsInitState(true);

	// Copy data from test object to buffer
	FReplicationInstanceOperations::Quantize(SerializationContext, AlignedBuffer, GetChangeMaskWriter(ProtocolA->ChangeMaskBitCount), ReplicationBridge->GetReplicationInstanceProtocol(TestObject->NetRefHandle), ProtocolA);

	// Verify that we did not overshoot
	UE_NET_ASSERT_EQ(ValueToSet, AlignedBuffer[ProtocolA->InternalTotalSize]);

	// Create second object 
	UTestReplicatedIrisObject* TestObjectB = CreateObjectWithDynamicState(PropertyComponentCount, IrisComponentCount, DynamicStateComponentCount);
	ReplicationBridge->BeginReplication(TestObjectB);
	const FReplicationProtocol* ProtocolB = ReplicationSystem->GetReplicationProtocol(TestObjectB->NetRefHandle);
	UE_NET_ASSERT_EQ(ProtocolA, ProtocolB);

	// Push state data to ObjectB
	FReplicationInstanceOperations::DequantizeAndApply(SerializationContext, GetTempAllocator(), GetChangeMaskStorage(), ReplicationBridge->GetReplicationInstanceProtocol(TestObjectB->NetRefHandle), AlignedBuffer, ProtocolB);

	// check if we did get the expected values
	UE_NET_ASSERT_EQ(1, TestObjectB->IntA);
	UE_NET_ASSERT_EQ((int8)2, TestObjectB->IntC);
	UE_NET_ASSERT_EQ_MSG(0, TestObjectB->StructD.NotReplicatedIntA, "Property that isn't replicated has been overwritten");
	UE_NET_ASSERT_EQ(int32('B'), TestObjectB->StructD.IntB);
	UE_NET_ASSERT_EQ(3, TestObjectB->Components[0]->IntA);
	UE_NET_ASSERT_EQ(4, TestObjectB->Components[11]->IntA);
	
	FTestReplicatedIrisComponent* Comp0 = TestObjectB->IrisComponents[0].Get();
	FTestReplicatedIrisComponent* Comp2 = TestObjectB->IrisComponents[2].Get();

	UE_NET_ASSERT_EQ(5, Comp0->ReplicationState.GetIntA());
	UE_NET_ASSERT_EQ(6, Comp2->ReplicationState.GetIntA());

	for (uint32 DynamicComponentIt = 0; DynamicComponentIt < DynamicStateComponentCount - 1; ++DynamicComponentIt)
	{
		UE_NET_ASSERT_EQ(TestObjectB->DynamicStateComponents[DynamicComponentIt]->IntArray.Num(), 0);
	}

	UE_NET_ASSERT_EQ(TestObjectB->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray.Num(), 3);

	UE_NET_ASSERT_EQ(TestObjectB->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray[0], 1);
	UE_NET_ASSERT_EQ(TestObjectB->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray[1], 2);
	UE_NET_ASSERT_EQ(TestObjectB->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray[2], 3);

	UE_NET_ASSERT_EQ(TestObjectB->DynamicStateComponents[DynamicStateComponentCount - 1]->IntStaticArray[4], int8(127));

	// Free dynamic state
	FReplicationProtocolOperations::FreeDynamicState(SerializationContext, AlignedBuffer, ProtocolA);

	// Destroy the handle
	ReplicationBridge->EndReplication(TestObject);
}

UE_NET_TEST_FIXTURE(FTestReplicationOperationsFixture, CanSerializeAndDeserialize)
{
	constexpr uint32 PropertyComponentCount = 12U;
	constexpr uint32 IrisComponentCount = 3U;
	constexpr uint32 DynamicStateComponentCount = 3U;

	UTestReplicatedIrisObject* TestObject = CreateObjectWithDynamicState(PropertyComponentCount, IrisComponentCount, DynamicStateComponentCount);
	
	// Create NetRefHandle for the CreatedHandle,
	FNetRefHandle CreatedHandle = ReplicationBridge->BeginReplication(TestObject);
	const FReplicationProtocol* ProtocolA = ReplicationSystem->GetReplicationProtocol(CreatedHandle);

	// Set some values
	TestObject->IntA = 1;
	TestObject->IntC = 2;
	TestObject->Components[0]->IntA = 3;
	TestObject->Components[11]->IntA = 4;
	TestObject->IrisComponents[0]->ReplicationState.SetIntA(5);
	TestObject->IrisComponents[2]->ReplicationState.SetIntA(6);
	TestObject->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray.SetNumUninitialized(3);
	TestObject->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray[0] = 1;
	TestObject->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray[1] = 2;
	TestObject->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray[2] = 3;
	TestObject->DynamicStateComponents[DynamicStateComponentCount - 1]->IntStaticArray[4] = 127;

	// Poll data to update property replication state
	FReplicationInstanceOperations::PollAndCopyPropertyData(ReplicationBridge->GetReplicationInstanceProtocol(TestObject->NetRefHandle));

	// Quantize to buffer
	const uint32 BufferSize = 1024;
	const uint8 ValueToSet = 0xFE;

	UE_NET_ASSERT_GT(BufferSize, ProtocolA->InternalTotalSize);

	uint8 Buffer[2][BufferSize];	
	FMemory::Memset(Buffer, ValueToSet, 2*BufferSize);
	uint8* AlignedBuffer = Align(Buffer[0], ProtocolA->InternalTotalAlignment);
	uint8* AlignedBufferB = Align(Buffer[1], ProtocolA->InternalTotalAlignment);
	FMemory::Memzero(AlignedBuffer, ProtocolA->InternalTotalSize);
	FMemory::Memzero(AlignedBufferB, ProtocolA->InternalTotalSize);

	// Copy data from test object to buffer
	FNetSerializationContext SerializationContext;
	FInternalNetSerializationContext InternalContext;
	SerializationContext.SetInternalContext(&InternalContext);
	SerializationContext.SetIsInitState(true);
	FReplicationInstanceOperations::Quantize(SerializationContext, AlignedBuffer, GetChangeMaskWriter(ProtocolA->ChangeMaskBitCount), ReplicationBridge->GetReplicationInstanceProtocol(TestObject->NetRefHandle), ProtocolA);

	// Verify that we did not overshoot
	UE_NET_ASSERT_EQ(ValueToSet, AlignedBuffer[ProtocolA->InternalTotalSize]);

	// Create second object 
	UTestReplicatedIrisObject* TestObjectB = CreateObjectWithDynamicState(PropertyComponentCount, IrisComponentCount, DynamicStateComponentCount);
	ReplicationBridge->BeginReplication(TestObjectB);
	const FReplicationProtocol* ProtocolB = ReplicationSystem->GetReplicationProtocol(TestObjectB->NetRefHandle);
	UE_NET_ASSERT_EQ(ProtocolA, ProtocolB);


	// Serialize state data
	uint8 BitStreamBuffer[BufferSize];

	FNetBitStreamWriter Writer;
	{
		
		Writer.InitBytes(BitStreamBuffer, BufferSize);
	
		FNetSerializationContext Context(&Writer);
		Context.SetInternalContext(&InternalContext);
		Context.SetIsInitState(true);

		FReplicationProtocolOperations::SerializeWithMask(Context, GetChangeMaskStorage(), AlignedBuffer, ProtocolA);

		Writer.CommitWrites();

		FReplicationProtocolOperations::FreeDynamicState(SerializationContext, AlignedBuffer, ProtocolA);
	}

	// Deserialize state data
	{
		FNetBitStreamReader Reader;
		Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());

		FNetSerializationContext Context(&Reader);
		Context.SetInternalContext(&InternalContext);
		Context.SetIsInitState(true);

		// Reset changemask buffer
		FMemory::Memset(ChangeMaskBuffer, 0u);

		// Read data into AlignedBuffer
		FReplicationProtocolOperations::DeserializeWithMask(Context, GetChangeMaskStorage(), AlignedBufferB, ProtocolB);

		// Push deserialized state data to ObjectB 
		FReplicationInstanceOperations::DequantizeAndApply(Context, GetTempAllocator(), GetChangeMaskStorage(), ReplicationBridge->GetReplicationInstanceProtocol(TestObjectB->NetRefHandle), AlignedBufferB, ProtocolB);
	}

	// check if we did get the expected values
	UE_NET_ASSERT_EQ(1, TestObjectB->IntA);
	UE_NET_ASSERT_EQ((int8)2, TestObjectB->IntC);
	UE_NET_ASSERT_EQ(3, TestObjectB->Components[0]->IntA);
	UE_NET_ASSERT_EQ(4, TestObjectB->Components[11]->IntA);
	
	FTestReplicatedIrisComponent* Comp0 = TestObjectB->IrisComponents[0].Get();
	FTestReplicatedIrisComponent* Comp2 = TestObjectB->IrisComponents[2].Get();

	UE_NET_ASSERT_EQ(5, TestObjectB->IrisComponents[0]->ReplicationState.GetIntA());
	UE_NET_ASSERT_EQ(6, TestObjectB->IrisComponents[2]->ReplicationState.GetIntA());

	UE_NET_ASSERT_EQ(TestObjectB->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray.Num(), 3);
	UE_NET_ASSERT_EQ(TestObjectB->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray[0], 1);
	UE_NET_ASSERT_EQ(TestObjectB->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray[1], 2);
	UE_NET_ASSERT_EQ(TestObjectB->DynamicStateComponents[DynamicStateComponentCount - 1]->IntArray[2], 3);

	UE_NET_ASSERT_EQ(TestObjectB->DynamicStateComponents[DynamicStateComponentCount - 1]->IntStaticArray[4], int8(127));

	FReplicationProtocolOperations::FreeDynamicState(SerializationContext, AlignedBuffer, ProtocolA);
	FReplicationProtocolOperations::FreeDynamicState(SerializationContext, AlignedBufferB, ProtocolA);

	// Destroy the handle
	ReplicationBridge->EndReplication(TestObject);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, PreAndPostNetReceivedOnlyCalledOnce)
{
	// Create replicated object 
	UTestReplicatedIrisObject* ServerTestObject = Server->CreateObject(1, 0);
	Server->ReplicationBridge->BeginReplication(ServerTestObject);

	// Set some values
	ServerTestObject->Components[0]->IntA = 3;
	ServerTestObject->Components[0]->IntB = 3;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn the replicated instance
	Server->UpdateAndSend({ Client });

	// Find the client instance and make sure it replicated
	UTestReplicatedIrisObject* ClientTestObject = Cast<UTestReplicatedIrisObject>(Client->ReplicationBridge->GetReplicatedObject(ServerTestObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientTestObject, nullptr);
	UE_NET_ASSERT_EQ(3, ClientTestObject->Components[0]->IntA);
	UE_NET_ASSERT_EQ(3, ClientTestObject->Components[0]->IntB);
	
	// Check NetReceive callbacks
	UE_NET_ASSERT_EQ(1U, ClientTestObject->Components[0]->CallCounts.PreNetReceiveCounter);
	UE_NET_ASSERT_EQ(1U, ClientTestObject->Components[0]->CallCounts.PostNetReceiveCounter);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, RepNotifyIsCalledForSimpleProperty)
{
	// Create replicated object 
	UTestReplicatedIrisObject* ServerTestObject = Server->CreateObject(1, 0);
	Server->ReplicationBridge->BeginReplication(ServerTestObject);

	// Set some values
	ServerTestObject->Components[0]->IntB = 3;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn the replicated instance
	Server->UpdateAndSend({ Client });

	// Find the client instance and make sure it replicated
	UTestReplicatedIrisObject* ClientTestObject = Cast<UTestReplicatedIrisObject>(Client->ReplicationBridge->GetReplicatedObject(ServerTestObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientTestObject, nullptr);
	UE_NET_ASSERT_EQ(3, ClientTestObject->Components[0]->IntB);

	// Check that we got exactly one RepNotify call
	UE_NET_ASSERT_EQ(ClientTestObject->Components[0]->CallCounts.IntBRepNotifyCounter, 1U);
}

UE_NET_TEST_FIXTURE(FTestReplicationOperationsFixture, InitPropertyIsNotUpdatedWhenInitStateIsExcluded)
{
	constexpr uint32 PropertyComponentCount = 1;
	UTestReplicatedIrisObject* TestObject = CreateObject(PropertyComponentCount, 0);
	
	// Create NetRefHandle for the CreatedHandle,
	FNetRefHandle CreatedHandle = ReplicationBridge->BeginReplication(TestObject);
	const FReplicationProtocol* Protocol = ReplicationSystem->GetReplicationProtocol(CreatedHandle);

	// Set some values
	TestObject->Components[0]->IntB ^= 1;

	// Poll data to update property replication state
	FReplicationInstanceOperations::PollAndCopyPropertyData(ReplicationBridge->GetReplicationInstanceProtocol(TestObject->NetRefHandle));

	// Quantize to buffer
	uint8* AlignedBuffer = (uint8*)(GetTempAllocator()).Alloc(Protocol->InternalTotalSize, Protocol->InternalTotalAlignment);

	// Copy data from test object to buffer
	FNetSerializationContext NetContext;
	FInternalNetSerializationContext InternalContext;
	NetContext.SetInternalContext(&InternalContext);
	NetContext.SetIsInitState(false);
	FReplicationInstanceOperations::Quantize(NetContext, AlignedBuffer, GetChangeMaskWriter(Protocol->ChangeMaskBitCount), ReplicationBridge->GetReplicationInstanceProtocol(TestObject->NetRefHandle), Protocol);

	// Create second object 
	UTestReplicatedIrisObject* TestObjectB = CreateObject(PropertyComponentCount, 0);
	ReplicationBridge->BeginReplication(TestObjectB);

	// Check that we've not yet gotten any RepNotify calls
	UE_NET_ASSERT_EQ(TestObjectB->Components[0]->CallCounts.IntBRepNotifyCounter, 0U);

	// Push state data to ObjectB
	FReplicationInstanceOperations::DequantizeAndApply(NetContext, GetTempAllocator(), GetChangeMaskStorage(), ReplicationBridge->GetReplicationInstanceProtocol(TestObjectB->NetRefHandle), AlignedBuffer, Protocol);

	// Check that the IntB property has not been modified
	UE_NET_ASSERT_EQ(TestObjectB->Components[0]->IntB, 0);

	// Check that we did not get a RepNotify call for the init property.
	UE_NET_ASSERT_EQ(TestObjectB->Components[0]->CallCounts.IntBRepNotifyCounter, 0U);

	ReplicationBridge->EndReplication(TestObject);
	ReplicationBridge->EndReplication(TestObjectB);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, RepNotifyIsCalledExactlyOnceForArrays)
{
	// Create replicated object 
	UTestReplicatedIrisObject* ServerTestObject = Server->CreateObjectWithDynamicState(0, 0, 1);
	Server->ReplicationBridge->BeginReplication(ServerTestObject);

	// Set some values
	ServerTestObject->DynamicStateComponents[0]->IntArray.SetNum(4);
	ServerTestObject->DynamicStateComponents[0]->IntArray[0] = 1;
	ServerTestObject->DynamicStateComponents[0]->IntArray[1] = 1;
	ServerTestObject->DynamicStateComponents[0]->IntArray[2] = 7;
	ServerTestObject->DynamicStateComponents[0]->IntArray[3] = 4;

	ServerTestObject->DynamicStateComponents[0]->IntStaticArray[1] = 5;
	ServerTestObject->DynamicStateComponents[0]->IntStaticArray[5] = 1;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn the replicated instance
	Server->UpdateAndSend({ Client });

	// Find the client instance and make sure it replicated
	UTestReplicatedIrisObject* ClientTestObject = Cast<UTestReplicatedIrisObject>(Client->ReplicationBridge->GetReplicatedObject(ServerTestObject->NetRefHandle));
	UE_NET_ASSERT_NE(ClientTestObject, nullptr);
	UE_NET_ASSERT_EQ(4, ClientTestObject->DynamicStateComponents[0]->IntArray.Num());

	// Check that we got exactly one RepNotify call per array
	UE_NET_ASSERT_EQ(ClientTestObject->DynamicStateComponents[0]->CallCounts.IntArrayRepNotifyCounter, 1U);
	UE_NET_ASSERT_EQ(ClientTestObject->DynamicStateComponents[0]->CallCounts.IntStaticArrayRepNotifyCounter, 1U);
}

class FTestReplicationOperationsForObjectsFixture : public FTestReplicationOperationsFixture
{
protected:
	virtual void SetUp() override
	{
		CvarIrisPushModel = IConsoleManager::Get().FindConsoleVariable(TEXT("net.Iris.PushModelMode"));
		check(CvarIrisPushModel != nullptr && CvarIrisPushModel->IsVariableInt());
		PushModelMode = CvarIrisPushModel->GetInt();
		CvarIrisPushModel->Set(1, ECVF_SetByCode);

		FTestReplicationOperationsFixture::SetUp();
	}

	virtual void TearDown() override
	{
		FTestReplicationOperationsFixture::TearDown();

		CvarIrisPushModel->Set(PushModelMode, ECVF_SetByCode);
	}

private:
	IConsoleVariable* CvarIrisPushModel = nullptr;
	int PushModelMode = 0;
};

UE_NET_TEST_FIXTURE(FTestReplicationOperationsForObjectsFixture, StaleObjectPointerIsUpdated)
{
	const bool bIsGarbageEliminationEnabled = UObjectBaseUtility::IsGarbageEliminationEnabled();
	UObjectBaseUtility::ForceSetGarbageEliminationEnabledForTest(true);
	ON_SCOPE_EXIT
	{
		UObjectBaseUtility::ForceSetGarbageEliminationEnabledForTest(bIsGarbageEliminationEnabled);
	};

	// Create two objects, one which is referencing the other. Verify reference gets updated when first object is destroyed.
	UTestReplicatedIrisObject* Object0 = CreateObject();

	UTestReplicatedIrisObject* Object1 = CreateObject();
	Object1->AddComponents({.ObjectReferenceComponentCount = 1});
	Object1->ObjectReferenceComponents[0]->RawObjectPtrRef = Object0;
	FNetRefHandle HandleToObject1 = ReplicationBridge->BeginReplication(Object1);

	const FReplicationInstanceProtocol* InstanceProtocol1 = ReplicationBridge->GetReplicationInstanceProtocol(HandleToObject1);
	const FReplicationProtocol* Protocol1 = ReplicationSystem->GetReplicationProtocol(HandleToObject1);

	// Quantize to buffer
	uint8* StateBuffers1[] = 
	{
		(uint8*)(GetTempAllocator()).Alloc(Protocol1->InternalTotalSize, Protocol1->InternalTotalAlignment),
		(uint8*)(GetTempAllocator()).Alloc(Protocol1->InternalTotalSize, Protocol1->InternalTotalAlignment),
	};
	for (uint8* StateBuffer : StateBuffers1)
	{
		FMemory::Memzero(StateBuffer, Protocol1->InternalTotalSize);
	}

	FNetSerializationContext SerializationContext;
	FInternalNetSerializationContext InternalContext(ReplicationSystem);
	SerializationContext.SetInternalContext(&InternalContext);
	SerializationContext.SetIsInitState(true);

	// Copy data from test object to buffer
	ReplicationSystem->NetUpdate(0.033f);
	FReplicationInstanceOperations::Quantize(SerializationContext, StateBuffers1[0], GetChangeMaskWriter(Protocol1->ChangeMaskBitCount), InstanceProtocol1, Protocol1);
	ReplicationSystem->PostSendUpdate();

	// Destroy first object and invalidate references to it.
	ReplicationBridge->EndReplication(Object0);
	DestroyObject(Object0);
	constexpr bool bPerformFullPurge = false;
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, bPerformFullPurge);

	ReplicationSystem->NetUpdate(0.033f);
	FReplicationInstanceOperations::Quantize(SerializationContext, StateBuffers1[1], GetChangeMaskWriter(Protocol1->ChangeMaskBitCount), InstanceProtocol1, Protocol1);
	ReplicationSystem->PostSendUpdate();

	// Compare the two state buffers. Despite that we haven't manually changed any object references we expect them to differ thanks to garbage collection support.
	UE_NET_ASSERT_FALSE(FReplicationProtocolOperationsInternal::IsEqualQuantizedState(SerializationContext, StateBuffers1[0], StateBuffers1[1], Protocol1));
}

UE_NET_TEST_FIXTURE(FTestReplicationOperationsForObjectsFixture, StaleObjectPointerPolledAfterGC)
{
	const bool bIsGarbageEliminationEnabled = UObjectBaseUtility::IsGarbageEliminationEnabled();
	UObjectBaseUtility::ForceSetGarbageEliminationEnabledForTest(true);
	ON_SCOPE_EXIT
	{
		UObjectBaseUtility::ForceSetGarbageEliminationEnabledForTest(bIsGarbageEliminationEnabled);
	};

	// Create two objects, one which is referencing the other. Verify reference gets updated when first object is destroyed.
	UTestReplicatedIrisObject* Object0 = CreateObject();

	UTestReplicatedIrisObject* Object1 = CreateObject();
	Object1->AddComponents({.ObjectReferenceComponentCount = 1});
	Object1->ObjectReferenceComponents[0]->RawObjectPtrRef = Object0;
	FNetRefHandle HandleToObject1 = ReplicationBridge->BeginReplication(Object1);

	const FReplicationInstanceProtocol* InstanceProtocol1 = ReplicationBridge->GetReplicationInstanceProtocol(HandleToObject1);
	const FReplicationProtocol* Protocol1 = ReplicationSystem->GetReplicationProtocol(HandleToObject1);

	// Verify protocol, if it fails, the test needs to be updated
	UE_NET_ASSERT_EQ(Protocol1->ReplicationStateCount, 2U);
	UE_NET_ASSERT_EQ((uint32)Protocol1->ReplicationStateDescriptors[1]->MemberCount, 3U);
	UE_NET_ASSERT_TRUE(FCString::Strcmp(Protocol1->ReplicationStateDescriptors[1]->MemberDebugDescriptors[1].DebugName->Name, TEXT("RawObjectPtrRef")) == 0);

	// Update polling
	ReplicationSystem->NetUpdate(0.033f);
	ReplicationSystem->PostSendUpdate();

	// Get polled value
	UObject* PolledPointerValue0 = nullptr;
	((FPropertyReplicationFragment*)InstanceProtocol1->Fragments[1])->GetPropertyReplicationState()->GetPropertyValue(1, &PolledPointerValue0);

	// Should be same as source
	UE_NET_ASSERT_TRUE(PolledPointerValue0 == Object1->ObjectReferenceComponents[0]->RawObjectPtrRef);

	// Destroy first object and invalidate references to it.
	ReplicationBridge->EndReplication(Object0);
	DestroyObject(Object0);
	constexpr bool bPerformFullPurge = false;
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, bPerformFullPurge);

	// Update polling
	ReplicationSystem->NetUpdate(0.033f);
	ReplicationSystem->PostSendUpdate();

	// Get polled value
	UObject* PolledPointerValue1 = nullptr;
	((FPropertyReplicationFragment*)InstanceProtocol1->Fragments[1])->GetPropertyReplicationState()->GetPropertyValue(1, &PolledPointerValue1);

	// Polled value should now be nullptr
	UE_NET_ASSERT_TRUE(PolledPointerValue1 == nullptr);
}

class FTestReplicationOperationsOnInitStateFixture : public FTestReplicationOperationsFixture
{
protected:
	static const FReplicationStateDescriptor* GetFirstInitStateDescriptor(const FReplicationProtocol* Protocol)
	{
		for (const FReplicationStateDescriptor* StateDescriptor : MakeArrayView(Protocol->ReplicationStateDescriptors, Protocol->ReplicationStateCount))
		{
			if (EnumHasAnyFlags(StateDescriptor->Traits, EReplicationStateTraits::InitOnly))
			{
				return StateDescriptor;
			}
		}

		return nullptr;
	}
};

UE_NET_TEST_FIXTURE(FTestReplicationOperationsOnInitStateFixture, StateWithInitTraitExists)
{
	constexpr uint32 PropertyComponentCount = 2;
	UTestReplicatedIrisObject* TestObject = CreateObject(PropertyComponentCount, 0);

	const FNetRefHandle Handle = ReplicationBridge->BeginReplication(TestObject);
	const FReplicationProtocol* Protocol = ReplicationSystem->GetReplicationProtocol(Handle);
	uint32 InitStateCount = 0;
	for (const FReplicationStateDescriptor* StateDescriptor : MakeArrayView(Protocol->ReplicationStateDescriptors, Protocol->ReplicationStateCount))
	{
		InitStateCount += EnumHasAnyFlags(StateDescriptor->Traits, EReplicationStateTraits::InitOnly);
	}
	// There should be one init state per component having an init property
	UE_NET_ASSERT_EQ(InitStateCount, PropertyComponentCount);
}

UE_NET_TEST_FIXTURE(FTestReplicationOperationsOnInitStateFixture, InitStateHasNoChangeMask)
{
	constexpr uint32 PropertyComponentCount = 1;
	UTestReplicatedIrisObject* TestObject = CreateObject(PropertyComponentCount, 0);

	const FNetRefHandle Handle = ReplicationBridge->BeginReplication(TestObject);
	const FReplicationProtocol* Protocol = ReplicationSystem->GetReplicationProtocol(Handle);
	const FReplicationStateDescriptor* InitStateDescriptor = GetFirstInitStateDescriptor(Protocol);

	UE_NET_ASSERT_EQ(InitStateDescriptor->ChangeMaskBitCount, uint16(0));

	ReplicationBridge->EndReplication(TestObject);
}

class FTestReplicationOperationsOnConnectionFilterStateFixture : public FTestReplicationOperationsFixture
{
protected:
};

UE_NET_TEST_FIXTURE(FTestReplicationOperationsOnConnectionFilterStateFixture, ProtocolHasLifetimeConditionalsTrait)
{
	UTestReplicatedIrisObject::FComponents Components;
	Components.ConnectionFilteredComponentCount = 2;
	UTestReplicatedIrisObject* TestObject = CreateObject();
	TestObject->AddComponents(Components);

	const FNetRefHandle Handle = ReplicationBridge->BeginReplication(TestObject);
	const FReplicationProtocol* Protocol = ReplicationSystem->GetReplicationProtocol(Handle);

	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(Protocol->ProtocolTraits, EReplicationProtocolTraits::HasLifetimeConditionals));

	ReplicationBridge->EndReplication(TestObject);
}

UE_NET_TEST_FIXTURE(FTestReplicationOperationsOnConnectionFilterStateFixture, StateWitHasLifetimeConditionalsTraitExists)
{
	UTestReplicatedIrisObject::FComponents Components;
	Components.ConnectionFilteredComponentCount = 2;
	UTestReplicatedIrisObject* TestObject = CreateObject();
	TestObject->AddComponents(Components);

	const FNetRefHandle Handle = ReplicationBridge->BeginReplication(TestObject);
	const FReplicationProtocol* Protocol = ReplicationSystem->GetReplicationProtocol(Handle);
	uint32 LifetimeConditionalsStateCount = 0;
	for (const FReplicationStateDescriptor* StateDescriptor : MakeArrayView(Protocol->ReplicationStateDescriptors, Protocol->ReplicationStateCount))
	{
		LifetimeConditionalsStateCount += EnumHasAnyFlags(StateDescriptor->Traits, EReplicationStateTraits::HasLifetimeConditionals);
	}

	UE_NET_ASSERT_EQ(LifetimeConditionalsStateCount, Components.ConnectionFilteredComponentCount);

	ReplicationBridge->EndReplication(TestObject);
}

}
