// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/ReplicationSystem/ReplicationRecord.h"

namespace UE::Net::Private
{

static constexpr FReplicationRecord::ReplicationRecordIndex InvalidReplicationRecordIndex = FReplicationRecord::InvalidReplicationRecordIndex;
static constexpr FReplicationRecord::ReplicationRecordIndex MaxReplicationRecordCount = FReplicationRecord::MaxReplicationRecordCount;

UE_NET_TEST(ReplicationRecord, ResetRecordList)
{
	FReplicationRecord ReplicationRecord;
	FReplicationRecord::FRecordInfoList List;

	ReplicationRecord.ResetList(List);

	UE_NET_ASSERT_EQ(List.LastRecordIndex, InvalidReplicationRecordIndex);
	UE_NET_ASSERT_EQ(List.FirstRecordIndex, InvalidReplicationRecordIndex);
}

UE_NET_TEST(ReplicationRecord, PushInfoAndAddToList)
{
	FReplicationRecord ReplicationRecord;
	FReplicationRecord::FRecordInfoList List;

	ReplicationRecord.ResetList(List);

	// Insert entries in list
	const uint32 InfoCount = 10;
	for (uint32 It = 0; It < InfoCount; ++It)
	{
		FReplicationRecord::FRecordInfo Info;
		ReplicationRecord.PushInfoAndAddToList(List, Info);

		UE_NET_ASSERT_EQ(It, (uint32)List.LastRecordIndex);
	}
	UE_NET_ASSERT_EQ(0U, (uint32)List.FirstRecordIndex);

	// Iterate over entries from start to end
	FReplicationRecord::ReplicationRecordIndex CurrentIndex = List.FirstRecordIndex;
	FReplicationRecord::ReplicationRecordIndex CurrentExpectedIndex = 0;
	
	while (CurrentIndex != InvalidReplicationRecordIndex)
	{
		UE_NET_ASSERT_EQ(CurrentExpectedIndex, CurrentIndex);

		const FReplicationRecord::FRecordInfo* CurrentRecordInfo = ReplicationRecord.GetInfoForIndex(CurrentIndex);
		UE_NET_ASSERT_NE(CurrentRecordInfo, nullptr);

		CurrentIndex = CurrentRecordInfo->NextIndex;
		++CurrentExpectedIndex;
	};
	UE_NET_ASSERT_EQ(InfoCount, static_cast<uint32>(CurrentExpectedIndex));
}

UE_NET_TEST(ReplicationRecord, PopInfoAndRemoveFromList)
{
	FReplicationRecord ReplicationRecord;
	FReplicationRecord::FRecordInfoList List;

	ReplicationRecord.ResetList(List);

	// Insert entries in list
	const uint32 InfoCount = 10;
	for (uint32 It = 0; It < InfoCount; ++It)
	{
		FReplicationRecord::FRecordInfo Info;
		ReplicationRecord.PushInfoAndAddToList(List, Info);
	}

	for (uint32 It = 0; It < InfoCount; ++It)
	{
		// We expect the order to be the same when we pushed data
		UE_NET_ASSERT_EQ(It, (uint32)ReplicationRecord.GetFrontIndex());
		ReplicationRecord.PopInfoAndRemoveFromList(List);
		
		if (It < (InfoCount - 1))
		{
			// First record index should now point to It +1 or be invalid
			UE_NET_ASSERT_EQ(It + 1 , (uint32)List.FirstRecordIndex);
			UE_NET_ASSERT_EQ(InfoCount - 1, (uint32)List.LastRecordIndex);
		}
		else
		{
			// List should now be empty
			UE_NET_ASSERT_EQ(InvalidReplicationRecordIndex, List.LastRecordIndex);
			UE_NET_ASSERT_EQ(InvalidReplicationRecordIndex, List.FirstRecordIndex);
		}
	}
}

UE_NET_TEST(ReplicationRecord, TestMaxCapacityAndWraparound)
{
	FReplicationRecord ReplicationRecord;
	FReplicationRecord::FRecordInfoList List;

	ReplicationRecord.ResetList(List);

	// Insert entries in list to max capacity
	const uint32 InfoCount = MaxReplicationRecordCount;
	for (uint32 It = 0; It < InfoCount; ++It)
	{
		FReplicationRecord::FRecordInfo Info;
		Info.Index = It;
		ReplicationRecord.PushInfoAndAddToList(List, Info);
	}

	UE_NET_ASSERT_EQ((uint32)List.LastRecordIndex, MaxReplicationRecordCount - 1U);
	UE_NET_ASSERT_EQ((uint32)List.FirstRecordIndex, 0U);

	// Pop one so we can test wraparound
	ReplicationRecord.PopInfoAndRemoveFromList(List);
	
	UE_NET_ASSERT_EQ((uint32)ReplicationRecord.GetFrontIndex(), 1U);

	// Push one more
	FReplicationRecord::FRecordInfo Info;
	ReplicationRecord.PushInfoAndAddToList(List, Info);

	// should wraparound
	UE_NET_ASSERT_EQ(0U, (uint32)List.LastRecordIndex);

	// Access Last pushed entry and verify that it points to what we expect
	FReplicationRecord::FRecordInfo* LastInfo = ReplicationRecord.GetInfoForIndex(List.LastRecordIndex);
	UE_NET_ASSERT_NE(LastInfo, nullptr);
	UE_NET_ASSERT_EQ(InvalidReplicationRecordIndex, LastInfo->NextIndex);
}

}
