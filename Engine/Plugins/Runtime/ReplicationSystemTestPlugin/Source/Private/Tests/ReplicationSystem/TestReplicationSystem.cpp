// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSystemServerClientTestFixture.h"
#include "HAL/IConsoleManager.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Metrics/NetMetrics.h"
#include "Iris/ReplicationSystem/ReplicationRecord.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/ReplicationReader.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Misc/ScopeExit.h"
#include "Net/Core/NetToken/NetToken.h"
#include "UObject/CoreNetTypes.h"

namespace UE::Net::Private
{

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, ReplicateAndDestroySingleObject)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0,0);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that created server handle now also exists on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));

	// Destroy the spawned object on server
	Server->DestroyObject(ServerObject);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that object now is destroyed on client as well
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerObject->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestDroppedDestroyed)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0,0);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that created server handle now also exists on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));

	// Destroy the spawned object on server
	Server->DestroyObject(ServerObject);

	// Send and drop packet
	Server->UpdateAndSend({Client}, DoNotDeliverPacket);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that object now is destroyed on client
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerObject->NetRefHandle));
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestDestroyWhilePendingCreate)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0,0);
	const FNetRefHandle ServerObjectRefHandle = ServerObject->NetRefHandle;

	// Send and drop packet
	Server->UpdateAndSend({Client}, DoNotDeliverPacket);

	// Destroy
	Server->DestroyObject(ServerObject);

	// Verify that the object does not exist on the client
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerObjectRefHandle));

	// Send and drop packet
	Server->UpdateAndSend({Client}, DoNotDeliverPacket);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Verify the object still doesn't exist on the client
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerObjectRefHandle));
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestDestroyWhileWaitingOnCreate)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0,0);
	const FNetRefHandle ServerObjectRefHandle = ServerObject->NetRefHandle;

	// Write packet with create
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Destroy while we are waiting for confirmation
	Server->DestroyObject(ServerObject);

	// Write packet with destroy
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Drop packet with create
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// Verify that the object does not exists on the client
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerObjectRefHandle));

	// Deliver packet with destroy
	Server->DeliverTo(Client, DeliverPacket);

	// Verify the object still doesn't exist on the client
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerObjectRefHandle));
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestDestroyWithDataInFlight)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);
	const FNetRefHandle ServerObjectRefHandle = ServerObject->NetRefHandle;
	Server->GetReplicationSystem()->SetStaticPriority(ServerObjectRefHandle, 1.f);

	// Send packet with create
	Server->UpdateAndSend({Client});

	// Verify that the object exists on the client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObjectRefHandle));

	// Modify some data to mark object dirty
	ServerObject->IntA = 13;

	// Write a packet with updated data
	Server->NetUpdate();
	UE_NET_ASSERT_TRUE(Server->SendTo(Client));
	Server->PostSendUpdate();

	// Destroy while we are waiting for ack on update
	Server->DestroyObject(ServerObject);

	// Write packet with destroy
	Server->NetUpdate();
	UE_NET_ASSERT_TRUE(Server->SendTo(Client));
	Server->PostSendUpdate();

	// Drop and report packet with update as lost
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// Verify that the object still exists on the client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObjectRefHandle));
	
	// Deliver packet with destroy
	Server->DeliverTo(Client, DeliverPacket);

	// Verify that the object is destroyed on the client
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerObjectRefHandle));
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, ReplicateAndDestroySubObject)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0,0);
	const FNetRefHandle ServerObjectRefHandle = ServerObject->NetRefHandle;
	// Spawn second object on server as a subobject
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	const FNetRefHandle ServerSubObjectRefHandle = ServerSubObject->NetRefHandle;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that created server objects now also exist on the client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObjectRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerSubObjectRefHandle));

	// Destroy the spawned subobject on server
	Server->DestroyObject(ServerSubObject);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that only the subobject is destroyed on the client
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerSubObjectRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObjectRefHandle));
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, ReplicateAndDestroyMultipleSubObjects)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	const FNetRefHandle ServerObjectRefHandle = ServerObject->NetRefHandle;
	// Spawn subobject on server
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	FNetRefHandle ServerSubObjectRefHandle = ServerSubObject->NetRefHandle;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that created server objects now also exist on the client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObjectRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerSubObjectRefHandle));

	// Destroy the spawned subobject on server
	Server->DestroyObject(ServerSubObject);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that the subobject is destroyed on the client
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerSubObjectRefHandle));

	// Spawn second object on server as a subobject
	ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	ServerSubObjectRefHandle = ServerSubObject->NetRefHandle;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that second subobject replicated properly to server
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerSubObjectRefHandle));

	// Destroy the spawned object on server
	Server->DestroyObject(ServerSubObject);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that the second subobjects object is destroyed on the client
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerSubObjectRefHandle));
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, ReplicateAndDestroySubObjectAndDestroyOwner)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	const FNetRefHandle ServerObjectRefHandle = ServerObject->NetRefHandle;
	// Spawn subobject on server
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	const FNetRefHandle ServerSubObjectRefHandle = ServerSubObject->NetRefHandle;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that created server handles now also exist on the client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObjectRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerSubObjectRefHandle));

	// Destroy the spawned subobject on server
	Server->DestroyObject(ServerSubObject);

	// Destroy owner after spawned subobject on server
	Server->DestroyObject(ServerObject);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that the subobject is destroyed on the client
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerSubObjectRefHandle));

	// Verify that the root object is destroyed on the client
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerObjectRefHandle));
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, ReplicateAndDestroySubObjectAndDestroyOwnerWithDataInFlight)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	const FNetRefHandle ServerObjectRefHandle = ServerObject->NetRefHandle;
	// Spawn subobject on server
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	const FNetRefHandle ServerSubObjectRefHandle = ServerSubObject->NetRefHandle;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that created server handles now also exist on the client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObjectRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerSubObjectRefHandle));

	// Destroy the spawned subobject on server
	Server->DestroyObject(ServerSubObject);

	// Send and drop packet
	Server->UpdateAndSend({Client}, DoNotDeliverPacket);

	// Destroy owner after we spawned subobject on server
	Server->DestroyObject(ServerObject);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Verify that the subobject is destroyed on the client
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerSubObjectRefHandle));
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, ReplicateAndDestroySubObjectWithLostData)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	const FNetRefHandle ServerObjectRefHandle = ServerObject->NetRefHandle;
	// Spawn subobject on server
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	const FNetRefHandle ServerSubObjectRefHandle = ServerSubObject->NetRefHandle;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that created server objects exist on the client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObjectRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerSubObjectRefHandle));

	// Destroy the spawned subobject on server
	Server->DestroyObject(ServerSubObject);

	// Send and drop packet
	Server->UpdateAndSend({Client}, DoNotDeliverPacket);

	// Send and deliver packet
	Server->UpdateAndSend({Client}, DeliverPacket);

	// Verify that subobject is destroyed on the client
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerSubObjectRefHandle));
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, ReplicateAndDestroySubObjectPendingCreateConfirmation)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);
	const FNetRefHandle ServerObjectRefHandle = ServerObject->NetRefHandle;
	// Spawn subobject on server
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	const FNetRefHandle ServerSubObjectRefHandle = ServerSubObject->NetRefHandle;

	uint32 NumUnAcknowledgedPackets = 0;
	// Write a packet
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Destroy the spawned subobject on server
	Server->DestroyObject(ServerSubObject);

	// We have no data to send but we want to tick ReplicationSystem to capture state change
	Server->NetUpdate();
	UE_NET_ASSERT_FALSE(Server->SendTo(Client));
	Server->PostSendUpdate();

	// Drop creation packet
	Server->DeliverTo(Client, DoNotDeliverPacket);

	// As the second update did not send any data we do not have anything to deliver
	//Server->DeliverTo(Client, true);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that the subobject does not exist on the client
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerSubObjectRefHandle));

	// The root object should exist on the client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObjectRefHandle));
}

// In this test we're going to create a subobject after the root has been created on the client. Then create it with a bit of latency and destroy root prior to subobject being created on the client.
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, LateCreatedSubObjectIsDestroyedWithRoot)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	const FNetRefHandle ServerObjectRefHandle = ServerObject->NetRefHandle;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObjectRefHandle));

	// Spawn subobject
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObjectRefHandle, UTestReplicatedIrisObject::FComponents{});
	const FNetRefHandle ServerSubObjectRefHandle = ServerSubObject->NetRefHandle;

	// Put subobject in WaitOnCreateConfirmation state
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Destroy the root object
	Server->DestroyObject(ServerObject);

	// Write a packet
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Deliver first packet
	Server->DeliverTo(Client, DeliverPacket);

	// Verify the root and subobject are created/still created on the client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObjectRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerSubObjectRefHandle));

	// Deliver second packet
	Server->DeliverTo(Client, DeliverPacket);

	// Verify the root and subobject are created/still created on the client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObjectRefHandle));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerSubObjectRefHandle));

	// Update and send
	Server->UpdateAndSend({Client});

	// Verify the root and subobject are fully destroyed
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerObjectRefHandle));
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerSubObjectRefHandle));
}

// In this test we're going to try destroying an object with thousands of subobjects atomically.
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, CanDestroyObjectHierarchyAtomically)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	const FNetRefHandle ServerObjectRefHandle = ServerObject->NetRefHandle;

	constexpr unsigned SubObjectCount = 2001;
	TArray<FNetRefHandle> ServerSubObjectRefHandles;
	ServerSubObjectRefHandles.Reserve(SubObjectCount);

	// Spawn thousands of subobjects over several frames to avoid huge object path.
	for (;ServerSubObjectRefHandles.Num() < SubObjectCount;)
	{
		constexpr uint32 MaxSubObjectCreationCountPerFrame = 15;
		for (unsigned It = 0; It < MaxSubObjectCreationCountPerFrame && ServerSubObjectRefHandles.Num() < SubObjectCount; ++It)
		{
			UReplicatedTestObject* ServerSubObject = Server->CreateSubObject<UReplicatedTestObject>(ServerObjectRefHandle);
			ServerSubObjectRefHandles.Add(ServerSubObject->NetRefHandle);
		}

		Server->UpdateAndSend({Client});
	}

	// We expect to be done creating the object hierarchy by now. Make sure of it.
	UE_NET_ASSERT_FALSE(Server->UpdateAndSend({ Client }));
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObjectRefHandle));
	for (const FNetRefHandle SubObjectRefHandle : ServerSubObjectRefHandles)
	{
		UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(SubObjectRefHandle));
	}

	// Destroy the root object
	Server->DestroyObject(ServerObject);

	for (unsigned It = 0, MaxTryCount = SubObjectCount/50; It < MaxTryCount; ++It)
	{
		const bool bDidSendSomething = Server->UpdateAndSend({ Client });

		// Verify the object hierarchy is destroyed as a whole or not at all. Once we've stopped sending data we should have destroyed the object as a whole on the client.
		const bool bRootIsResolvable = Client->IsResolvableNetRefHandle(ServerObjectRefHandle);
		if (bRootIsResolvable && bDidSendSomething)
		{
			for (const FNetRefHandle SubObjectRefHandle : ServerSubObjectRefHandles)
			{
				UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(SubObjectRefHandle));
			}
		}
		else
		{
			UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerObjectRefHandle));
			for (const FNetRefHandle SubObjectRefHandle : ReverseIterate(ServerSubObjectRefHandles))
			{
				UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(SubObjectRefHandle));
			}
		}

		if (!bDidSendSomething)
		{
			break;
		}
	}
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestSubObjectDefaultReplicationOrder)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

	// Spawn some subobjects
	UReplicatedSubObjectOrderObject* ServerSubObject0 = Server->CreateSubObject<UReplicatedSubObjectOrderObject>(ServerObject->NetRefHandle);
	UReplicatedSubObjectOrderObject* ServerSubObject1 = Server->CreateSubObject<UReplicatedSubObjectOrderObject>(ServerObject->NetRefHandle);
	UReplicatedSubObjectOrderObject* ServerSubObject2 = Server->CreateSubObject<UReplicatedSubObjectOrderObject>(ServerObject->NetRefHandle);

	UReplicatedSubObjectOrderObject::RepOrderCounter = 0U;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that objects have replicated
	UReplicatedSubObjectOrderObject* ClientSubObject0 = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject0->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientSubObject1 = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject1->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientSubObject2 = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject2->NetRefHandle));

	UE_NET_ASSERT_NE(ClientSubObject0, nullptr);
	UE_NET_ASSERT_NE(ClientSubObject1, nullptr);
	UE_NET_ASSERT_NE(ClientSubObject2, nullptr);

	// Verify that they have replicated in expected order
	UE_NET_ASSERT_EQ(ClientSubObject0->LastRepOrderCounter, 1U);
	UE_NET_ASSERT_GT(ClientSubObject1->LastRepOrderCounter, ClientSubObject0->LastRepOrderCounter);
	UE_NET_ASSERT_GT(ClientSubObject2->LastRepOrderCounter, ClientSubObject1->LastRepOrderCounter);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestSubObjectReplicationOrderWithChildSubObjectsReplicatingBeforeParent)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

	// Spawn subobject set to replicate its child subobjects before the parent
	UReplicatedSubObjectOrderObject* ServerSubObject0 = Server->CreateSubObjectWithParams<UReplicatedSubObjectOrderObject>(
		{
			.RootObjectHandle = ServerObject->NetRefHandle,
			.ChildSubObjectsReplicationOrder = EChildSubObjectsReplicationOrder::BeforeParent,
		});

	// Specify Subobject1 to replicate with SubObject0 as its parent, and since SubObject0 is set to replicate children before itself we expect this to be reflected in the order.
	UReplicatedSubObjectOrderObject* ServerSubObject1 = Server->CreateSubObjectWithParams<UReplicatedSubObjectOrderObject>(
		{
			.RootObjectHandle = ServerObject->NetRefHandle,
			.ParentSubObjectHandle = ServerSubObject0->NetRefHandle
		});

	// Create SubObject 2 to replicate with no specific order (it will be added Last in the root list and thus replicate last.)
	UReplicatedSubObjectOrderObject* ServerSubObject2 = Server->CreateSubObjectWithParams<UReplicatedSubObjectOrderObject>(
		{
			.RootObjectHandle = ServerObject->NetRefHandle,
		});

	UReplicatedSubObjectOrderObject::RepOrderCounter = 0U;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that objects have replicated
	UReplicatedSubObjectOrderObject* ClientSubObject0 = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject0->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientSubObject1 = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject1->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientSubObject2 = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject2->NetRefHandle));

	UE_NET_ASSERT_NE(ClientSubObject0, nullptr);
	UE_NET_ASSERT_NE(ClientSubObject1, nullptr);
	UE_NET_ASSERT_NE(ClientSubObject2, nullptr);

	// Verify that they have replicated in expected order setup earlier
	UE_NET_ASSERT_EQ(ClientSubObject1->LastRepOrderCounter, 1U);
	UE_NET_ASSERT_EQ(ClientSubObject0->LastRepOrderCounter, 2U);
	UE_NET_ASSERT_EQ(ClientSubObject2->LastRepOrderCounter, 3U);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestSubObjectReplicationOrderWithChildSubObjectsReplicatingAfterParent)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

	// Spawn subobject set to replicate its child subobjects after the parent
	UReplicatedSubObjectOrderObject* ServerSubObject0 = Server->CreateSubObjectWithParams<UReplicatedSubObjectOrderObject>(
		{
			.RootObjectHandle = ServerObject->NetRefHandle,
			.ChildSubObjectsReplicationOrder = EChildSubObjectsReplicationOrder::AfterParent,
		});

	// Specify Subobject1 to replicate with SubObject0 as its parent, and since SubObject0 is set to replicate children after itself we expect this to be reflected in the order.
	UReplicatedSubObjectOrderObject* ServerSubObject1 = Server->CreateSubObjectWithParams<UReplicatedSubObjectOrderObject>(
		{
			.RootObjectHandle = ServerObject->NetRefHandle, 
			.ParentSubObjectHandle = ServerSubObject0->NetRefHandle, 
		});

	// Specify SubObect 2 to replicate with no specific order (it will be added to the root list and thus replicate last)
	UReplicatedSubObjectOrderObject* ServerSubObject2 = Server->CreateSubObject<UReplicatedSubObjectOrderObject>(ServerObject->NetRefHandle);

	UReplicatedSubObjectOrderObject::RepOrderCounter = 0U;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that objects have replicated
	UReplicatedSubObjectOrderObject* ClientSubObject0 = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject0->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientSubObject1 = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject1->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientSubObject2 = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject2->NetRefHandle));

	UE_NET_ASSERT_NE(ClientSubObject0, nullptr);
	UE_NET_ASSERT_NE(ClientSubObject1, nullptr);
	UE_NET_ASSERT_NE(ClientSubObject2, nullptr);

	// Verify that they have replicated in expected order setup earlier
	UE_NET_ASSERT_EQ(ClientSubObject0->LastRepOrderCounter, 1U);
	UE_NET_ASSERT_EQ(ClientSubObject1->LastRepOrderCounter, 2U);
	UE_NET_ASSERT_EQ(ClientSubObject2->LastRepOrderCounter, 3U);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestSubObjectInsertAtStart)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedObjectTestSubObjectCreationOrder* ServerObject = Server->CreateObject<UReplicatedObjectTestSubObjectCreationOrder>();

	// Spawn a subobject
	UReplicatedSubObjectOrderObject* ServerSubObject0 = Server->CreateSubObjectWithParams<UReplicatedSubObjectOrderObject>(
		{
			.RootObjectHandle = ServerObject->NetRefHandle,
			.ChildSubObjectsReplicationOrder = EChildSubObjectsReplicationOrder::AfterParent,
		});
	
	// Spawn a subobject and make it replicate as a child to SubObject0,
	UReplicatedSubObjectOrderObject* ServerSubObject1 = Server->CreateSubObjectWithParams<UReplicatedSubObjectOrderObject>(
		{
			.RootObjectHandle = ServerObject->NetRefHandle,
			.ParentSubObjectHandle = ServerSubObject0->NetRefHandle, 
		});

	// Spawn a subobject and make it replicate as a child to SubObbject0 but before ServerSubObject1
	UReplicatedSubObjectOrderObject* ServerSubObject2 = Server->CreateSubObjectWithParams<UReplicatedSubObjectOrderObject>(
		{
			.RootObjectHandle = ServerObject->NetRefHandle,
			.ParentSubObjectHandle = ServerSubObject0->NetRefHandle, 
			.InsertionOrder = UE::Net::ESubObjectInsertionOrder::InsertAtStart
		});

	// Spawn a subobject and make it replicate first of all subobjects 
	UReplicatedSubObjectOrderObject* ServerSubObject3 = Server->CreateSubObjectWithParams<UReplicatedSubObjectOrderObject>(
		{
			.RootObjectHandle = ServerObject->NetRefHandle, 
			.InsertionOrder = UE::Net::ESubObjectInsertionOrder::InsertAtStart
		});

	UReplicatedSubObjectOrderObject::RepOrderCounter = 0U;

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

	// Verify that objects have replicated
	UReplicatedSubObjectOrderObject* ClientSubObject0 = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject0->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientSubObject1 = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject1->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientSubObject2 = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject2->NetRefHandle));
	UReplicatedSubObjectOrderObject* ClientSubObject3 = Cast<UReplicatedSubObjectOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject3->NetRefHandle));

	UE_NET_ASSERT_NE(ClientSubObject0, nullptr);
	UE_NET_ASSERT_NE(ClientSubObject1, nullptr);
	UE_NET_ASSERT_NE(ClientSubObject2, nullptr);

	// Verify that they were created in the expected order
	UE_NET_ASSERT_EQ(ClientSubObject3->CreationOrder, 1);
	UE_NET_ASSERT_EQ(ClientSubObject0->CreationOrder, 2);
	UE_NET_ASSERT_EQ(ClientSubObject2->CreationOrder, 3);
	UE_NET_ASSERT_EQ(ClientSubObject1->CreationOrder, 4);

	// Verify that they have replicated in the expected order
	UE_NET_ASSERT_EQ(ClientSubObject3->LastRepOrderCounter, 1U);
	UE_NET_ASSERT_EQ(ClientSubObject0->LastRepOrderCounter, 2U);
	UE_NET_ASSERT_EQ(ClientSubObject2->LastRepOrderCounter, 3U);
	UE_NET_ASSERT_EQ(ClientSubObject1->LastRepOrderCounter, 4U);
}


class FTestNetTokensFixture : public FReplicationSystemServerClientTestFixture
{
public:
	FStringTokenStore* ServerStringTokenStore = nullptr;
	FStringTokenStore* ClientStringTokenStore = nullptr;
	FReplicationSystemTestClient* Client = nullptr;
	const FNetTokenStoreState* ClientRemoteNetTokenState;
	const FNetTokenStoreState* ServerRemoteNetTokenState;

	FNetToken CreateAndExportNetToken(const FString& TokenString)
	{
		FNetToken Token = ServerStringTokenStore->GetOrCreateToken(TokenString);
		UNetTokenDataStream* NetTokenDataStream = Cast<UNetTokenDataStream>(Server->GetReplicationSystem()->GetDataStream(Client->ConnectionIdOnServer, FName("NetToken")));
		if (NetTokenDataStream)
		{
			NetTokenDataStream->AddNetTokenForExplicitExport(Token);
		}

		return Token;
	}

	FNetToken CreateAndExportNetTokenOnClient(const FString& TokenString)
	{
		FNetToken Token = ClientStringTokenStore->GetOrCreateToken(TokenString);
		UNetTokenDataStream* NetTokenDataStream = Cast<UNetTokenDataStream>(Client->GetReplicationSystem()->GetDataStream(Client->LocalConnectionId, FName("NetToken")));
		if (NetTokenDataStream)
		{
			NetTokenDataStream->AddNetTokenForExplicitExport(Token);
		}

		return Token;
	}

	virtual void SetUp() override
	{
		FReplicationSystemServerClientTestFixture::SetUp();

		Client = CreateClient();
		{
			FNetTokenStore* ServerTokenStore = Server->GetReplicationSystem()->GetNetTokenStore();
			ServerStringTokenStore = ServerTokenStore->GetDataStore<FStringTokenStore>();		
			ServerRemoteNetTokenState = ServerTokenStore->GetRemoteNetTokenStoreState(Client->ConnectionIdOnServer);
		}
		{
			FNetTokenStore* ClientTokenStore = Client->GetReplicationSystem()->GetNetTokenStore();
			ClientStringTokenStore = ClientTokenStore->GetDataStore<FStringTokenStore>();
			ClientRemoteNetTokenState = ClientTokenStore->GetRemoteNetTokenStoreState(Client->LocalConnectionId);
		}
	}
};

UE_NET_TEST_FIXTURE(FTestNetTokensFixture, NetToken)
{
	// Create token
	FString TokenStringA(TEXT("MyStringToken"));
	FNetToken StringTokenA = CreateAndExportNetToken(TokenStringA);

	// Send and drop packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, false);
	Server->PostSendUpdate();

	{
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetToken, ELogVerbosity::Fatal);

		// Verify that we cannot resolve the token on the client
		UE_NET_ASSERT_NE(TokenStringA, FString(ClientStringTokenStore->ResolveRemoteToken(StringTokenA, *ClientRemoteNetTokenState)));
	}

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that we can resolve the token on the client
	UE_NET_ASSERT_EQ(TokenStringA, FString(ClientStringTokenStore->ResolveRemoteToken(StringTokenA, *ClientRemoteNetTokenState)));
}

UE_NET_TEST_FIXTURE(FTestNetTokensFixture, NetTokenResendWithFullPacket)
{
	// Create token
	FString TokenStringA(TEXT("MyStringToken"));
	FNetToken StringTokenA = CreateAndExportNetToken(TokenStringA);

	// Send and drop packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, false);
	Server->PostSendUpdate();

	// Limit packet size
	Server->SetMaxSendPacketSize(128U);

	// Create a new token that will not fit in the packet and only fit the resend data
	FString TokenStringB(TEXT("MyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongString"));
	FNetToken StringTokenB = CreateAndExportNetToken(TokenStringB);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that we can resolve the token first token on the client even though second one should not fit
	UE_NET_ASSERT_EQ(TokenStringA, FString(ClientStringTokenStore->ResolveRemoteToken(StringTokenA, *ClientRemoteNetTokenState)));
	{
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetToken, ELogVerbosity::Fatal);
		UE_NET_ASSERT_NE(TokenStringB, FString(ClientStringTokenStore->ResolveRemoteToken(StringTokenB, *ClientRemoteNetTokenState)));
	}

	// Restore packet size and make sure that we get the second token through
	Server->SetMaxSendPacketSize(1024U);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	UE_NET_ASSERT_EQ(TokenStringB, FString(ClientStringTokenStore->ResolveRemoteToken(StringTokenB, *ClientRemoteNetTokenState)));
}

UE_NET_TEST_FIXTURE(FTestNetTokensFixture, NetTokenResendWithFullPacketAfterFirstResend)
{
	// Create token
	FString TestStringA(TEXT("MyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongString"));
	FNetToken StringTokenA = CreateAndExportNetToken(TestStringA);	

	// Send and delay delivery
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Create a new token that will not fit in the packet and only fit the resend data
	FString TestStringB(TEXT("MyOtherLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongStringMyLongString"));
	FNetToken StringTokenB = CreateAndExportNetToken(TestStringB);

	// Send and delay delivery
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	Server->DeliverTo(Client, false);
	Server->DeliverTo(Client, false);

	// Verify that tokens has not been received
	{
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetToken, ELogVerbosity::Fatal);

		UE_NET_ASSERT_NE(TestStringA, FString(ClientStringTokenStore->ResolveRemoteToken(StringTokenA, *ClientRemoteNetTokenState)));
		UE_NET_ASSERT_NE(TestStringB, FString(ClientStringTokenStore->ResolveRemoteToken(StringTokenB, *ClientRemoteNetTokenState)));
	}

	// Send and deliver packet which now should contain two entries in the resend queue
	Server->SetMaxSendPacketSize(1024);

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that we can resolve the token
	UE_NET_ASSERT_EQ(TestStringA, FString(ClientStringTokenStore->ResolveRemoteToken(StringTokenA, *ClientRemoteNetTokenState)));
	UE_NET_ASSERT_EQ(TestStringB, FString(ClientStringTokenStore->ResolveRemoteToken(StringTokenB, *ClientRemoteNetTokenState)));
}

UE_NET_TEST_FIXTURE(FTestNetTokensFixture, NetTokenSequenceTest)
{
	const FString TestStrings[] = {
		FString(TEXT("TokenA")),
		FString(TEXT("TokenB")),
		FString(TEXT("TokenC")),
		FString(TEXT("TokenD")),
		FString(TEXT("TokenE")),
		FString(TEXT("TokenF")),
	};

	const uint32 TokenCount = UE_ARRAY_COUNT(TestStrings);

	// Create token
	FNetToken StringTokenA = CreateAndExportNetToken(TestStrings[0]);
	FNetToken StringTokenB = CreateAndExportNetToken(TestStrings[1]);

	// Send packet
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Create token
	FNetToken StringTokenC = CreateAndExportNetToken(TestStrings[2]);

	// Create token
	FNetToken StringTokenD = CreateAndExportNetToken(TestStrings[3]);

	// Send packet
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Drop packet 
	Server->DeliverTo(Client, false);

	// Deliver packet 
	Server->DeliverTo(Client, true);

	// Create local tokens
	ClientStringTokenStore->GetOrCreateToken(TEXT("LocalTokenA"));
	ClientStringTokenStore->GetOrCreateToken(TEXT("LocalTokenB"));

	// Send packet with resend data
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	UE_NET_ASSERT_EQ(TestStrings[0], FString(ClientStringTokenStore->ResolveRemoteToken(StringTokenA, *ClientRemoteNetTokenState)));
	UE_NET_ASSERT_EQ(TestStrings[1], FString(ClientStringTokenStore->ResolveRemoteToken(StringTokenB, *ClientRemoteNetTokenState)));
	UE_NET_ASSERT_EQ(TestStrings[2], FString(ClientStringTokenStore->ResolveRemoteToken(StringTokenC, *ClientRemoteNetTokenState)));
	UE_NET_ASSERT_EQ(TestStrings[3], FString(ClientStringTokenStore->ResolveRemoteToken(StringTokenD, *ClientRemoteNetTokenState)));
}

UE_NET_TEST_FIXTURE(FTestNetTokensFixture, NetTokenResendAndDataInSamePacketTest)
{
	const FString TestStrings[] = {
		FString(TEXT("TokenA")),
		FString(TEXT("TokenB")),
	};

	const uint32 TokenCount = UE_ARRAY_COUNT(TestStrings);


	// Create token
	FNetToken StringTokenA = CreateAndExportNetToken(TestStrings[0]);

	// Send packet
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// drop data
	Server->DeliverTo(Client, false);

	// Create token
	FNetToken StringTokenB = CreateAndExportNetToken(TestStrings[1]);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	UE_NET_ASSERT_EQ(TestStrings[0], FString(ClientStringTokenStore->ResolveRemoteToken(StringTokenA, *ClientRemoteNetTokenState)));
	UE_NET_ASSERT_EQ(TestStrings[1], FString(ClientStringTokenStore->ResolveRemoteToken(StringTokenB, *ClientRemoteNetTokenState)));
}

UE_NET_TEST_FIXTURE(FTestNetTokensFixture, NetTokenAuthority)
{
	// Create token
	FString TokenStringA(TEXT("MyStringToken"));
	FNetToken NonAuthToken = CreateAndExportNetTokenOnClient(TokenStringA);

	UE_NET_ASSERT_EQ(NonAuthToken.IsAssignedByAuthority(), false);

	// Send from server
	Server->UpdateAndSend({Client});

	// Send from client
	Client->UpdateAndSend(Server);

	// We should be able to resolve the token on the server using remote
	UE_NET_ASSERT_EQ(TokenStringA, FString(ServerStringTokenStore->ResolveToken(NonAuthToken, ServerRemoteNetTokenState)));

	// Find server token.
	FNetToken AuthToken = CreateAndExportNetToken(TokenStringA);

	// It should be a different token as the server is authoriative
	UE_NET_ASSERT_FALSE(AuthToken == NonAuthToken);

	// Send from server
	Server->UpdateAndSend({Client});

	// Client should be able to resolve ServerToken
	UE_NET_ASSERT_EQ(TokenStringA, FString(ClientStringTokenStore->ResolveToken(AuthToken, ClientRemoteNetTokenState)));

	// If we now try to create a token for the string also received from the authority we expect it to give us the server token and allow us to use that instead of the local exported token.
	FNetToken NewClientToken = ClientStringTokenStore->GetOrCreateToken(TokenStringA);

	// We expect the tokens to be identical.
	UE_NET_ASSERT_TRUE(AuthToken == NewClientToken);
}

UE_NET_TEST_FIXTURE(FTestNetTokensFixture, NetTokenAuthTokenIsNotExportedFromClient)
{
	// Create token
	FString TokenStringA(TEXT("MyStringToken"));
	FNetToken AuthToken = CreateAndExportNetToken(TokenStringA);

	UE_NET_ASSERT_EQ(AuthToken.IsAssignedByAuthority(), true);

	// Send from server
	Server->UpdateAndSend({Client});

	// Expect to get auth token
	FNetToken ClientExpectedAuthToken = CreateAndExportNetTokenOnClient(TokenStringA);
	UE_NET_ASSERT_EQ(ClientExpectedAuthToken.IsAssignedByAuthority(), true);

	// Send from client
	Client->UpdateAndSend(Server);

	// $TODO: Expose some stats that we can query for exports.
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, AddRemoveFromConnectionScopeTest)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Add to group
	FNetObjectGroupHandle Group = ReplicationSystem->CreateGroup(NAME_None);
	ReplicationSystem->AddToGroup(Group, ServerObject->NetRefHandle);

	ReplicationSystem->AddExclusionFilterGroup(Group);
	ReplicationSystem->SetGroupFilterStatus(Group, ENetFilterStatus::Allow);

	// Start replicating object
	
	// Send packet
	// Expected state to be WaitOnCreateConfirmation
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Make sure we have data in flight
	++ServerObject->IntA;

	// Disallow group to trigger state change from PendingCreateConfirmation->PendingDestroy
	ReplicationSystem->SetGroupFilterStatus(Group, ENetFilterStatus::Disallow);	

	// Expect client to create object
	Server->DeliverTo(Client, true);
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Send packet
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Allow group to trigger state to ensure that we restart replication
	ReplicationSystem->SetGroupFilterStatus(Group, ENetFilterStatus::Allow);

	// Expect client to destroy object
	Server->DeliverTo(Client, true);
	UE_NET_ASSERT_EQ(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);

	// Trigger replication
	++ServerObject->IntA;

	// Send packet
	// WaitOnDestroyConfirmation -> WaitOnCreateConfirmation
	Server->UpdateAndSend({ Client });

	// Verify that the object got created again
	UE_NET_ASSERT_NE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle), nullptr);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestNetTemporary)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);
	UTestReplicatedIrisObject* ServerObject1 = Server->CreateObject(0,0);

	// Trigger replication
	ServerObject->IntA = 1;
	ServerObject1->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that client has received the data
	{
		UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

		UE_NET_ASSERT_TRUE(ClientObject != nullptr);
		UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObject->IntA);
	}

	{
		UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject1->NetRefHandle));

		UE_NET_ASSERT_TRUE(ClientObject != nullptr);
		UE_NET_ASSERT_EQ(ServerObject1->IntA, ClientObject->IntA);

	}

	// Mark the object as a net temporary
	ReplicationSystem->SetIsNetTemporary(ServerObject->NetRefHandle);

	// Modify the value
	ServerObject->IntA = 2;
	ServerObject1->IntA = 2;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that client has not received the data for changed temporary
	{
		UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

		UE_NET_ASSERT_TRUE(ClientObject != nullptr);
		UE_NET_ASSERT_NE(ServerObject->IntA, ClientObject->IntA);
	}

	// Verify that client has received the data for normal object
	{
		UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject1->NetRefHandle));

		UE_NET_ASSERT_TRUE(ClientObject != nullptr);
		UE_NET_ASSERT_EQ(ServerObject1->IntA, ClientObject->IntA);
	}

	// Test Late join
	// Add a client
	FReplicationSystemTestClient* Client2 = CreateClient();

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client2, true);
	Server->PostSendUpdate();

	// We should now have the latest state for both objects
	{
		UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client2->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

		UE_NET_ASSERT_TRUE(ClientObject != nullptr);
		UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObject->IntA);
	}

	// Verify that client has received the data for normal object
	{
		UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client2->GetReplicationBridge()->GetReplicatedObject(ServerObject1->NetRefHandle));

		UE_NET_ASSERT_TRUE(ClientObject != nullptr);
		UE_NET_ASSERT_EQ(ServerObject1->IntA, ClientObject->IntA);
	}
}

// Tests for TearOff
class FTearOffAndFlushTestFixture : public FReplicationSystemServerClientTestFixture
{
public:
	void UpdateAndVerifyThatAllInternalIndicesAreReleased(uint32 MaxIterattions = 3)
	{
		for (uint32 It = 0U; It < MaxIterattions; ++It)
		{
			Server->UpdateAndSend({Clients});
		}
		
		bool bHasInternalObjectsPendingDestroy = false;
		// Verify that we have released all internal objects and have no pending references.
		{
			UE::Net::Private::FNetRefHandleManager* NetRefHandleManager = &Server->ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
			UE_NET_ASSERT_EQ(NetRefHandleManager->GetObjectsPendingDestroy().Num(), 0);
		}

		for (FReplicationSystemTestClient* Client : Clients)
		{
			UE::Net::Private::FNetRefHandleManager* NetRefHandleManager = &Client->ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
			UE_NET_ASSERT_EQ(NetRefHandleManager->GetObjectsPendingDestroy().Num(), 0);
		}
	}
};

// Test TearOff for existing confirmed object
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTearOffExistingObject)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Trigger replication
	ServerObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Store Pointer to object 
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	UE_NET_ASSERT_TRUE(ClientObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);

	// Modify the value
	ServerObject->IntA = 2;

	// TearOff the object
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that ClientObject is torn-off and that the final state was applied
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
}

// Test TearOff for new object
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTearOffOnNewlyCreatedObject)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// We should not have any created objects
	const int32 NumObjectsCreatedOnClientBeforeReplication = Client->CreatedObjects.Num();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Set state
	ServerObject->IntA = 1;

	// TearOff the object before first replication
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Client should have created a object
	UE_NET_ASSERT_EQ(NumObjectsCreatedOnClientBeforeReplication + 1, Client->CreatedObjects.Num());

	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);

	// We should be able to get the object from the created objects array to validate the state
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->CreatedObjects[NumObjectsCreatedOnClientBeforeReplication].Get());

	// Verify that we replicated the expected state
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);
}

// Test TearOff resend for existing confirmed object with no state changes
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTearOffResendForExistingObjectWithoutDirtyState)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Trigger replication
	ServerObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Store Pointer to object 
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	UE_NET_ASSERT_NE(ClientObjectThatWillBeTornOff, nullptr);
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);

	// TearOff the object
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Send and do not deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DoNotDeliverPacket);
	Server->PostSendUpdate();

	// The ClientObject should still be found using the NetRefHandle
	UE_NET_ASSERT_NE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)), nullptr);

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that ClientObject is torn-off
	UE_NET_ASSERT_EQ(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)), nullptr);
}

// Test TearOff for new object and resend, this requires creation info to be cached.
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTearOffImmediateOnNewlyCreatedObjectResend)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// We should not have any created objects
	const int32 NumObjectsCreatedOnClientBeforeReplication = Client->CreatedObjects.Num();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Set state
	ServerObject->IntA = 1;

	// TearOff the object before first replication
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Send and do not deliver packet
	Server->UpdateAndSend({Client}, false);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Client should have created a object
	UE_NET_ASSERT_EQ(NumObjectsCreatedOnClientBeforeReplication + 1, Client->CreatedObjects.Num());

	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);

	// We should be able to get the object from the created objects array to validate the state
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->CreatedObjects[NumObjectsCreatedOnClientBeforeReplication].Get());

	// Verify that we replicated the expected state
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);
}

// Test TearOff for new object and resend if not fitting in packet, this requires creation info to be cached and that we retain object in scope
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTearOffImmediateOnNewlyCreatedObjectIfNotSentOnFirstUpdate)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// We should not have any created objects
	const int32 NumObjectsCreatedOnClientBeforeReplication = Client->CreatedObjects.Num();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Set state
	ServerObject->IntA = 1;

	// TearOff the object before first replication
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Send and do not deliver packet
	Server->UpdateAndSend({Client}, false);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Client should have created a object
	UE_NET_ASSERT_EQ(NumObjectsCreatedOnClientBeforeReplication + 1, Client->CreatedObjects.Num());

	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);

	// We should be able to get the object from the created objects array to validate the state
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->CreatedObjects[NumObjectsCreatedOnClientBeforeReplication].Get());

	// Verify that we replicated the expected state
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);
}

// Test TearOff for new subobject and resend, this requires creation info to be cached.
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTearOffImmediateOnNewlyCreatedSubObjectResend)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// We should not have any created objects
	const int32 NumObjectsCreatedOnClientBeforeReplication = Client->CreatedObjects.Num();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Set state
	ServerObject->IntA = 1;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	// Set state
	ServerSubObject->IntA = 1;

	// TearOff the subobject before first replication
	Server->ReplicationBridge->EndReplication(ServerSubObject, EEndReplicationFlags::TearOff);

	// Send and drop
	Server->UpdateAndSend({Client}, false);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Client should have created a object + subobject
	UE_NET_ASSERT_EQ(NumObjectsCreatedOnClientBeforeReplication + 2, Client->CreatedObjects.Num());

	// But as we have torn off the subobject it should no longer be a replicated object
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle)) == nullptr);

	// We should be able to get the object from the created objects array to validate the state
	UTestReplicatedIrisObject* ClientSubObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->CreatedObjects[NumObjectsCreatedOnClientBeforeReplication + 1].Get());

	// Verify that we replicated the expected state
	UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObjectThatWillBeTornOff->IntA);
}

UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestDefferedTearOffOnNewlyCreatedObjectResend)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// We should not have any created objects
	const int32 NumObjectsCreatedOnClientBeforeReplication = Client->CreatedObjects.Num();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Set state
	ServerObject->IntA = 1;

	// TearOff the object before first replication
	Server->ReplicationSystem->TearOffNextUpdate(ServerObject->NetRefHandle);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, false);
	Server->PostSendUpdate();

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();
	
	// End replication and destroy object
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Destroy);

	// Client should have created a object
	UE_NET_ASSERT_EQ(NumObjectsCreatedOnClientBeforeReplication + 1, Client->CreatedObjects.Num());

	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);

	// We should be able to get the object from the created objects array to validate the state
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->CreatedObjects[NumObjectsCreatedOnClientBeforeReplication].Get());

	// Verify that we replicated the expected state
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);
}

// Test TearOff for existing not yet confirmed object
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTearOffObjectPendingCreateConfirmation)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Trigger replication
	ServerObject->IntA = 1;

	// Send packet to get put the object in flight
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// TearOff the object
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Deliver Object (should now be created)
	Server->DeliverTo(Client, true);

	// Store Pointer to object and verify initial state
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	UE_NET_ASSERT_TRUE(ClientObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that ClientObject is TornOFf and that the final state was applied
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
}

// Test TearOff for existing object pending destroy (should do nothing)
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTearOffExistingObjectPendingDestroy)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Trigger replication
	ServerObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Store Pointer to object 
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	UE_NET_ASSERT_TRUE(ClientObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);

	// Modify the value
	ServerObject->IntA = 2;

	// Mark the object for destroy
	Server->ReplicationBridge->EndReplication(ServerObject);

	// TearOff the object
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that ClientObject is not tornOff and that the final state was not applied as we issued tearoff after ending replication
	UE_NET_ASSERT_NE(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
}

// Test TearOff resend 
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTearOffResend)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Trigger replication
	ServerObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Store Pointer to object 
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	UE_NET_ASSERT_TRUE(ClientObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);

	// Modify the value
	ServerObject->IntA = 2;

	// TearOff the object
	Server->ReplicationSystem->TearOffNextUpdate(ServerObject->NetRefHandle);

	// Send and deliver packet, in this case the packet containing 2 was lost, but, we did not know that when we 
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, false);
	Server->PostSendUpdate();

	// Object should now be torn-off, so it should not copy the latest state
	ServerObject->IntA = 3;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that ClientObject is torn-off and that the expected final state was applied
	UE_NET_ASSERT_EQ(2, ClientObjectThatWillBeTornOff->IntA);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
}

// Test TearOff does not pickup statechanges after tear off
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTornedOffObjectDoesNotCopyStateChanges)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Trigger replication
	ServerObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Store Pointer to object 
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	UE_NET_ASSERT_TRUE(ClientObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);

	// Modify the value
	ServerObject->IntA = 2;

	// TearOff the object
	Server->ReplicationSystem->TearOffNextUpdate(ServerObject->NetRefHandle);

	// Send and drop packet containing the value 2
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, false);
	Server->PostSendUpdate();

	// Object should now be torn-off, so it should not copy the latest state but instead resend the last copied state (2) along with the tear-off
	ServerObject->IntA = 3;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that ClientObject is TornOFf and that the expected final state was applied
	UE_NET_ASSERT_EQ(2, ClientObjectThatWillBeTornOff->IntA);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
}

// Test TearOff and SubObjects, SubObjects must apply state?
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestImmediateTearOffExistingObjectWithSubObject)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	// Trigger replication
	ServerObject->IntA = 1;
	ServerSubObject->IntA = 1;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Store Pointer to objects
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);

	UTestReplicatedIrisObject* ClientSubObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientSubObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObjectThatWillBeTornOff->IntA);

	// Modify the value of subobject only
	ServerSubObject->IntA = 2;

	// TearOff the object using immediate tear-off
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that ClientObject is torn-off and that the final state was applied to subObject 
	UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObjectThatWillBeTornOff->IntA);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle) == nullptr);

	// Verify that we have released all internal objects and have no pending references.
	UpdateAndVerifyThatAllInternalIndicesAreReleased();
}

/**
 * Below follows a series of tests that ensure that we do not leak internal net objects when doing tear off and flush in combination with subobject filtering
 */
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestAddCondNeverSubObjectAndTearOffSameFrame)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	Server->GetReplicationBridge()->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_Never);

	// Store Pointer to objects
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle) != nullptr);

	// TearOff the object using immediate tear-off
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we cannot find the objects on the client
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle) == nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle) == nullptr);

	// Verify that we have released all internal objects and have no pending references.
	UpdateAndVerifyThatAllInternalIndicesAreReleased();
}

UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestAddCondNeverSubObjectAndTearOffSameFrameDelayBeforeSend)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	Server->GetReplicationBridge()->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_Never);

	// Store Pointer to objects
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle) != nullptr);

	// TearOff the object using immediate tear-off
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Do a few updates without sending
	Server->UpdateAndSend({});
	Server->UpdateAndSend({});

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we cannot find the objects on the client
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle) == nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle) == nullptr);

	// Verify that we have released all internal objects and have no pending references.
	UpdateAndVerifyThatAllInternalIndicesAreReleased();
}

UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestAddCondNeverSubObjectAndTearOffSameFrameLostPacket)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	Server->GetReplicationBridge()->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_Never);

	// Store Pointer to objects
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle) != nullptr);

	// TearOff the object using immediate tear-off
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Send and drop packet
	Server->UpdateAndSend({Client}, /* bDeliver = */ false);

	// Update without sending
	Server->UpdateAndSend({});

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we cannot find the objectson the client
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle) == nullptr);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle) == nullptr);

	// Verify that we have released all internal objects and have no pending references.
	UpdateAndVerifyThatAllInternalIndicesAreReleased();
}

UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestAddCondNeverSubObjectAndDestroyRootWithFlushRootSameFrame)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	Server->GetReplicationBridge()->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_Never);

	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we have released all internal objects and have no pending references.
	UpdateAndVerifyThatAllInternalIndicesAreReleased();
}

UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestCreateRootAndCondNeverSubObjectAndDestroyRootWithFlushRootSameFrame)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::VeryVerbose);

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	Server->GetReplicationBridge()->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_Never);

	// Send and deliver packet
	Server->UpdateAndSend({});

	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Send and deliver packet
	Server->UpdateAndSend({});

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we have released all internal objects and have no pending references.
	UpdateAndVerifyThatAllInternalIndicesAreReleased();
}

UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestAddCondNeverSubObjectAndDestroyRootWithFlushRootSameFrameDelayBeforeSend)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	Server->GetReplicationBridge()->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_Never);

	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Do a few updates without sending
	Server->UpdateAndSend({});

	// Verify that we have released all internal objects and have no pending references.
	UpdateAndVerifyThatAllInternalIndicesAreReleased();
}

// Verify that we release all references to filtered out subobjects
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestAddCondNeverSubObjectAndDestroyRootWithFlushRootSameFrameLostPacket)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	Server->GetReplicationBridge()->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_Never);

	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Send and drop packet
	Server->UpdateAndSend({Client}, /* bDeliver = */ false);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we have released all internal objects and have no pending references.
	UpdateAndVerifyThatAllInternalIndicesAreReleased();
}

UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestFilterOutCreatedCondNeverSubObjectAndDestroyRootWithFlush)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that subobject exists
	UTestReplicatedIrisObject* ClientSubObjectThatWillBeDestroyedWithFlush = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientSubObjectThatWillBeDestroyedWithFlush != nullptr);

	// Now filter out the subobject, as it was created before being filtered out it will be destroyed
	Server->GetReplicationBridge()->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_Never);

	// Just dirty it
	++ServerSubObject->IntA;

	// End replication of root.
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Verify that we have released all internal objects and have no pending references.
	UpdateAndVerifyThatAllInternalIndicesAreReleased();

	// Verify that we did not get the final value as object was filtered out, we get away with this due the the fact that we keep all objects created on the client until GC
	UE_NET_ASSERT_NE(ClientSubObjectThatWillBeDestroyedWithFlush->IntA, ServerSubObject->IntA);
}

UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestFilterOutCreatedCondNeverSubObjectAndTearOff)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that subobject exists
	UTestReplicatedIrisObject* ClientSubObjectThatWillBeDestroyedWithFlush = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientSubObjectThatWillBeDestroyedWithFlush != nullptr);

	// Now filter out the subobject, but as we have created it must still replicate it with the tear-off.
	Server->GetReplicationBridge()->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_Never);

	// Just dirty it
	++ServerSubObject->IntA;

	// End replication of root.
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that we have released all internal objects and have no pending references.
	UpdateAndVerifyThatAllInternalIndicesAreReleased();

	// Verify that we did not get the final value as object was filtered out, we get away with this due the the fact that we keep all objects created on the client until GC
	UE_NET_ASSERT_NE(ClientSubObjectThatWillBeDestroyedWithFlush->IntA, ServerSubObject->IntA);
}

UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestFilterOutCondNeverSubObjectAndRenabledBeforeDestroyRootWithFlush)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::VeryVerbose);

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	// Filter out the subobject
	Server->GetReplicationBridge()->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_Never);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that subobject does not exists
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle) == nullptr);

	// Re-enable filtered out the subobject
	Server->GetReplicationBridge()->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_None);

	// End replication of root.
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that subobject exists
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle) != nullptr);

	Server->UpdateAndSend({Client});

	// Verify that subobject does not exists
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle) == nullptr);

	// Verify that we have released all internal objects and have no pending references.
	UpdateAndVerifyThatAllInternalIndicesAreReleased();
}

UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestFilterOutCondNeverSubObjectAndRenabledBeforeTearOff)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	// Filter out the subobject
	Server->GetReplicationBridge()->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_Never);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that subobject does not exists
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle) == nullptr);

	// Re-enable filtered out the subobject
	Server->GetReplicationBridge()->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_None);

	// Set a value that we can find to verify that the SubObject included state
	ServerSubObject->IntA = 3;

	// End replication of root.
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	const int32 PreUpdateCreatedObjectCount = Client->CreatedObjects.Num();

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	const int32 PostUpdateCreatedObjectCount = Client->CreatedObjects.Num();
	UE_NET_ASSERT_GT(PostUpdateCreatedObjectCount, PreUpdateCreatedObjectCount);

	// We expect the SubObject to have been replicated so it should exist in the persistent CreatedObjects array on the client
	UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->CreatedObjects[PostUpdateCreatedObjectCount - 1].Get());
	UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObject->IntA);

	// Verify that subobject does not exists as a replicated object
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle) == nullptr);

	// Verify that we have released all internal objects and have no pending references.
	UpdateAndVerifyThatAllInternalIndicesAreReleased();
}

// Test TearOff and SubObjects, SubObjects must apply state?
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestImmediateTearOffExistingObjectWithSubObjectDroppedData)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	// Trigger replication
	ServerObject->IntA = 1;
	ServerSubObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Store Pointer to objects
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);

	UTestReplicatedIrisObject* ClientSubObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientSubObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObjectThatWillBeTornOff->IntA);

	// Modify the value of subobject only
	ServerSubObject->IntA = 2;

	// TearOff the object using immediate tear-off
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Send and do not deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, false);
	Server->PostSendUpdate();

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that ClientObject is torn-off and that the final state was applied to subObject 
	UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObjectThatWillBeTornOff->IntA);
	UE_NET_ASSERT_TRUE(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle) == nullptr);
}

// Test creating a subobject after the root is already torn off
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestCreatingSubobjectAfterRootObjectTearOff)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with subobject on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UTestReplicatedIrisObject* ServerSubObject0 = Server->CreateSubObject(ServerObject->NetRefHandle);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// TearOff the object
	Server->ReplicationSystem->TearOffNextUpdate(ServerObject->NetRefHandle);

	// Send and deliver packet
	Server->NetUpdate();
	Server->PostSendUpdate();

	{
		FTestEnsureScope EnsureScope;
		UTestReplicatedIrisObject* ServerSubObject1 = nullptr;
		// Add a new subobject. This shouldn't become a replicated subobject of the already torn off root object.
		{
			LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Error);
			ServerSubObject1 = Server->CreateSubObject(ServerObject->NetRefHandle);

			// We expect an ensure here, as the object should not longer be considered replicated.
			// Cannot really rely on ensure counts as other tests might affect it if the same ensure has already been triggered...
			//UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 1);

			// The subobject should not start replicating.
			UE_NET_ASSERT_FALSE(ServerSubObject1->NetRefHandle.IsValid());
		}
	}

	// Send and deliver packet
	Server->UpdateAndSend({ Client });

}

// Test dropped creation of subobject dirties owner
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestDroppedCreationForSubobjectDirtiesOwner)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Store Pointer to objects
	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientObject != nullptr);

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	// Send and do not deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, false);
	Server->PostSendUpdate();

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that ClientObject now is created as expected
	UTestReplicatedIrisObject* ClientSubObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientSubObject != nullptr);
}

// Test replicated destroy for not created object
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestReplicatedDestroyForNotCreatedObject)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Update and delay delivery
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Destroy object
	Server->ReplicationBridge->EndReplication(ServerObject);

	// Update and delay delivery
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Drop first packet containing creation info for object
	Server->DeliverTo(Client, false);

	// Deliver second packet that should contain destroy
	Server->DeliverTo(Client, true);

	// Verify that the object does not exist on client
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
}


// Test replicated SubObjectDestroy for not created subobject
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestReplicatedSubObjectDestroyForNotCreatedObject)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Replicate object
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObjectWithNoReplicatedMembers* ServerSubObject = Server->CreateSubObject<UTestReplicatedIrisObjectWithNoReplicatedMembers>(ServerObject->NetRefHandle);

	// Update and delay delivery
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Destroy subobject
	Server->ReplicationBridge->EndReplication(ServerSubObject);

	// Update and delay delivery
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Drop first packet containing creation info for subobject
	Server->DeliverTo(Client, false);

	// Deliver second packet that should contain replicated subobject destroy
	Server->DeliverTo(Client, true);

	// Verify that the object still exists on client
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);

	// Verify that the subobject does not exist on client
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObjectWithNoReplicatedMembers>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle)) == nullptr);
}

// Test replicated SubObjectDestroy for filtered out subobject
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestReplicatedSubObjectDestroyForFilteredOutSubObject)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObjectWithNoReplicatedMembers* ServerSubObject = Server->CreateSubObject<UTestReplicatedIrisObjectWithNoReplicatedMembers>(ServerObject->NetRefHandle);

	// Replicate object
	Server->UpdateAndSend({Client});

	// Verify that the subobject does  exist on client
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObjectWithNoReplicatedMembers>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle)) != nullptr);

	// Set condition
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_Never);

	// Destroy subobject
	Server->ReplicationBridge->EndReplication(ServerSubObject);

	// Replicate object
	Server->UpdateAndSend({Client});

	// Verify that the object still exists on client
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);

	// Verify that the subobject does not exist on client
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObjectWithNoReplicatedMembers>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle)) == nullptr);
}

// Test replicated SubObjectDestroy for filtered out subobject
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestReplicatedSubObjectDestroyForFilteredOutSubObjectBeforeSend)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObjectWithNoReplicatedMembers* ServerSubObject = Server->CreateSubObject<UTestReplicatedIrisObjectWithNoReplicatedMembers>(ServerObject->NetRefHandle);

	// Set condition
	Server->ReplicationBridge->SetSubObjectNetCondition(ServerSubObject->NetRefHandle, ELifetimeCondition::COND_Never);

	// Replicate object
	Server->UpdateAndSend({Client});

	// Verify that the subobject does not exist on client
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObjectWithNoReplicatedMembers>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle)) == nullptr);

	// Destroy subobject
	Server->ReplicationBridge->EndReplication(ServerSubObject);

	// Replicate object
	Server->UpdateAndSend({Client});

	// Verify that the object still exists on client
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);

	// Verify that the subobject does not exist on client
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObjectWithNoReplicatedMembers>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle)) == nullptr);
}


// Test tear-off object in PendingCreate state to ensure that tear-off logic works as expected
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTearOffObjectWithNoFragmentsDoesNotTriggerCheckIfPendingCreateWhenDestroyed)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObjectWithNoReplicatedMembers* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithNoReplicatedMembers>();

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, false);
	Server->PostSendUpdate();

	// Tear-off using immediate tear-off
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Trigger the next update but avoid sending any data so that we keep the object in the PendingCreation state while we flush the Handles PendingTearOff Array which occurs in PostSendUpdate
	Server->NetUpdate();
	Server->PostSendUpdate();
}

// Test tear-off subobject in PendingCreate state to ensure that tear-off logic works as expected
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTearOffSubObjectWithNoFragmentsDoesNotTriggerCheckIfPendingCreateWhenDestroyed)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObjectWithNoReplicatedMembers* ServerSubObject = Server->CreateSubObject<UTestReplicatedIrisObjectWithNoReplicatedMembers>(ServerObject->NetRefHandle);

	// Update and drop
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, false);
	Server->PostSendUpdate();

	// Tear-off using immediate tear-off
	Server->ReplicationBridge->EndReplication(ServerSubObject, EEndReplicationFlags::TearOff);

	// Trigger the next update but avoid sending any data so that we keep the sub-object in the PendingCreation state while we flush the Handles PendingTearOff Array which occurs in PostSendUpdate
	Server->NetUpdate();
	Server->PostSendUpdate();
}

// Test TearOff and SubObjects, SubObjects must apply state?
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTearOffNextUpdateExistingObjectWithSubObject)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	// Trigger replication
	ServerObject->IntA = 1;
	ServerSubObject->IntA = 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Store Pointer to objects
	UTestReplicatedIrisObject* ClientObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerObject->IntA, ClientObjectThatWillBeTornOff->IntA);

	UTestReplicatedIrisObject* ClientSubObjectThatWillBeTornOff = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientSubObjectThatWillBeTornOff != nullptr);
	UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObjectThatWillBeTornOff->IntA);

	// Modify the value of subobject only
	ServerSubObject->IntA = 2;

	// TearOff the object using immediate tear-off
	Server->ReplicationSystem->TearOffNextUpdate(ServerObject->NetRefHandle);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that ClientObject is torn-off and that the final state was applied to subObject 
	UE_NET_ASSERT_EQ(ServerSubObject->IntA, ClientSubObjectThatWillBeTornOff->IntA);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle)) == nullptr);
}

// Test TearOff and destroy 
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTearOffNextUpdateWithUnexpectedDestroyObject)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	FNetRefHandleManager* NetRefHandleManager = &ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Create a pushbased object on server
	UTestReplicatedIrisPushModelObject* ServerObject = NewObject<UTestReplicatedIrisPushModelObject>();
	ServerObject->NetRefHandle = Server->ReplicationBridge->BeginReplication(ServerObject);

	const FInternalNetRefIndex ServerObjectInternalIndex = NetRefHandleManager->GetInternalIndex(ServerObject->NetRefHandle);

	Server->UpdateAndSend({Client});

	// Verify that we no longer have any references to the object
	UE_NET_ASSERT_EQ(uint16(1), NetRefHandleManager->GetNetObjectRefCount(ServerObjectInternalIndex));

	// Initiate tear-off
	Server->ReplicationSystem->TearOffNextUpdate(ServerObject->NetRefHandle);

	// Put tear-off in flight
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Destroy object in a bad way.
	ServerObject->MarkAsGarbage();
	constexpr bool bPerformFullPurge = false;
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, bPerformFullPurge);

	Server->DeliverTo(Client, true);

	// As we have issued tear-off, this should not cause an ensure
	{
		FEnsureScope EnsureScope;

		Server->UpdateAndSend({Client});

		UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 0);
	}

	// Verify that we no longer have any references to the object
	UE_NET_ASSERT_EQ(uint16(0), NetRefHandleManager->GetNetObjectRefCount(ServerObjectInternalIndex));
}

// Test TearOff and destroy of SubObjects that are still pending create/tearoff
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTearOffNextUpdateWithUnexpectedDestroySubObject)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	FNetRefHandleManager* NetRefHandleManager = &ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Spawn second object on server as a subobject without adding a strong reference
	UTestReplicatedIrisObject* ServerSubObject = NewObject<UTestReplicatedIrisObject>();
	ServerSubObject->NetRefHandle = Server->ReplicationBridge->BeginReplication(ServerObject->NetRefHandle, ServerSubObject);
	
	const FInternalNetRefIndex ServerObjectInternalIndex = NetRefHandleManager->GetInternalIndex(ServerObject->NetRefHandle);
	const FInternalNetRefIndex SubObjectObjectInternalIndex = NetRefHandleManager->GetInternalIndex(ServerSubObject->NetRefHandle);

	Server->UpdateAndSend({Client});

	// TearOff the object this will also tear-off subobject
	Server->ReplicationSystem->TearOffNextUpdate(ServerObject->NetRefHandle);

	// Just logic update, object will be pending tearoff.
	Server->UpdateAndSend({});

	// Destroy subobject in a bad way.
	ServerSubObject->MarkAsGarbage();
	constexpr bool bPerformFullPurge = false;
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, bPerformFullPurge);

	// As we have issued tear-off, this should not cause an ensure
	{
		FEnsureScope EnsureScope;

		Server->UpdateAndSend({Client});

		UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 0);
	}

	// Verify that we no longer have any references to the objects
	UE_NET_ASSERT_EQ(uint16(0), NetRefHandleManager->GetNetObjectRefCount(ServerObjectInternalIndex));
	UE_NET_ASSERT_EQ(uint16(0), NetRefHandleManager->GetNetObjectRefCount(SubObjectObjectInternalIndex));
}

// Test TearOff and destroy of SubObjects that are still pending create/tearoff
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestImmediateTearOffWithUnexpectedDestroySubObject)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	FNetRefHandleManager* NetRefHandleManager = &ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Explicit spawn of second object on server as a subobject to avoid hard refernce
	UTestReplicatedIrisObject* ServerSubObject = NewObject<UTestReplicatedIrisObject>();
	FNetRefHandle SubObjectHandle = Server->ReplicationBridge->BeginReplication(ServerObject->NetRefHandle, ServerSubObject);
	ServerSubObject->NetRefHandle = SubObjectHandle;

	const FInternalNetRefIndex ServerObjectInternalIndex = NetRefHandleManager->GetInternalIndex(ServerObject->NetRefHandle);
	const FInternalNetRefIndex SubObjectObjectInternalIndex = NetRefHandleManager->GetInternalIndex(ServerSubObject->NetRefHandle);

	Server->UpdateAndSend({Client});

	// Immediate tearoff
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::TearOff);

	// Destroy subobject in a bad way.
	ServerSubObject->MarkAsGarbage();
	constexpr bool bPerformFullPurge = false;
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, bPerformFullPurge);

	// As we have issued tear-off, this should not cause an ensure
	{
		FEnsureScope EnsureScope;

		Server->UpdateAndSend({Client});

		UE_NET_ASSERT_EQ(EnsureScope.GetCount(), 0);
	}

	// Verify that we no longer have any references to the objects
	UE_NET_ASSERT_EQ(uint16(0), NetRefHandleManager->GetNetObjectRefCount(ServerObjectInternalIndex));
	UE_NET_ASSERT_EQ(uint16(0), NetRefHandleManager->GetNetObjectRefCount(SubObjectObjectInternalIndex));
}

// Test TearOff and destroy of SubObjects that are still pending create/tearoff
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestTearOffNextUpdateExistingObjectWithSubObjectPendingCreation)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	FNetRefHandleManager* NetRefHandleManager = &ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);

	// Spawn second object on server as a subobject
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);

	// Trigger replication
	ServerObject->IntA = 1;
	ServerSubObject->IntA = 1;

	const FInternalNetRefIndex ServerObjectInternalIndex = NetRefHandleManager->GetInternalIndex(ServerObject->NetRefHandle);
	const FInternalNetRefIndex SubObjectObjectInternalIndex = NetRefHandleManager->GetInternalIndex(ServerSubObject->NetRefHandle);

	// Trigger presend without send to add the objects to scope
	Server->NetUpdate();
	Server->PostSendUpdate();

	UE_NET_ASSERT_EQ(uint16(1), NetRefHandleManager->GetNetObjectRefCount(ServerObjectInternalIndex));
	UE_NET_ASSERT_EQ(uint16(1), NetRefHandleManager->GetNetObjectRefCount(SubObjectObjectInternalIndex));

	// TearOff the object this will also tear-off subobject
	Server->ReplicationSystem->TearOffNextUpdate(ServerObject->NetRefHandle);

	// Update logic, object should be removed from scope but still exist as pending create in
	Server->NetUpdate();
	Server->PostSendUpdate();

	UE_NET_ASSERT_EQ(uint16(1), NetRefHandleManager->GetNetObjectRefCount(ServerObjectInternalIndex));
	UE_NET_ASSERT_EQ(uint16(1), NetRefHandleManager->GetNetObjectRefCount(SubObjectObjectInternalIndex));

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Destroy the object
	Server->DestroyObject(ServerObject);

	// Verify that we no longer have any references to the object
	UE_NET_ASSERT_EQ(uint16(0), NetRefHandleManager->GetNetObjectRefCount(ServerObjectInternalIndex));
	UE_NET_ASSERT_EQ(uint16(0), NetRefHandleManager->GetNetObjectRefCount(SubObjectObjectInternalIndex));
}

// Test that we can replicate an object with no replicated properties
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestReplicatedObjectWithNoReplicatedProperties)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObjectWithNoReplicatedMembers* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithNoReplicatedMembers>();
	const FNetRefHandle ServerHandle = ServerObject->NetRefHandle;

	UE_NET_ASSERT_TRUE(ServerHandle.IsValid());

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	UTestReplicatedIrisObjectWithNoReplicatedMembers* ClientObject = Cast<UTestReplicatedIrisObjectWithNoReplicatedMembers>(Client->GetReplicationBridge()->GetReplicatedObject(ServerHandle));
	UE_NET_ASSERT_TRUE(ClientObject != nullptr);

	// Destroy object
	Server->DestroyObject(ServerObject);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	ClientObject = Cast<UTestReplicatedIrisObjectWithNoReplicatedMembers>(Client->GetReplicationBridge()->GetReplicatedObject(ServerHandle));
	UE_NET_ASSERT_TRUE(ClientObject == nullptr);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestObjectPollFramePeriod)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0, 0);

	// Spawn second object on server that later will be added as a dependent object
	UObjectReplicationBridge::FRootObjectReplicationParams Params;
	Params.PollFrequency = Server->ConvertPollPeriodIntoFrequency(1U);
	UTestReplicatedIrisObject* ServerObjectPolledEveryOtherFrame = Server->CreateObject(Params);

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Store Pointer to objects and verify state after initial replication
	UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UTestReplicatedIrisObject* ClientObjectPolledEveryOtherFrame = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectPolledEveryOtherFrame->NetRefHandle));

	UE_NET_ASSERT_NE(ClientObjectPolledEveryOtherFrame, nullptr);
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
	UE_NET_ASSERT_EQ(ClientObjectPolledEveryOtherFrame->IntA, ServerObjectPolledEveryOtherFrame->IntA);

	// After two value updates it's expected that the polling occurs exactly one time for the object with poll frame period 1 (meaning every other frame).
	bool SlowPollObjectHasBeenEqual = false;
	bool SlowPollObjectHasBeenInequal = false;

	// Update values
	ServerObject->IntA += 1;
	ServerObjectPolledEveryOtherFrame->IntA += 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
	SlowPollObjectHasBeenEqual |= (ClientObjectPolledEveryOtherFrame->IntA == ServerObjectPolledEveryOtherFrame->IntA);
	SlowPollObjectHasBeenInequal |= (ClientObjectPolledEveryOtherFrame->IntA != ServerObjectPolledEveryOtherFrame->IntA);

	// Update values
	ServerObject->IntA += 1;
	ServerObjectPolledEveryOtherFrame->IntA += 1;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that both objects now are in sync
	UE_NET_ASSERT_EQ(ClientObject->IntA, ServerObject->IntA);
	SlowPollObjectHasBeenEqual |= (ClientObjectPolledEveryOtherFrame->IntA == ServerObjectPolledEveryOtherFrame->IntA);
	SlowPollObjectHasBeenInequal |= (ClientObjectPolledEveryOtherFrame->IntA != ServerObjectPolledEveryOtherFrame->IntA);

	UE_NET_ASSERT_TRUE(SlowPollObjectHasBeenEqual);
	UE_NET_ASSERT_TRUE(SlowPollObjectHasBeenInequal);
}

// Test that broken objects can be skipped by client
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestClientCanSkipBrokenObject)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestReplicatedIrisObject* ServerObjectA = Server->CreateObject(0,0);
	UTestReplicatedIrisObject* ServerObjectB = Server->CreateObject(0,0);

	{
		// Setup client to fail to create next remote object
		ServerObjectA->bForceFailToInstantiateOnRemote = true;

		// Suppress ensure that will occur due to failing to instantiate the object
		UReplicatedTestObjectBridge::FSuppressCreateInstanceFailedEnsureScope SuppressEnsureScope(*Client->GetReplicationBridge());

		// Disable error logging as we know we will fail.
		auto IrisLogVerbosity = UE_GET_LOG_VERBOSITY(LogIris);
		LogIris.SetVerbosity(ELogVerbosity::NoLogging);

		// Send and deliver packet
		Server->NetUpdate();
		Server->SendAndDeliverTo(Client, true);
		Server->PostSendUpdate();

		// Restore LogVerbosity
		LogIris.SetVerbosity(IrisLogVerbosity);
	}

	// We expect replication of ObjectA to have failed
	{
		UTestReplicatedIrisObject* ClientObjectA = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectA->NetRefHandle));
		UE_NET_ASSERT_TRUE(ClientObjectA == nullptr);
	}

	// ObjectB should have been replicated ok
	{
		UTestReplicatedIrisObject* ClientObjectB = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectB->NetRefHandle));
		UE_NET_ASSERT_TRUE(ClientObjectB != nullptr);
	}

	// Modify both objects to make them replicate again
	++ServerObjectA->IntA;
	++ServerObjectB->IntA;

	// Send and deliver packet to verify that client ignores the broken object
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// We expect replication of ObjectA to have failed
	{
		UTestReplicatedIrisObject* ClientObjectA = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectA->NetRefHandle));
		UE_NET_ASSERT_TRUE(ClientObjectA == nullptr);
	}

	// Filter out ObjectA to tell the client that the object has gone out of scope
	ReplicationSystem->AddToGroup(ReplicationSystem->GetNotReplicatedNetObjectGroup(), ServerObjectA->NetRefHandle);

	// Send and deliver packet, the client should now remove the broken object from the list of broken objects
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Enable replication of ObjectA again to try to replicate it to server now that it should succeed
	ReplicationSystem->RemoveFromGroup(ReplicationSystem->GetNotReplicatedNetObjectGroup(), ServerObjectA->NetRefHandle);

	// Set ObjectA to be able instantiate on client again
	ServerObjectA->bForceFailToInstantiateOnRemote = false;

	// Client should now be able to instantiate the object
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// We expect replication of ObjectA to have succeeded this time
	{
		UTestReplicatedIrisObject* ClientObjectA = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectA->NetRefHandle));
		UE_NET_ASSERT_TRUE(ClientObjectA != nullptr);
	}
}


// Test that PropertyReplication properly handles partial states during Apply
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestPartialDequantize)
{
	// Enable cvars to exercise path that store previous state for OnReps to make sure we exercise path that accumulate dirty changes so that we have a complete state.
	IConsoleVariable* CVarUsePrevReceivedStateForOnReps = IConsoleManager::Get().FindConsoleVariable(TEXT("net.Iris.UsePrevReceivedStateForOnReps"));
	check(CVarUsePrevReceivedStateForOnReps != nullptr && CVarUsePrevReceivedStateForOnReps->IsVariableBool());
	const bool bUsePrevReceivedStateForOnReps = CVarUsePrevReceivedStateForOnReps->GetBool();
	CVarUsePrevReceivedStateForOnReps->Set(true, ECVF_SetByCode);

	// Make sure we allow partial dequantize
	IConsoleVariable* CVarForceFullDequantizeAndApply = IConsoleManager::Get().FindConsoleVariable(TEXT("net.iris.ForceFullDequantizeAndApply"));
	check(CVarForceFullDequantizeAndApply != nullptr && CVarForceFullDequantizeAndApply->IsVariableBool());
	const bool bForceFullDequantizeAndApply = CVarForceFullDequantizeAndApply->GetBool();
	CVarForceFullDequantizeAndApply->Set(false, ECVF_SetByCode);

	ON_SCOPE_EXIT
	{
		// Restore cvars
		CVarUsePrevReceivedStateForOnReps->Set(bUsePrevReceivedStateForOnReps, ECVF_SetByCode);
		CVarForceFullDequantizeAndApply->Set(bForceFullDequantizeAndApply, ECVF_SetByCode);
	};

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestReplicatedObjectWithRepNotifies* ServerObjectA = Server->CreateObject<UTestReplicatedObjectWithRepNotifies>();
	
	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify assumptions
	// Object should exist on client and have default state
	UTestReplicatedObjectWithRepNotifies* ClientObjectA = Cast<UTestReplicatedObjectWithRepNotifies>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObjectA->NetRefHandle));
	UE_NET_ASSERT_NE(ClientObjectA, nullptr);

	UE_NET_ASSERT_EQ(ServerObjectA->IntA, ClientObjectA->IntA);
	UE_NET_ASSERT_EQ(ClientObjectA->PrevIntAStoredInOnRep, -1);
	UE_NET_ASSERT_EQ(ServerObjectA->IntB, ClientObjectA->IntB);
	UE_NET_ASSERT_EQ(ClientObjectA->PrevIntBStoredInOnRep, -1);
	UE_NET_ASSERT_EQ(ServerObjectA->IntC, ClientObjectA->IntC);

	// Modify only IntA
	ServerObjectA->IntA = 1;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify assumptions
	// Only IntA should have been modified
	UE_NET_ASSERT_EQ(ServerObjectA->IntA, ClientObjectA->IntA);
	UE_NET_ASSERT_EQ(ClientObjectA->PrevIntAStoredInOnRep, -1);
	UE_NET_ASSERT_EQ(ServerObjectA->IntB, ClientObjectA->IntB);
	UE_NET_ASSERT_EQ(ClientObjectA->PrevIntBStoredInOnRep, -1);
	UE_NET_ASSERT_EQ(ServerObjectA->IntC, ClientObjectA->IntC);

	// Modify only IntB
	ServerObjectA->IntB = 1;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify assumptions
	// Only IntA should have been modified
	UE_NET_ASSERT_EQ(ServerObjectA->IntA, ClientObjectA->IntA);
	UE_NET_ASSERT_EQ(ClientObjectA->PrevIntAStoredInOnRep, -1);

	UE_NET_ASSERT_EQ(ServerObjectA->IntB, ClientObjectA->IntB);
	UE_NET_ASSERT_EQ(ClientObjectA->PrevIntBStoredInOnRep, -1);
	UE_NET_ASSERT_EQ(ServerObjectA->IntC, ClientObjectA->IntC);

	// Modify only IntA
	ServerObjectA->IntA = 2;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify assumptions
	// IntA should have been modified, and if everything works correctly PrevIntAStoredInOnRep should be 1
	UE_NET_ASSERT_EQ(ServerObjectA->IntA, ClientObjectA->IntA);
	UE_NET_ASSERT_EQ(ClientObjectA->PrevIntAStoredInOnRep, 1);

	UE_NET_ASSERT_EQ(ServerObjectA->IntB, ClientObjectA->IntB);
	UE_NET_ASSERT_EQ(ClientObjectA->PrevIntBStoredInOnRep, -1);
	UE_NET_ASSERT_EQ(ServerObjectA->IntC, ClientObjectA->IntC);

	// Verify that we do not apply repnotifies if we do not receive data from server by modifying values on the client and verifying that they do not get overwritten
	ServerObjectA->IntB = 2;
	ClientObjectA->IntA = -1;
	ClientObjectA->PrevIntAStoredInOnRep = -1;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify assumptions, since we messed with IntA and PrevIntAStoredInOnRep locally they have the value we set but IntB should be updated according to replicated state
	UE_NET_ASSERT_NE(ServerObjectA->IntA, ClientObjectA->IntA);
	UE_NET_ASSERT_EQ(ClientObjectA->PrevIntAStoredInOnRep, -1);
	UE_NET_ASSERT_EQ(ServerObjectA->IntB, ClientObjectA->IntB);
	UE_NET_ASSERT_EQ(ClientObjectA->PrevIntBStoredInOnRep, 1);
	UE_NET_ASSERT_EQ(ServerObjectA->IntC, ClientObjectA->IntC);

}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestNetMetric)
{
	{
		FNetMetric Metric(50.0);
		UE_NET_ASSERT_TRUE(Metric.GetDataType() == FNetMetric::EDataType::Double);
	}

	{
		FNetMetric Metric(50.f);
		UE_NET_ASSERT_TRUE(Metric.GetDataType() == FNetMetric::EDataType::Double);
	}

	{
		FNetMetric Metric;
		UE_NET_ASSERT_TRUE(Metric.GetDataType() == FNetMetric::EDataType::None);
		float Value = 100.f;
		Metric.Set(Value);
		UE_NET_ASSERT_TRUE(Metric.GetDataType() == FNetMetric::EDataType::Double);
	}

	{
		FNetMetric Metric(5U);
		UE_NET_ASSERT_TRUE(Metric.GetDataType() == FNetMetric::EDataType::Unsigned);
	}

	{
		FNetMetric Metric;
		UE_NET_ASSERT_TRUE(Metric.GetDataType() == FNetMetric::EDataType::None);
		uint32 Value = 100U;
		Metric.Set(Value);
		UE_NET_ASSERT_TRUE(Metric.GetDataType() == FNetMetric::EDataType::Unsigned);
	}

	{
		FNetMetric Metric(-5);
		UE_NET_ASSERT_TRUE(Metric.GetDataType() == FNetMetric::EDataType::Signed);
	}

	{
		FNetMetric Metric;
		UE_NET_ASSERT_TRUE(Metric.GetDataType() == FNetMetric::EDataType::None);
		Metric.Set(5);
		UE_NET_ASSERT_TRUE(Metric.GetDataType() == FNetMetric::EDataType::Signed);
	}
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestReplicationRecordStarvation)
{
	IConsoleVariable* CVarReplicationRecordStarvationThreshold = IConsoleManager::Get().FindConsoleVariable(TEXT("net.Iris.ReplicationWriterReplicationRecordStarvationThreshold"));
	UE_NET_ASSERT_NE(CVarReplicationRecordStarvationThreshold, nullptr);
	UE_NET_ASSERT_TRUE(CVarReplicationRecordStarvationThreshold->IsVariableInt());
	const int32 PrevReplicationRecordStarvationThreshold = CVarReplicationRecordStarvationThreshold->GetInt();
	ON_SCOPE_EXIT
	{
		CVarReplicationRecordStarvationThreshold->Set(PrevReplicationRecordStarvationThreshold, ECVF_SetByCode);
	};
	
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Set starvation threshold to highest possible
	CVarReplicationRecordStarvationThreshold->Set(FReplicationRecord::MaxReplicationRecordCount, ECVF_SetByCode);

	// Consume one ReplicationRecord to enter starvation
	UReplicatedTestObject* FirstObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	const FNetRefHandle FirstObjectRefHandle = FirstObject->NetRefHandle;

	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Try creating a second object. This should not succeed but we won't be able to test until we've delivered packets. 
	UReplicatedTestObject* SecondObject = Server->CreateObject(UTestReplicatedIrisObject::FComponents{});
	const FNetRefHandle SecondObjectRefHandle = SecondObject->NetRefHandle;

	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	Server->DeliverTo(Client, DeliverPacket);
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(FirstObjectRefHandle));

	// The second packet, if any, should not allow object replication due to starvation.
	Server->DeliverTo(Client, DeliverPacket);
	UE_NET_ASSERT_FALSE(Client->IsResolvableNetRefHandle(SecondObjectRefHandle));

	// Now we should be able replicate objects again. Retry sending the second object.
	Server->NetUpdate();
	Server->SendUpdate(Client->ConnectionIdOnServer);
	Server->PostSendUpdate();

	// Try destroying the first object. This should not succeed. 
	Server->DestroyObject(FirstObject);

	// Deliver the attempt to create the second object and verify it exists on the client.
	Server->DeliverTo(Client, DeliverPacket);
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(SecondObjectRefHandle));

	// The second packet, if any, should not allow object destruction due to starvation.
	Server->DeliverTo(Client, DeliverPacket);
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(FirstObjectRefHandle));

	// Now we should be able to destroy objects again. Retry destroying the first object
	Server->UpdateAndSend({Client});
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(FirstObjectRefHandle));
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestSubObjectReplicatedDestroyBeforePostNetReceive)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UReplicatedTestObject* ServerObject = Server->CreateObject(0, 0);

	// Spawn some subobjects
	UReplicatedSubObjectDestroyOrderObject* ServerSubObject0 = Server->CreateSubObject<UReplicatedSubObjectDestroyOrderObject>(ServerObject->NetRefHandle);
	UReplicatedSubObjectDestroyOrderObject* ServerSubObject1 = Server->CreateSubObject<UReplicatedSubObjectDestroyOrderObject>(ServerObject->NetRefHandle);
	UReplicatedSubObjectDestroyOrderObject* ServerSubObject2 = Server->CreateSubObject<UReplicatedSubObjectDestroyOrderObject>(ServerObject->NetRefHandle);

	UReplicatedSubObjectDestroyOrderObject::RepOrderCounter = 0U;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that objects have replicated
	UReplicatedSubObjectDestroyOrderObject* ClientSubObject0 = Cast<UReplicatedSubObjectDestroyOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject0->NetRefHandle));
	UReplicatedSubObjectDestroyOrderObject* ClientSubObject1 = Cast<UReplicatedSubObjectDestroyOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject1->NetRefHandle));
	UReplicatedSubObjectDestroyOrderObject* ClientSubObject2 = Cast<UReplicatedSubObjectDestroyOrderObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject2->NetRefHandle));

	UE_NET_ASSERT_NE(ClientSubObject0, nullptr);
	UE_NET_ASSERT_NE(ClientSubObject1, nullptr);
	UE_NET_ASSERT_NE(ClientSubObject2, nullptr);

	// Verify that they have replicated in expected order
	UE_NET_ASSERT_EQ(ClientSubObject0->LastRepOrderCounter, 1U);
	UE_NET_ASSERT_GT(ClientSubObject1->LastRepOrderCounter, ClientSubObject0->LastRepOrderCounter);
	UE_NET_ASSERT_GT(ClientSubObject2->LastRepOrderCounter, ClientSubObject1->LastRepOrderCounter);	

	// setup a watch on the client
	ClientSubObject2->SetObjectExpectedToBeDestroyed(ClientSubObject1);

	// Dirty some data on server and destroy SubObject1
	++ServerSubObject2->IntA;
	Server->ReplicationBridge->EndReplication(ServerSubObject1, EEndReplicationFlags::Destroy);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify assumptions, replicated subobject destroy should have been issued before ClientSubObjects2 s call to PostNetReceive
	UE_NET_ASSERT_TRUE(ClientSubObject2->bObjectExistedInPreNetReceive);
	UE_NET_ASSERT_FALSE(ClientSubObject2->bObjectExistedInPostNetReceive);
	UE_NET_ASSERT_FALSE(ClientSubObject2->ObjectToWatch->IsValid());
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestStablyNamedObject)
{
	const TCHAR* StableObjectName = TEXT("StableObjectName");

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Create stably named object on both server and client
	UTestReplicatedIrisObject* ServerObject = Server->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName);
	Client->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName);

	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsValid());
	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsStatic());

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// We expect object to exist on both server and client
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
 
	// Stop/end replication using async path
	Server->ReplicationBridge->EndReplication(ServerObject);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// We expect object to no longer exist on client and server
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
}

// expect fail here as we try to replicate object before it is allowed
#if 0
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestStablyNamedObjectCannotReplicateWhilePendingDestroy)
{
	const TCHAR* StableObjectName = TEXT("StableObjectName");

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Create stably named object on both server and client
	UTestReplicatedIrisObject* ServerObject = Server->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName);
	Client->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName);

	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsValid());
	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsStatic());

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// We expect object to exist on both server and client
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
 
	// Stop replication
	Server->ReplicationBridge->EndReplication(ServerObject);

	// Try to restart replication	
	Server->ReplicationBridge->BeginReplication(ServerObject);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// We expect object to no longer exist on client and server
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
}
#endif

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestStablyNamedObjectAsyncStopReplication)
{
	const TCHAR* StableObjectName = TEXT("StableObjectName");

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Create stably named object on both server and client
	UTestReplicatedIrisObject* ServerObject = Server->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName);
	Client->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName);

	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsValid());
	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsStatic());

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// We expect object to exist on both server and client
	const UTestReplicatedIrisObject* ClientObject = Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(ClientObject != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);

	// Stop replication using async path
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::AllowAsyncStopReplication | EEndReplicationFlags::Destroy);
	
	// Should be pending stop replication
	UE_NET_ASSERT_TRUE(Server->ReplicationBridge->IsPendingStopReplication(ServerObject->NetRefHandle));
	// Still replicated until the async stop is completed.
	UE_NET_ASSERT_TRUE(Server->ReplicationBridge->IsReplicatedHandle(ServerObject->NetRefHandle));
	
	// This should cancel the async stop replication and keep replicating the object
	Server->ReplicationBridge->BeginReplication(ServerObject);

	// Should be pending stop replication
	UE_NET_ASSERT_FALSE(Server->ReplicationBridge->IsPendingStopReplication(ServerObject->NetRefHandle));

	// Send and deliver packet
	Server->UpdateAndSend({Client});
		
	// We expect object to exist on both server and client and that it should be the same instance as we had before.
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObject);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
 
	// Stop replication using async path
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::AllowAsyncStopReplication | EEndReplicationFlags::Destroy);
	UE_NET_ASSERT_TRUE(Server->ReplicationBridge->IsPendingStopReplication(ServerObject->NetRefHandle));

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// We expect object to no longer exist on client and server
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestStablyNamedObjectAsyncStopReplicationWithTwoClients)
{
	const TCHAR* StableObjectName = TEXT("StableObjectName");

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add clients
	CreateClient();
	CreateClient();

	// Create stably named object on both server and clients
	UTestReplicatedIrisObject* ServerObject = Server->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName);
	TArray<UTestReplicatedIrisObject*> ClientObjects;
	ClientObjects.Add(Clients[0]->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName));
	ClientObjects.Add(Clients[1]->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName));

	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsValid());
	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsStatic());

	// Send and deliver packet to first client but not to the other
	Server->UpdateAndSend({Clients[0]});

	// We expect object to exist on server and clients[0]
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[0]);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);

	// Stop replication using async path, intentionally not passing the destroy flag to keep instance
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::AllowAsyncStopReplication);

	// Should be pending stop replication
	UE_NET_ASSERT_TRUE(Server->ReplicationBridge->IsPendingStopReplication(ServerObject->NetRefHandle));
	// Still replicated until the async stop is completed.
	UE_NET_ASSERT_TRUE(Server->ReplicationBridge->IsReplicatedHandle(ServerObject->NetRefHandle));
	
	// This should cancel the async stop replication and keep replicating the object
	Server->ReplicationBridge->BeginReplication(ServerObject);

	// Should be pending stop replication
	UE_NET_ASSERT_FALSE(Server->ReplicationBridge->IsPendingStopReplication(ServerObject->NetRefHandle));

	// Send and deliver packet to both clients
	Server->UpdateAndSend(Clients);
		
	// We expect object to exist on server and both clients
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[0]);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[1]);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
 
	// Stop replication using async path, intentionally not passing the destroy flag to keep instance
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::AllowAsyncStopReplication);

	// Send and deliver packet to only client 0
	Server->UpdateAndSend({Clients[0]});

	// We expect object to no longer exist on clients[0] and server should be pending async stop replication
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) ==  ClientObjects[1]);
	UE_NET_ASSERT_TRUE(Server->ReplicationBridge->IsPendingStopReplication(ServerObject->NetRefHandle));

	// This should cancel the async stop replication and resume replicating the object, 
	Server->ReplicationBridge->BeginReplication(ServerObject);
	UE_NET_ASSERT_FALSE(Server->ReplicationBridge->IsPendingStopReplication(ServerObject->NetRefHandle));

	// Update and send to all 
	Server->UpdateAndSend(Clients);

	// Object should now be restored on both clients
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[0]);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[1]);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);

	// Stop replication using async path, intentionally not passing the destroy flag to keep instance
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::AllowAsyncStopReplication);

	// Update and send to all 
	Server->UpdateAndSend(Clients);

	// Send and deliver packet
	Server->UpdateAndSend(Clients);

	// We now expect object to be considered fully destroyed.
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestDynamicObjectAsyncStopReplication)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Create dynamic object
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject<UTestReplicatedIrisObject>();

	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsValid());
	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsDynamic());

	const FNetRefHandle OriginalServerHandle = ServerObject->NetRefHandle;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Stop replication using async path
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::AllowAsyncStopReplication | EEndReplicationFlags::Destroy);

	// Should be pending stop replication
	UE_NET_ASSERT_TRUE(Server->ReplicationBridge->IsPendingStopReplication(ServerObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Server->ReplicationBridge->IsReplicatedHandle(ServerObject->NetRefHandle));
	
	// This should cancel the async stop replication and keep replicating the object using the same handle
	Server->ReplicationBridge->BeginReplication(ServerObject);

	// Verify that we did not get a new handle
	UE_NET_ASSERT_EQ(OriginalServerHandle, ServerObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(Server->ReplicationBridge->IsReplicatedHandle(ServerObject->NetRefHandle));

	// Should be pending stop replication
	UE_NET_ASSERT_FALSE(Server->ReplicationBridge->IsPendingStopReplication(ServerObject->NetRefHandle));

	// Send and deliver packet
	Server->UpdateAndSend({Client});
		
	// We expect object to exist on both server and client
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
 
	// Stop replication using async path
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::AllowAsyncStopReplication | EEndReplicationFlags::Destroy);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Send and deliver packet, really to process delivery notifications.
	Server->UpdateAndSend({Client});

	// We expect object to no longer exist on client and server
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);

	// This should start replicating the object again using a new handle, as the object had finished the async stop replication.
	Server->ReplicationBridge->BeginReplication(ServerObject);

	// We should get a new handle for our dynamic object
	Server->ReplicationBridge->BeginReplication(ServerObject);

	UE_NET_ASSERT_NE(OriginalServerHandle, ServerObject->NetRefHandle);	
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestDynamicObjectAsyncStopReplicationForSubObject)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Create dynamic object
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject<UTestReplicatedIrisObject>();

	// Create subobject
	UReplicatedTestObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle, 0, 0);
	const FNetRefHandle ServerSubObjectRefHandle = ServerSubObject->NetRefHandle;

	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsValid());
	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsDynamic());

	const FNetRefHandle OriginalServerSubObjectHandle = ServerSubObject->NetRefHandle;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// We expect object to exist on both server and client
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UReplicatedTestObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UReplicatedTestObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle)) != nullptr);

	// Stop replication of subobject using async path
	Server->ReplicationBridge->EndReplication(ServerSubObject, EEndReplicationFlags::AllowAsyncStopReplication | EEndReplicationFlags::Destroy);

	// Should be pending stop replication
	UE_NET_ASSERT_TRUE(Server->ReplicationBridge->IsPendingStopReplication(ServerSubObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Server->ReplicationBridge->IsReplicatedHandle(ServerSubObject->NetRefHandle));
	
	// This should cancel the async stop replication and keep replicating the object using the same handle
	Server->ReplicationBridge->BeginReplication(ServerObject->NetRefHandle, ServerSubObject);

	// Verify that we did not get a new handle
	UE_NET_ASSERT_EQ(OriginalServerSubObjectHandle, ServerSubObject->NetRefHandle);
	UE_NET_ASSERT_TRUE(Server->ReplicationBridge->IsReplicatedHandle(ServerSubObject->NetRefHandle));

	// Should not be pending stop replication
	UE_NET_ASSERT_FALSE(Server->ReplicationBridge->IsPendingStopReplication(ServerSubObject->NetRefHandle));

	// Send and deliver packet
	Server->UpdateAndSend({Client});
		
	// We expect object to exist on both server and client
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UReplicatedTestObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UReplicatedTestObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle)) != nullptr);

	// Stop replication using async path
	Server->ReplicationBridge->EndReplication(ServerSubObject, EEndReplicationFlags::AllowAsyncStopReplication | EEndReplicationFlags::Destroy);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// We expect Object to still exist on client and server
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// We expect SubObject to no longer exist on client and server
	UE_NET_ASSERT_TRUE(Cast<UReplicatedTestObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle)) == nullptr);
	UE_NET_ASSERT_TRUE(Cast<UReplicatedTestObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerSubObject->NetRefHandle)) == nullptr);

	// This should start replicating the object again using a new handle, as the object have completed the async stop replication.
	Server->ReplicationBridge->BeginReplication(ServerObject->NetRefHandle, ServerSubObject);

	UE_NET_ASSERT_NE(OriginalServerSubObjectHandle, ServerSubObject->NetRefHandle);	
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestDynamicObjectAsyncStopReplicationWitTwoClients)
{
	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Create clients
	CreateClient();
	CreateClient();

	// Create dynamic object
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject<UTestReplicatedIrisObject>();

	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsValid());
	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsDynamic());

	const FNetRefHandle OriginalServerHandle = ServerObject->NetRefHandle;

	// Send and deliver packet to clients
	Server->UpdateAndSend(Clients);

	// The object should now exist on all clients and server
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	
	// Stop replication using async path
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::AllowAsyncStopReplication | EEndReplicationFlags::Destroy);
	// Should be pending stop replication
	UE_NET_ASSERT_TRUE(Server->ReplicationBridge->IsPendingStopReplication(ServerObject->NetRefHandle));

	// Send and deliver packet to client 0
	Server->UpdateAndSend({Clients[0]});

	// We expect object to exist on server and clients[1], but not on clients[0]
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	// Should be pending stop replication
	UE_NET_ASSERT_TRUE(Server->ReplicationBridge->IsPendingStopReplication(ServerObject->NetRefHandle));

	// This should cancel the async stop replication and keep replicating the object using the same handle
	Server->ReplicationBridge->BeginReplication(ServerObject);

	// Should still use the Original handle
	UE_NET_ASSERT_EQ(OriginalServerHandle, ServerObject->NetRefHandle);

	// Should not be pending stop replication
	UE_NET_ASSERT_FALSE(Server->ReplicationBridge->IsPendingStopReplication(ServerObject->NetRefHandle));

	// Send and deliver packet to clients
	Server->UpdateAndSend(Clients);

	// The object should now exist on all clients again
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);

	// Stop replication using async path
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::AllowAsyncStopReplication);

	// Send and deliver packet to client 0
	Server->UpdateAndSend({Clients[0]});

	// We expect object to no longer exist on client 0
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	// Should be pending stop replication
	UE_NET_ASSERT_TRUE(Server->ReplicationBridge->IsPendingStopReplication(ServerObject->NetRefHandle));

	// Send and deliver packet to all clients
	Server->UpdateAndSend(Clients);

	// Send and deliver packet to all clients
	Server->UpdateAndSend(Clients);

	// We expect object to no longer exist and no longer be pending end replication
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
	UE_NET_ASSERT_FALSE(Server->ReplicationBridge->IsPendingStopReplication(ServerObject->NetRefHandle));

	// This should start replicating the same object again using a new handle
	Server->ReplicationBridge->BeginReplication(ServerObject);
	UE_NET_ASSERT_NE(OriginalServerHandle, ServerObject->NetRefHandle);	

	// Send and deliver packet to all clients
	Server->UpdateAndSend(Clients);

	// New instance of dynamic object should exist on all clients
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestStablyNamedObjectStartStopStartReplicationWithTwoClients)
{
	//LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Verbose);

	const TCHAR* StableObjectName = TEXT("StableObjectName");

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add clients
	CreateClient();
	CreateClient();

	// Create stably named object on both server and clients
	UTestReplicatedIrisObject* ServerObject = Server->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName);
	TArray<UTestReplicatedIrisObject*> ClientObjects;
	ClientObjects.Add(Clients[0]->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName));
	ClientObjects.Add(Clients[1]->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName));

	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsValid());
	UE_NET_ASSERT_TRUE(ServerObject->NetRefHandle.IsStatic());

	// Set some data so that we can identify the "incarnation"
	ServerObject->IntA = 1;

	// Send and deliver packet to first client but not to the other
	Server->UpdateAndSend({Clients[0]});

	// We expect object to exist on server and clients[0]
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[0]);
	UE_NET_ASSERT_FALSE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[1]);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);

	// Stop replication, not asking for a destroy as we do not want the client to destroy the instances.
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::None);

	// Should no longer be considered replicated
	UE_NET_ASSERT_FALSE(Server->ReplicationBridge->IsReplicatedHandle(ServerObject->NetRefHandle));
	
	// Start replication again, but since we are starting to replicate an object already replicating to some clients there will have to ask the clients to conform the destroy before we can replicate the new instance.
	Server->ReplicationBridge->BeginReplication(ServerObject);

	// Set some data so that we can identify the "incarnation"
	ServerObject->IntA = 2;

	// Object should exist on server...but in its next incarnation, so replication will restart
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);

	// Send and deliver packet to both clients
	Server->UpdateAndSend(Clients);

	// As we already was replicating the object we need to wait for the clients to confirm this before we replicate a new instance of the same object,
	// In this case as we had replicated the old instance to client[0] but not to client[1] so client[0] will have the old instance be destroyed while the client[1] will receive the new instnace
	UE_NET_ASSERT_FALSE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[0]);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[1]);

	// Client[0] should have received the data from the first instance
	UE_NET_ASSERT_EQ(ClientObjects[0]->IntA, 1);
	// Client[1] should have received the data from got the latest instance
	UE_NET_ASSERT_EQ(ClientObjects[1]->IntA, 2);

	// Set some data so that we can identify the "incarnation"
	ServerObject->IntA = 3;
	
	// Send and deliver packet to both clients
	Server->UpdateAndSend(Clients);

	// New instance should now have been replicated to both clients
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[0]);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[1]);

	// Both clients are now expected to have received the latest state.
	UE_NET_ASSERT_EQ(ClientObjects[0]->IntA, 3);
	UE_NET_ASSERT_EQ(ClientObjects[1]->IntA, 3);
 
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::None);

	// Update and send to all 
	Server->UpdateAndSend(Clients);

	// Object should now be destroyed on all clients and server
	UE_NET_ASSERT_FALSE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[0]);
	UE_NET_ASSERT_FALSE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[1]);
	UE_NET_ASSERT_FALSE(Cast<UTestReplicatedIrisObject>(Server->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) != nullptr);
}

/** Test combinations of start / stop / start for stable object using flush. */
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestStableStartFlushStartStopStartReplication)
{
	const TCHAR* StableObjectName = TEXT("StableObjectName");

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add client
	CreateClient();

	// Create stably named object on both server and clients
	UTestReplicatedIrisObject* ServerObject = Server->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName);
	TArray<UTestReplicatedIrisObject*> ClientObjects;
	ClientObjects.Add(Clients[0]->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName));

	// Test multiple start stop during same frame
	Server->ReplicationBridge->BeginReplication(ServerObject);

	Server->UpdateAndSend(Clients);

	ServerObject->IntA = 33;

	// Stop replicating using flush
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Flush);

	// Start replicating again
	Server->ReplicationBridge->BeginReplication(ServerObject);

	ServerObject->IntA = 44;

	// Stop replication, without flush, we do not expect to see this state on the client
	Server->ReplicationBridge->EndReplication(ServerObject);

	// Start replicating again
	Server->ReplicationBridge->BeginReplication(ServerObject);

	ServerObject->IntA = 55;

	// Stop replicating using flush
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Flush);

	// Start replication again. As we have issued multiple start/stops on the same stable object requesting flush it will take a few updates for the final incarnation to be replicated.
	Server->ReplicationBridge->BeginReplication(ServerObject);
	ServerObject->IntA = 66;

	Server->UpdateAndSend(Clients);

	// We expect the first flushed incarnation to have been created
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[0]);
	UE_NET_ASSERT_EQ(ClientObjects[0]->IntA, 33);

	Server->UpdateAndSend(Clients);

	// Should be destroyed
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);

	Server->UpdateAndSend(Clients);

	// We expect the next flushed incarnation to have been created
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[0]);
	UE_NET_ASSERT_EQ(ClientObjects[0]->IntA, 55);

	Server->UpdateAndSend(Clients);

	// Should be destroyed
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);

	Server->UpdateAndSend(Clients);

	// We expect the final flushed incarnation to have been created
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[0]);
	UE_NET_ASSERT_EQ(ClientObjects[0]->IntA, 66);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestStablyNamedObjectMultiStartStopStartReplicationWithTwoClients)
{
	const TCHAR* StableObjectName = TEXT("StableObjectName");

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add clients
	CreateClient();

	// Create stably named object on both server and client
	UTestReplicatedIrisObject* ServerObject = Server->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName);
	TArray<UTestReplicatedIrisObject*> ClientObjects;
	ClientObjects.Add(Clients[0]->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName));

	// Test multiple start stop during same frame
	Server->ReplicationBridge->BeginReplication(ServerObject);

	// Update and send to all 
	Server->UpdateAndSend(Clients);

	ServerObject->IntA = 33;
	// As we use flush, state will be captured immediately, and connections already scoping object will initiate flush.
	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Flush);

	// As we used flush, begin/start replication using the same internal index.
	Server->ReplicationBridge->BeginReplication(ServerObject);

	ServerObject->IntA = 55;

	// Stop replication: Note: That this will fully detach instance from NetObject
	Server->ReplicationBridge->EndReplication(ServerObject);

	// Start replication again. As we are still replicating the first incarnation, there will be some latency before the last state is replicated as a new NetObject.
	Server->ReplicationBridge->BeginReplication(ServerObject);
	ServerObject->IntA = 66;

	// Late join client 1, as it joined before the NetUpdate and we used flush on the original incarnation should be added to its scope as well.
	CreateClient();
	ClientObjects.Add(Clients[1]->CreateStablyNamedObject<UTestReplicatedIrisObject>(StableObjectName));

	Server->UpdateAndSend(Clients);
	
	// Should exist on both clients, with he same state
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[0]);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[1]);

	// We expect the flushed state for both clients
	UE_NET_ASSERT_EQ(ClientObjects[0]->IntA, 33);
	UE_NET_ASSERT_EQ(ClientObjects[1]->IntA, 33);

	Server->UpdateAndSend(Clients);

	// Now we expect first incarnation to have been destroyed
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == nullptr);

	Server->UpdateAndSend(Clients);

	// Should now exist on both clients
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[0]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[0]);
	UE_NET_ASSERT_TRUE(Cast<UTestReplicatedIrisObject>(Clients[1]->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle)) == ClientObjects[1]);

	// We expect the latest state should be delivered
	UE_NET_ASSERT_EQ(ClientObjects[0]->IntA, 66);
	UE_NET_ASSERT_EQ(ClientObjects[1]->IntA, 66);
}

UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestCreateAndDestroyObjectsWithNoAttachedClientsArePolledBeforeSend)
{
	using namespace UE::Net::Private;

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;
	UE::Net::Private::FNetRefHandleManager* ServerNetRefHandleManager = &Server->ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject(0,0);
	const FInternalNetRefIndex ServerObjectInternalIndex = ServerNetRefHandleManager->GetInternalIndex(ServerObject->NetRefHandle);

	// Update
	Server->UpdateAndSend({});

	// Disconnect client
	DestroyClient(Client);

	// Update
	Server->UpdateAndSend({});

	// Destroy objects
	Server->DestroyObject(ServerObject);

	// Update
	Server->UpdateAndSend({});

	// Create object that should get the same internalindex as the first one
	UTestReplicatedIrisObject* ServerObject2 = Server->CreateObject(0,0);
	const FInternalNetRefIndex ServerObject2InternalIndex = ServerNetRefHandleManager->GetInternalIndex(ServerObject2->NetRefHandle);
	UE_NET_ASSERT_EQ(ServerObjectInternalIndex, ServerObject2InternalIndex);

	// Update
	Server->UpdateAndSend({});

	// Connect a new client
	FReplicationSystemTestClient* Client2 = CreateClient();

	// Send and deliver packet
	Server->UpdateAndSend({Client2});

	// Verify that we received the expected state
	{
		UTestReplicatedIrisObject* ClientObject2 = Cast<UTestReplicatedIrisObject>(Client2->GetReplicationBridge()->GetReplicatedObject(ServerObject2->NetRefHandle));
		UE_NET_ASSERT_TRUE(ServerObject2->StructD.IntB == ClientObject2->StructD.IntB);
	}
}

// Test references to a stable subobject of a dynamic replicated object
UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestRemapOfReferenceToStableSubObject)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	// Since we are validating this as part of this test, we need to ensure it is enabled
	TGuardConsoleVariable<bool> CVarOverride(TEXT("net.iris.EvictTrackedReferencesWithDynamicOuterFromCache"), true);

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with stable subobject on server
	UTestReplicatedIrisObjectWithStableSubObject* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithStableSubObject>();

	// Spawn objects referencing the first one and its subobject
	UTestReplicatedIrisObjectWithObjectReference* ServerObject2 = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();
	UTestReplicatedIrisObjectWithObjectReference* ServerObject3 = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();

	// Object2 references default (stable named subobject) 
	ServerObject2->WeakObjectPtrObjectRef = ServerObject->DefaultSubObject;

	// Object3 references dynamic object
	ServerObject3->WeakObjectPtrObjectRef = ServerObject;

	Server->UpdateAndSend({Client});

	// Verify that objects exist and that we can correctly resolve the references
	{
		UTestReplicatedIrisObjectWithStableSubObject* ClientObject = Cast<UTestReplicatedIrisObjectWithStableSubObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));	
		UTestReplicatedIrisObjectWithObjectReference* ClientObject2 = Cast<UTestReplicatedIrisObjectWithObjectReference>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject2->NetRefHandle));	
		UTestReplicatedIrisObjectWithObjectReference* ClientObject3 = Cast<UTestReplicatedIrisObjectWithObjectReference>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject3->NetRefHandle));	

		UE_NET_ASSERT_NE(ClientObject, nullptr);
		UE_NET_ASSERT_NE(ClientObject2, nullptr);
		UE_NET_ASSERT_NE(ClientObject3, nullptr);
		UE_NET_ASSERT_EQ((UObject*)ClientObject->DefaultSubObject, (UObject*)ClientObject2->WeakObjectPtrObjectRef.Get());
		UE_NET_ASSERT_EQ((UObject*)ClientObject, (UObject*)ClientObject3->WeakObjectPtrObjectRef.Get());
	}

	// We want to monitor the reader
	const FReplicationConnection* Connection = Client->GetReplicationSystem()->GetReplicationSystemInternal()->GetConnections().GetConnection(Client->LocalConnectionId);
	const FReplicationReader* ReplicationReader = Connection->ReplicationReader;
	const FObjectReferenceCache* ServerObjectReferenceCache = &Server->GetReplicationSystem()->GetReplicationSystemInternal()->GetObjectReferenceCache();
	const FObjectReferenceCache* ClientObjectReferenceCache = &Client->GetReplicationSystem()->GetReplicationSystemInternal()->GetObjectReferenceCache();

	// Verify tracking for client
	{
		const uint32 NumTrackedUnresolvedHandleToDependents = ReplicationReader->GetNumTrackedUnresolvedHandleToDependents();
		const uint32 NumTrackedResolvedDynamicHandleToDependents = ReplicationReader->GetNumTrackedResolvedDynamicHandleToDependents();
		const uint32 NumObjectsWithTrackedSubObjects = ClientObjectReferenceCache->GetNumObjectsWithTrackedSubObjectHandles();

		UE_NET_ASSERT_EQ(NumTrackedUnresolvedHandleToDependents, 0U);
		UE_NET_ASSERT_EQ(NumTrackedResolvedDynamicHandleToDependents, 2U);
		UE_NET_ASSERT_EQ(NumObjectsWithTrackedSubObjects, 1U);
	}

	// Verify tracking for server
	{
		const uint32 NumObjectsWithTrackedSubObjects = ServerObjectReferenceCache->GetNumObjectsWithTrackedSubObjectHandles();

		// We expect that we are tracking one object with relative references.
		UE_NET_ASSERT_EQ(NumObjectsWithTrackedSubObjects, 1U);
	}

	// Filter out object with default subobject
	ReplicationSystem->AddToGroup(ReplicationSystem->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);

	Server->UpdateAndSend({Client});

	// Verify new state
	{
		UTestReplicatedIrisObjectWithStableSubObject* ClientObject = Cast<UTestReplicatedIrisObjectWithStableSubObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));	
		UTestReplicatedIrisObjectWithObjectReference* ClientObject2 = Cast<UTestReplicatedIrisObjectWithObjectReference>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject2->NetRefHandle));	
		UTestReplicatedIrisObjectWithObjectReference* ClientObject3 = Cast<UTestReplicatedIrisObjectWithObjectReference>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject3->NetRefHandle));	

		UE_NET_ASSERT_EQ(ClientObject, nullptr);
		UE_NET_ASSERT_NE(ClientObject2, nullptr);

		// Issue: Not really related to networking.
		// To fix, default subobjects needs to be invalidated with owning objects.
		// Since we have not run GC this is still legit and nothing marks it as garbage even though the root is.
		// UE_NET_ASSERT_EQ((UObject*)ClientObject2->WeakObjectPtrObjectRef.Get(), nullptr);
		UE_NET_ASSERT_EQ((UObject*)ClientObject3->WeakObjectPtrObjectRef.Get(), nullptr);
	}

	// Reenable replication of ObjectA again to try to replicate it to server now that it should succeed
	ReplicationSystem->RemoveFromGroup(ReplicationSystem->GetNotReplicatedNetObjectGroup(), ServerObject->NetRefHandle);

	// Create new object referencing the original default subobject on the server.
	UTestReplicatedIrisObjectWithObjectReference* ServerObject4 = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();
	// Object4 references default (stable named subobject) 
	ServerObject4->WeakObjectPtrObjectRef = ServerObject->DefaultSubObject;

	Server->UpdateAndSend({Client});

	// Verify tracking for client
	{
		const uint32 NumTrackedUnresolvedHandleToDependents = ReplicationReader->GetNumTrackedUnresolvedHandleToDependents();
		const uint32 NumTrackedResolvedDynamicHandleToDependents = ReplicationReader->GetNumTrackedResolvedDynamicHandleToDependents();
		const uint32 NumObjectsWithTrackedSubObjects = ClientObjectReferenceCache->GetNumObjectsWithTrackedSubObjectHandles();

		UE_NET_ASSERT_EQ(NumTrackedUnresolvedHandleToDependents, 0U);
		UE_NET_ASSERT_EQ(NumTrackedResolvedDynamicHandleToDependents, 3U);
		UE_NET_ASSERT_EQ(NumObjectsWithTrackedSubObjects, 1U);
	}

	// Verify that objects exist and that we can resolve the reference
	{
		UTestReplicatedIrisObjectWithStableSubObject* ClientObject = Cast<UTestReplicatedIrisObjectWithStableSubObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));	
		UTestReplicatedIrisObjectWithObjectReference* ClientObject2 = Cast<UTestReplicatedIrisObjectWithObjectReference>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject2->NetRefHandle));	
		UTestReplicatedIrisObjectWithObjectReference* ClientObject3 = Cast<UTestReplicatedIrisObjectWithObjectReference>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject3->NetRefHandle));	
		UTestReplicatedIrisObjectWithObjectReference* ClientObject4 = Cast<UTestReplicatedIrisObjectWithObjectReference>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject4->NetRefHandle));	

		UE_NET_ASSERT_NE(ClientObject, nullptr);
		UE_NET_ASSERT_NE(ClientObject2, nullptr);
		UE_NET_ASSERT_NE(ClientObject3, nullptr);

		// This should be remapped as it is a reference to a replicated object.
		UE_NET_ASSERT_EQ((UObject*)ClientObject, (UObject*)ClientObject3->WeakObjectPtrObjectRef.Get());

		// This should be properly resolved to the new instance
		UE_NET_ASSERT_EQ((UObject*)ClientObject->DefaultSubObject, (UObject*)ClientObject4->WeakObjectPtrObjectRef.Get());

		// This should be remapped to the new instance.
		UE_NET_ASSERT_EQ((UObject*)ClientObject->DefaultSubObject, (UObject*)ClientObject2->WeakObjectPtrObjectRef.Get());
	}

	// Destroy object
	Server->DestroyObject(ServerObject);

	Server->UpdateAndSend({Client});
	Server->UpdateAndSend({Client});

	// Verify tracking for client
	{
		const uint32 NumTrackedUnresolvedHandleToDependents = ReplicationReader->GetNumTrackedUnresolvedHandleToDependents();
		const uint32 NumTrackedResolvedDynamicHandleToDependents = ReplicationReader->GetNumTrackedResolvedDynamicHandleToDependents();
		const uint32 NumObjectsWithTrackedSubObjects = ClientObjectReferenceCache->GetNumObjectsWithTrackedSubObjectHandles();

		// We now expect all to be cleaned up
		UE_NET_ASSERT_EQ(NumTrackedUnresolvedHandleToDependents, 0U);
		UE_NET_ASSERT_EQ(NumTrackedResolvedDynamicHandleToDependents, 0U);
		UE_NET_ASSERT_EQ(NumObjectsWithTrackedSubObjects, 0U);
	}

	// Verify tracking for server
	{
		const uint32 NumObjectsWithTrackedSubObjects = ServerObjectReferenceCache->GetNumObjectsWithTrackedSubObjectHandles();

		// We now expect all to be cleaned up
		UE_NET_ASSERT_EQ(NumObjectsWithTrackedSubObjects, 0U);
	}
}

#if 0 //$TODO: Currently the test framework does not support stable subobjects in the creationheader, See: iris specific tests in UNetTestStableSubObject instead for tests with deferred creation
UE_NET_TEST_FIXTURE(FTearOffAndFlushTestFixture, TestDeferredCreateOfFlushedRootWithDefaultSubObject)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::VeryVerbose);

	UReplicationSystem* ReplicationSystem = Server->ReplicationSystem;

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object with default subobject on server
	UTestReplicatedIrisObjectWithStableSubObject* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithStableSubObject>();

	// Replicate the default subobject as well.
	Server->ReplicationBridge->BeginReplication(ServerObject->NetRefHandle, ServerObject->DefaultSubObject);

	// Spawn objects referencing the first one and its subobject
	UTestReplicatedIrisObjectWithObjectReference* ServerObject2 = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();
	UTestReplicatedIrisObjectWithObjectReference* ServerObject3 = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();

	Server->ReplicationBridge->EndReplication(ServerObject, EEndReplicationFlags::Destroy | EEndReplicationFlags::Flush);

	// Before fix this would remove all cached entries in object referencecache prematurely 
	Server->UpdateAndSend({});

	Server->UpdateAndSend({Client});
}
#endif

} // end namespace UE::Net::Private


