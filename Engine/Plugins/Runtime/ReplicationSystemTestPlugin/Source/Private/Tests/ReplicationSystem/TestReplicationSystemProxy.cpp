// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSystemProxyTestFixture.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"

namespace UE::Net::Private
{

UE_NET_TEST_FIXTURE(FReplicationSystemProxyTestFixture, ReplicationDelegates)
{
	// Add a backend server
	FReplicationSystemTestServer* Server = CreateServer();

	// Add a proxy server
	FReplicationSystemTestProxy* Proxy = CreateProxy();

	// Make connections
	ConnectProxyToBackendServer(Proxy, Server);

	struct FDelegateCounter
	{
		uint32 RootObjectCreate = 0;
		uint32 RootObjectDetach = 0;
		uint32 SubObjectCreate = 0;
		uint32 SubObjectDetach = 0;	
	};

	FDelegateCounter DelegateCounter;

	// Track the amount of times each delegate is called.
	UObjectReplicationBridge* BackendBridge = Cast<UObjectReplicationBridge>(GetProxyClientForBackendServer(Server, Proxy)->GetReplicationBridge());

	FDelegateHandle RootObjectPostInitHandle = GetRootObjectPostInitDelegate(BackendBridge).AddLambda(
		[&DelegateCounter](UE::Net::FNetRefHandle RootHandle, UObject* RootObject, UE::Net::FNetObjectFactoryId FactoryId) 
		{
			DelegateCounter.RootObjectCreate++; 
		});
	FDelegateHandle RootObjectDetachHandle = GetRootObjectDetachedDelegate(BackendBridge).AddLambda(
		[&DelegateCounter](UE::Net::FNetRefHandle RootHandle, UObject* RootObject, UE::Net::EDetachReason Reason)
		{
			DelegateCounter.RootObjectDetach++; 
		});
	FDelegateHandle SubObjectPostInitHandle = GetSubObjectPostInitDelegate(BackendBridge).AddLambda(
		[&DelegateCounter](UE::Net::FNetRefHandle SubObjectHandle, UObject* SubObject, UE::Net::FNetRefHandle RootObjectHandle, UObject* RootObject, UE::Net::FNetObjectFactoryId FactoryId)
		{
			DelegateCounter.SubObjectCreate++;
		});
	FDelegateHandle SubObjectDetachedHandle = GetSubObjectDetachedDelegate(BackendBridge).AddLambda(
		[&DelegateCounter](UE::Net::FNetRefHandle SubObjectHandle, UObject* SubObject, UE::Net::FNetRefHandle RootObjectHandle, UObject* RootObject, UE::Net::EDetachReason Reason)
		{
			DelegateCounter.SubObjectDetach++;
		});

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();
	UpdateAndSendServerToAllProxies(Server);
	UE_NET_ASSERT_EQ(DelegateCounter.RootObjectCreate, 1U);
	UE_NET_ASSERT_EQ(DelegateCounter.RootObjectDetach, 0U);
	UE_NET_ASSERT_EQ(DelegateCounter.SubObjectCreate, 0U);
	UE_NET_ASSERT_EQ(DelegateCounter.SubObjectDetach, 0U);

	// Spawn sub-object on server
	UTestReplicatedIrisObject* ServerSubObject = Server->CreateSubObject(ServerObject->NetRefHandle);
	UpdateAndSendServerToAllProxies(Server);
	UE_NET_ASSERT_EQ(DelegateCounter.RootObjectCreate, 1U);
	UE_NET_ASSERT_EQ(DelegateCounter.RootObjectDetach, 0U);
	UE_NET_ASSERT_EQ(DelegateCounter.SubObjectCreate, 1U);
	UE_NET_ASSERT_EQ(DelegateCounter.SubObjectDetach, 0U);

	// Destroy sub-object on server
	Server->DestroyObject(ServerSubObject);
	UpdateAndSendServerToAllProxies(Server);
	UE_NET_ASSERT_EQ(DelegateCounter.RootObjectCreate, 1U);
	UE_NET_ASSERT_EQ(DelegateCounter.RootObjectDetach, 0U);
	UE_NET_ASSERT_EQ(DelegateCounter.SubObjectCreate, 1U);
	UE_NET_ASSERT_EQ(DelegateCounter.SubObjectDetach, 1U);

	// Destroy object on server
	Server->DestroyObject(ServerObject);
	UpdateAndSendServerToAllProxies(Server);
	UE_NET_ASSERT_EQ(DelegateCounter.RootObjectCreate, 1U);
	UE_NET_ASSERT_EQ(DelegateCounter.RootObjectDetach, 1U);
	UE_NET_ASSERT_EQ(DelegateCounter.SubObjectCreate, 1U);
	UE_NET_ASSERT_EQ(DelegateCounter.SubObjectDetach, 1U);

	GetRootObjectPostInitDelegate(BackendBridge).Remove(RootObjectPostInitHandle);
	GetRootObjectDetachedDelegate(BackendBridge).Remove(RootObjectDetachHandle);
	GetSubObjectPostInitDelegate(BackendBridge).Remove(SubObjectPostInitHandle);
	GetSubObjectDetachedDelegate(BackendBridge).Remove(SubObjectDetachedHandle);
}

UE_NET_TEST_FIXTURE(FReplicationSystemProxyTestFixture, ReplicateAndDestroySingleObject)
{
	// Add a backend server
	FReplicationSystemTestServer* Server = CreateServer();

	// Add a proxy server
	FReplicationSystemTestProxy* Proxy = CreateProxy();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Make connections
	ConnectProxyToBackendServer(Proxy, Server);
	ConnectClientToProxy(Client, Proxy);

	// Spawn object on server
	UTestReplicatedIrisObject* ServerObject = Server->CreateObject();

	// Send and deliver packet, server -> proxy
	UpdateAndSendServerToAllProxies(Server);

	// Verify that created server handle now also exists on the proxy backend
	FReplicationSystemTestClient* ProxyClient = GetProxyClientForBackendServer(Server, Proxy);
	UTestReplicatedIrisObject* ProxyBackendTestObject = ProxyClient->GetObjectAs<UTestReplicatedIrisObject>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_NE(ProxyBackendTestObject, nullptr);

	// Verify the create server handle now also exists in the proxy frontend
	UTestReplicatedIrisObject* ProxyFrontendTestObject = Proxy->GetFrontendServer()->GetObjectAs<UTestReplicatedIrisObject>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_NE(ProxyFrontendTestObject, nullptr);

	// Send and deliver packet, proxy -> client
	UpdateAndSendProxyToAllClients(Proxy);

	// Verify that created server handle now also exists on client
	UE_NET_ASSERT_TRUE(Client->IsResolvableNetRefHandle(ServerObject->NetRefHandle));

	// Destroy the spawned object on server
	Server->DestroyObject(ServerObject);

	// Send and deliver packet, server -> proxy
	UpdateAndSendServerToAllProxies(Server);

	// Verify that object now is destroyed on proxy as well
	UE_NET_ASSERT_FALSE(ProxyClient->IsValidNetRefHandle(ServerObject->NetRefHandle));

	// Send and deliver packet, proxy -> client
	UpdateAndSendProxyToAllClients(Proxy);

	// Verify that object now is destroyed on client as well
	UE_NET_ASSERT_FALSE(Client->IsValidNetRefHandle(ServerObject->NetRefHandle));
}

// Test objects and simple properties replicating from two backend servers through one proxy to one client.
UE_NET_TEST_FIXTURE(FReplicationSystemProxyTestFixture, TestTwoServers)
{
	// Add two backend servers
	constexpr uint32 InitialNetRefHandleIndexServer1 = 1000;
	FReplicationSystemTestServer* Server1 = CreateServer(InitialNetRefHandleIndexServer1);

	constexpr uint32 InitialNetRefHandleIndexServer2 = 2000;
	FReplicationSystemTestServer* Server2 = CreateServer(InitialNetRefHandleIndexServer2);

	// Add a proxy server
	FReplicationSystemTestProxy* Proxy = CreateProxy();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Make connections
	ConnectProxyToBackendServer(Proxy, Server1);
	ConnectProxyToBackendServer(Proxy, Server2);
	ConnectClientToProxy(Client, Proxy);

	// Spawn one object on each server
	UTestReplicatedIrisObject* ServerObject1 = Server1->CreateObject();
	UTestReplicatedIrisObject* ServerObject2 = Server2->CreateObject();
	
	const int32 IntServer1 = 10;
	const int32 IntServer2 = 20;

	ServerObject1->IntA = IntServer1;
	ServerObject2->IntA = IntServer2;
	
	// Send and deliver packet, servers -> proxy
	UpdateAndSendServerToAllProxies(Server1);
	UpdateAndSendServerToAllProxies(Server2);

	// Verify that objects have been spawned on the proxy backend
	FReplicationSystemTestClient* ProxyBackendClient1 = GetProxyClientForBackendServer(Server1, Proxy);
	FReplicationSystemTestClient* ProxyBackendClient2 = GetProxyClientForBackendServer(Server2, Proxy);
	UTestReplicatedIrisObject* ProxyObject1 = ProxyBackendClient1->GetObjectAs<UTestReplicatedIrisObject>(ServerObject1->NetRefHandle);
	UTestReplicatedIrisObject* ProxyObject2 = ProxyBackendClient2->GetObjectAs<UTestReplicatedIrisObject>(ServerObject2->NetRefHandle);
	UE_NET_ASSERT_NE(ProxyObject1, nullptr);
	UE_NET_ASSERT_NE(ProxyObject2, nullptr);
	UE_NET_ASSERT_NE(ProxyObject1, ProxyObject2);

	// Verify that the values are correct on the proxy
	UE_NET_ASSERT_EQ(ProxyObject1->IntA, IntServer1);
	UE_NET_ASSERT_EQ(ProxyObject2->IntA, IntServer2);

	// Verify that objects have been added to the proxy frontend
	UE_NET_ASSERT_TRUE(Proxy->GetFrontendServer()->IsValidNetRefHandle(ProxyObject1->NetRefHandle));
	UE_NET_ASSERT_TRUE(Proxy->GetFrontendServer()->IsValidNetRefHandle(ProxyObject2->NetRefHandle));

	// Send and deliver packet, proxy -> client
	UpdateAndSendProxyToAllClients(Proxy);

	// Verify that objects have been spawned on client
	UTestReplicatedIrisObject* ClientObject1 = Client->GetObjectAs<UTestReplicatedIrisObject>(ServerObject1->NetRefHandle);
	UTestReplicatedIrisObject* ClientObject2 = Client->GetObjectAs<UTestReplicatedIrisObject>(ServerObject2->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject1, nullptr);
	UE_NET_ASSERT_NE(ClientObject2, nullptr);
	UE_NET_ASSERT_NE(ClientObject1, ClientObject2);

	// Verify that the values are correct on the client
	UE_NET_ASSERT_EQ(ClientObject1->IntA, IntServer1);
	UE_NET_ASSERT_EQ(ClientObject2->IntA, IntServer2);
}

// Test proxy mode replication systems replicating properties of a reused object after a mock migration to another backend replication system/server.
UE_NET_TEST_FIXTURE(FReplicationSystemProxyTestFixture, ReuseObjectPropertyReplication)
{
	// Add two backend servers. Explicitly use the same initial NetRefHandleIndex because
	// this test fakes migration where the handle IDs have to match. We will only spawn one root object
	// on each server (the one on server 2 being when it's "migrated" from server 1).
	constexpr uint32 InitialNetRefHandleIndexServer1 = 1000;
	FReplicationSystemTestServer* Server1 = CreateServer(InitialNetRefHandleIndexServer1);

	constexpr uint32 InitialNetRefHandleIndexServer2 = 1000;
	FReplicationSystemTestServer* Server2 = CreateServer(InitialNetRefHandleIndexServer2);

	// Add a proxy server
	FReplicationSystemTestProxy* Proxy = CreateProxy();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Make connections
	ConnectProxyToBackendServer(Proxy, Server1);
	ConnectProxyToBackendServer(Proxy, Server2);
	ConnectClientToProxy(Client, Proxy);

	// Spawn one object on one server
	UTestReplicatedIrisObject* Server1RootObject = Server1->CreateObject();
	UTestReplicatedIrisObject* Server1SubObject = Server1->CreateSubObject(Server1RootObject->NetRefHandle);

	const int32 RootObjectInt = 10;
	const int32 SubObjectInt = 30;
	Server1RootObject->IntA = RootObjectInt;
	Server1SubObject->IntA = SubObjectInt;

	// Send and deliver packet, servers -> proxy
	UpdateAndSendServerToAllProxies(Server1);
	UpdateAndSendServerToAllProxies(Server2);

	// Verify that objects have been spawned on the proxy
	FReplicationSystemTestClient* ProxyBackendClient1 = GetProxyClientForBackendServer(Server1, Proxy);
	FReplicationSystemTestClient* ProxyBackendClient2 = GetProxyClientForBackendServer(Server2, Proxy);
	UTestReplicatedIrisObject* ProxyRootObject = ProxyBackendClient1->GetObjectAs<UTestReplicatedIrisObject>(Server1RootObject->NetRefHandle);
	UTestReplicatedIrisObject* ProxySubObject = ProxyBackendClient1->GetObjectAs<UTestReplicatedIrisObject>(Server1SubObject->NetRefHandle);
	UE_NET_ASSERT_NE(ProxyRootObject, nullptr);
	UE_NET_ASSERT_NE(ProxyRootObject, Server1RootObject);
	UE_NET_ASSERT_NE(ProxySubObject, nullptr);
	UE_NET_ASSERT_NE(ProxySubObject, Server1SubObject);

	// Check property values on the proxy
	UE_NET_ASSERT_EQ(ProxyRootObject->IntA, RootObjectInt);
	UE_NET_ASSERT_EQ(ProxySubObject->IntA, SubObjectInt);
	
	// Verify that objects have been added to the proxy frontend
	UE_NET_ASSERT_TRUE(Proxy->GetFrontendServer()->IsValidNetRefHandle(ProxyRootObject->NetRefHandle));
	UE_NET_ASSERT_TRUE(Proxy->GetFrontendServer()->IsValidNetRefHandle(ProxySubObject->NetRefHandle));

	// Send and deliver packet, proxy -> client
	UpdateAndSendProxyToAllClients(Proxy);

	// Verify that the object has been spawned on client
	UTestReplicatedIrisObject* ClientRootObject = Client->GetObjectAs<UTestReplicatedIrisObject>(Server1RootObject->NetRefHandle);
	UTestReplicatedIrisObject* ClientSubObject = Client->GetObjectAs<UTestReplicatedIrisObject>(Server1SubObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientRootObject, nullptr);
	UE_NET_ASSERT_NE(ClientRootObject, ProxyRootObject);
	UE_NET_ASSERT_NE(ClientSubObject, nullptr);
	UE_NET_ASSERT_NE(ClientSubObject, ProxySubObject);

	// Verify that the values are correct on the client
	UE_NET_ASSERT_EQ(ClientRootObject->IntA, RootObjectInt);
	UE_NET_ASSERT_EQ(ClientSubObject->IntA, SubObjectInt);

	// Fake a "migration" of the object to server 2
	UTestReplicatedIrisObject* Server2RootObject = Server2->CreateObject();
	UTestReplicatedIrisObject* Server2SubObject = Server2->CreateSubObject(Server2RootObject->NetRefHandle);
	Server2RootObject->IntA = Server1RootObject->IntA;
	Server2SubObject->IntA = Server1SubObject->IntA;

	// Make sure the RefHandles match
	UE_NET_ASSERT_EQ(Server1RootObject->NetRefHandle.GetId(), Server2RootObject->NetRefHandle.GetId());
	UE_NET_ASSERT_EQ(Server1SubObject->NetRefHandle.GetId(), Server2SubObject->NetRefHandle.GetId());

	Server1->DestroyObject(Server1SubObject, EEndReplicationFlags::ProxyReuse);
	Server1->DestroyObject(Server1RootObject, EEndReplicationFlags::ProxyReuse);
	Server1RootObject = nullptr;
	Server1SubObject = nullptr;

	// Send and deliver packet, servers -> proxy
	UpdateAndSendServerToAllProxies(Server1);
	UpdateAndSendServerToAllProxies(Server2);

	// Verify that the root object is still the same on the proxy
	UTestReplicatedIrisObject* ProxyRootObjectAfterMigration = ProxyBackendClient2->GetObjectAs<UTestReplicatedIrisObject>(Server2RootObject->NetRefHandle);
	UE_NET_ASSERT_EQ(ProxyRootObjectAfterMigration, ProxyRootObject);
	UE_NET_ASSERT_EQ(ProxyRootObjectAfterMigration->IntA, RootObjectInt);

	// Verify that the subobject is still the same on the proxy
	UTestReplicatedIrisObject* ProxySubObjectAfterMigration = ProxyBackendClient2->GetObjectAs<UTestReplicatedIrisObject>(Server2SubObject->NetRefHandle);
	UE_NET_ASSERT_EQ(ProxySubObjectAfterMigration, ProxySubObject);
	UE_NET_ASSERT_EQ(ProxySubObjectAfterMigration->IntA, SubObjectInt);

	// Send and deliver packet, proxy -> client
	UpdateAndSendProxyToAllClients(Proxy);

	// Verify that the object is still the same on the client
	UTestReplicatedIrisObject* ClientRootObjectAfterMigration = Client->GetObjectAs<UTestReplicatedIrisObject>(Server2RootObject->NetRefHandle);
	UE_NET_ASSERT_EQ(ClientRootObjectAfterMigration, ClientRootObject);
	UE_NET_ASSERT_EQ(ClientRootObjectAfterMigration->IntA, RootObjectInt);

	UTestReplicatedIrisObject* ClientSubObjectAfterMigration = Client->GetObjectAs<UTestReplicatedIrisObject>(Server2SubObject->NetRefHandle);
	UE_NET_ASSERT_EQ(ClientSubObjectAfterMigration, ClientSubObject);
	UE_NET_ASSERT_EQ(ClientSubObjectAfterMigration->IntA, SubObjectInt);

	// Replicate a new value on the object from server 2
	const int32 NewRootObjectIntValue = RootObjectInt * 2;
	Server2RootObject->IntA = NewRootObjectIntValue;

	const int32 NewSubObjectIntValue = SubObjectInt * 2;
	Server2SubObject->IntA = NewSubObjectIntValue;

	// Send and deliver packet, servers -> proxy
	UpdateAndSendServerToAllProxies(Server1);
	UpdateAndSendServerToAllProxies(Server2);

	UE_NET_ASSERT_EQ(ProxyRootObjectAfterMigration->IntA, NewRootObjectIntValue);
	UE_NET_ASSERT_EQ(ProxySubObjectAfterMigration->IntA, NewSubObjectIntValue);

	// Send and deliver packet, proxy -> client
	UpdateAndSendProxyToAllClients(Proxy);

	UE_NET_ASSERT_EQ(ClientRootObjectAfterMigration->IntA, NewRootObjectIntValue);
	UE_NET_ASSERT_EQ(ClientSubObjectAfterMigration->IntA, NewSubObjectIntValue);
}

// Test proxy mode replication systems destroying a reused object after a mock migration to another backend replication system/server.
UE_NET_TEST_FIXTURE(FReplicationSystemProxyTestFixture, ReuseObjectDestruction)
{
	// Add two backend servers. Explicitly use the same initial NetRefHandleIndex because
	// this test fakes migration where the handle IDs have to match. We will only spawn one object
	// on each server (the one on server 2 being when it's "migrated" from server 1).
	constexpr uint32 InitialNetRefHandleIndexServer1 = 1000;
	FReplicationSystemTestServer* Server1 = CreateServer(InitialNetRefHandleIndexServer1);

	constexpr uint32 InitialNetRefHandleIndexServer2 = 1000;
	FReplicationSystemTestServer* Server2 = CreateServer(InitialNetRefHandleIndexServer2);

	// Add a proxy server
	FReplicationSystemTestProxy* Proxy = CreateProxy();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Make connections
	ConnectProxyToBackendServer(Proxy, Server1);
	ConnectProxyToBackendServer(Proxy, Server2);
	ConnectClientToProxy(Client, Proxy);

	// Spawn one object on one server
	UTestReplicatedIrisObject* Server1RootObject = Server1->CreateObject();
	UTestReplicatedIrisObject* Server1SubObject = Server1->CreateSubObject(Server1RootObject->NetRefHandle);

	// Send and deliver packet, servers -> proxy
	UpdateAndSendServerToAllProxies(Server1);
	UpdateAndSendServerToAllProxies(Server2);

	// Verify that objects have been spawned on the proxy
	FReplicationSystemTestClient* ProxyBackendClient1 = GetProxyClientForBackendServer(Server1, Proxy);
	FReplicationSystemTestClient* ProxyBackendClient2 = GetProxyClientForBackendServer(Server2, Proxy);
	UTestReplicatedIrisObject* ProxyRootObject = ProxyBackendClient1->GetObjectAs<UTestReplicatedIrisObject>(Server1RootObject->NetRefHandle);
	UTestReplicatedIrisObject* ProxySubObject = ProxyBackendClient1->GetObjectAs<UTestReplicatedIrisObject>(Server1SubObject->NetRefHandle);
	UE_NET_ASSERT_NE(ProxyRootObject, nullptr);
	UE_NET_ASSERT_NE(ProxyRootObject, Server1RootObject);
	UE_NET_ASSERT_NE(ProxySubObject, nullptr);
	UE_NET_ASSERT_NE(ProxySubObject, Server1SubObject);

	// Send and deliver packet, proxy -> client
	UpdateAndSendProxyToAllClients(Proxy);

	// Verify that the object has been spawned on client
	UTestReplicatedIrisObject* ClientRootObject = Client->GetObjectAs<UTestReplicatedIrisObject>(Server1RootObject->NetRefHandle);
	UTestReplicatedIrisObject* ClientSubObject = Client->GetObjectAs<UTestReplicatedIrisObject>(Server1SubObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientRootObject, nullptr);
	UE_NET_ASSERT_NE(ClientRootObject, ProxyRootObject);
	UE_NET_ASSERT_NE(ClientSubObject, nullptr);
	UE_NET_ASSERT_NE(ClientSubObject, ProxySubObject);

	// Fake a "migration" of the object to server 2
	UTestReplicatedIrisObject* Server2RootObject = Server2->CreateObject();
	UTestReplicatedIrisObject* Server2SubObject = Server2->CreateSubObject(Server2RootObject->NetRefHandle);

	// Make sure the RefHandles match
	FNetRefHandle SavedRootObjectHandle = Server1RootObject->NetRefHandle;
	FNetRefHandle SavedSubObjectHandle = Server1SubObject->NetRefHandle;
	UE_NET_ASSERT_EQ(Server1RootObject->NetRefHandle.GetId(), Server2RootObject->NetRefHandle.GetId());
	UE_NET_ASSERT_EQ(Server1SubObject->NetRefHandle.GetId(), Server2SubObject->NetRefHandle.GetId());

	Server1->DestroyObject(Server1SubObject, EEndReplicationFlags::ProxyReuse);
	Server1->DestroyObject(Server1RootObject, EEndReplicationFlags::ProxyReuse);
	Server1RootObject = nullptr;
	Server1SubObject = nullptr;

	// Send and deliver packet, servers -> proxy
	UpdateAndSendServerToAllProxies(Server1);
	UpdateAndSendServerToAllProxies(Server2);

	// Make sure proxy object is still the same
	UTestReplicatedIrisObject* ProxyRootObjectAfterMigration = ProxyBackendClient2->GetObjectAs<UTestReplicatedIrisObject>(SavedRootObjectHandle);
	UTestReplicatedIrisObject* ProxySubObjectAfterMigration = ProxyBackendClient2->GetObjectAs<UTestReplicatedIrisObject>(SavedSubObjectHandle);
	UE_NET_ASSERT_EQ(ProxyRootObjectAfterMigration, ProxyRootObject);
	UE_NET_ASSERT_EQ(ProxySubObjectAfterMigration, ProxySubObject);

	// Save weak pointers so we can check for the object actually being destroyed on the proxy
	TWeakObjectPtr<UTestReplicatedIrisObject> WeakProxyRootObject = ProxyRootObjectAfterMigration;
	TWeakObjectPtr<UTestReplicatedIrisObject> WeakProxySubObject = ProxySubObjectAfterMigration;

	// Send and deliver packet, proxy -> client
	UpdateAndSendProxyToAllClients(Proxy);

	// Verify that the object is still the same on the client
	UTestReplicatedIrisObject* ClientRootObjectAfterMigration = Client->GetObjectAs<UTestReplicatedIrisObject>(SavedRootObjectHandle);
	UTestReplicatedIrisObject* ClientSubObjectAfterMigration = Client->GetObjectAs<UTestReplicatedIrisObject>(SavedSubObjectHandle);
	UE_NET_ASSERT_EQ(ClientRootObjectAfterMigration, ClientRootObject);
	UE_NET_ASSERT_EQ(ClientSubObjectAfterMigration, ClientSubObject);

	// Save weak pointers so we can check for the object actually being destroyed on the client
	TWeakObjectPtr<UTestReplicatedIrisObject> WeakClientRootObject = ClientRootObjectAfterMigration;
	TWeakObjectPtr<UTestReplicatedIrisObject> WeakClientSubObject = ClientSubObjectAfterMigration;

	// Destroy the subobject now on server 2
	Server2->DestroyObject(Server2SubObject, EEndReplicationFlags::Destroy);
	
	// Send and deliver packet, servers -> proxy
	UpdateAndSendServerToAllProxies(Server1);
	UpdateAndSendServerToAllProxies(Server2);

	// Root object should still exist
	UTestReplicatedIrisObject* ProxyRootObjectAfterSubObjectDestruction = ProxyBackendClient2->GetObjectAs<UTestReplicatedIrisObject>(SavedRootObjectHandle);
	UE_NET_ASSERT_EQ(ProxyRootObjectAfterSubObjectDestruction, ProxyRootObjectAfterMigration);
	UE_NET_ASSERT_TRUE(WeakProxyRootObject.IsValid());

	// Subobject should be destroyed
	UTestReplicatedIrisObject* ProxySubObjectAfterSubObjectDestruction = ProxyBackendClient2->GetObjectAs<UTestReplicatedIrisObject>(SavedSubObjectHandle);
	UE_NET_ASSERT_EQ(ProxySubObjectAfterSubObjectDestruction, nullptr);
	UE_NET_ASSERT_FALSE(WeakProxySubObject.IsValid());

	// Send and deliver packet, proxy -> client
	UpdateAndSendProxyToAllClients(Proxy);

	// Root object should still exist
	UTestReplicatedIrisObject* ClientRootObjectAfterSubObjectDestruction = Client->GetObjectAs<UTestReplicatedIrisObject>(SavedRootObjectHandle);
	UE_NET_ASSERT_EQ(ClientRootObjectAfterSubObjectDestruction, ClientRootObjectAfterMigration);
	UE_NET_ASSERT_TRUE(WeakClientRootObject.IsValid());

	// Subobject should be destroyed
	UTestReplicatedIrisObject* ClientSubObjectAfterSubObjectDestruction = Client->GetObjectAs<UTestReplicatedIrisObject>(SavedSubObjectHandle);
	UE_NET_ASSERT_EQ(ClientSubObjectAfterSubObjectDestruction, nullptr);
	UE_NET_ASSERT_FALSE(WeakClientSubObject.IsValid());

	// Destroy the root object now on server 2
	Server2->DestroyObject(Server2RootObject, EEndReplicationFlags::Destroy);

	// Send and deliver packet, servers -> proxy
	UpdateAndSendServerToAllProxies(Server1);
	UpdateAndSendServerToAllProxies(Server2);

	// Root object should be destroyed
	UTestReplicatedIrisObject* ProxyRootObjectAfterRootObjectDestruction = ProxyBackendClient2->GetObjectAs<UTestReplicatedIrisObject>(SavedRootObjectHandle);
	UE_NET_ASSERT_EQ(ProxyRootObjectAfterRootObjectDestruction, nullptr);
	UE_NET_ASSERT_FALSE(WeakProxyRootObject.IsValid());

	// Send and deliver packet, proxy -> client
	UpdateAndSendProxyToAllClients(Proxy);

	UTestReplicatedIrisObject* ClientRootObjectAfterRootObjectDestruction = Client->GetObjectAs<UTestReplicatedIrisObject>(SavedRootObjectHandle);
	UE_NET_ASSERT_EQ(ClientRootObjectAfterRootObjectDestruction, nullptr);
	UE_NET_ASSERT_FALSE(WeakClientRootObject.IsValid());
}

} // end namespace UE::Net::Private
