// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestStructNetTokenDataStore.h"
#include "ReplicationSystemServerClientTestFixture.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Iris/ReplicationSystem/StructNetTokenDataStore.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Logging/LogScopedVerbosityOverride.h"

// Note: this is in engine code.
#include "Net/Iris/ReplicationSystem/StructNetTokenDataStoreHelper.h"
#include "Net/UnrealNetwork.h"

namespace UE::Net
{
	
FTestMessage& operator<<(FTestMessage& Message, const FTestStructNetTokenStoreStruct& StructValue)
{
	return Message << TEXT("StructValue: A:") << StructValue.IntA << TEXT("B:") << StructValue.IntB;
}
	
} // end of namespace UE::Net;

UE_NET_IMPLEMENT_NAMED_NETTOKEN_STRUCT_SERIALIZERS(TestStructNetTokenStoreStruct)

UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES(TestStructNetTokenStoreStructDerived);

////////////////////////////////////////////////
// Tests
////////////////////////////////////////////////

namespace UE::Net
{
	// Force instantiate template to force template statics to be instantiated.
	template class TStructNetTokenDataStore<FTestStructNetTokenStoreStruct>;
	using FTestStructNetTokenDataStore = TStructNetTokenDataStore<FTestStructNetTokenStoreStruct>;
};

namespace UE::Net::Private
{

class FTestStructNetTokensFixture : public FReplicationSystemServerClientTestFixture
{
public:
	using DataType = FTestStructNetTokenDataStore::DataType;

	FTestStructNetTokenDataStore* ServerStructTokenStore = nullptr;
	FTestStructNetTokenDataStore* ClientStructTokenStore = nullptr;

	FReplicationSystemTestClient* Client = nullptr;
	
	const FNetTokenStoreState* ClientRemoteNetTokenState;
	const FNetTokenStoreState* ServerRemoteNetTokenState;

	FNetToken CreateAndExportNetToken(const DataType& Struct)
	{
		FNetToken Token = ServerStructTokenStore->GetOrCreateToken(Struct);
		UNetTokenDataStream* NetTokenDataStream = Cast<UNetTokenDataStream>(Server->GetReplicationSystem()->GetDataStream(Client->ConnectionIdOnServer, FName("NetToken")));
		if (NetTokenDataStream)
		{
			NetTokenDataStream->AddNetTokenForExplicitExport(Token);
		}

		return Token;
	}

	FNetToken CreateAndExportNetTokenOnClient(const DataType& Struct)
	{
		FNetToken Token = ClientStructTokenStore->GetOrCreateToken(Struct);
		UNetTokenDataStream* NetTokenDataStream = Cast<UNetTokenDataStream>(Client->GetReplicationSystem()->GetDataStream(Client->LocalConnectionId, FName("NetToken")));
		if (NetTokenDataStream)
		{
			NetTokenDataStream->AddNetTokenForExplicitExport(Token);
		}

		return Token;
	}

	virtual void SetUp() override
	{
		FReplicationSystemServerClientTestFixture::SetUp();
		NetTokenDataStoreUtil.AddNetTokenStoreTypeIdPair(FTestStructNetTokenStoreStruct::GetTokenStoreName().ToString(), 3);

		Client = CreateClient();
		{
			FNetTokenStore* ServerTokenStore = Server->GetReplicationSystem()->GetNetTokenStore();

			// Create server token store
			ServerTokenStore->CreateAndRegisterDataStore<FTestStructNetTokenDataStore>();
			ServerStructTokenStore = ServerTokenStore->GetDataStore<FTestStructNetTokenDataStore>();		
			ServerRemoteNetTokenState = ServerTokenStore->GetRemoteNetTokenStoreState(Client->ConnectionIdOnServer);
		}
		{
			FNetTokenStore* ClientTokenStore = Client->GetReplicationSystem()->GetNetTokenStore();

			// Create client token store
			ClientTokenStore->CreateAndRegisterDataStore<FTestStructNetTokenDataStore>();
			ClientStructTokenStore = ClientTokenStore->GetDataStore<FTestStructNetTokenDataStore>();		
			ClientRemoteNetTokenState = ClientTokenStore->GetRemoteNetTokenStoreState(Client->LocalConnectionId);
		}
	}

	virtual void TearDown()
	{
		FReplicationSystemServerClientTestFixture::TearDown();
	}	
};

// Test that we can send a FTestStructNetTokenStoreStruct as a NetToken (explicit)
UE_NET_TEST_FIXTURE(FTestStructNetTokensFixture, StructAsNetToken)
{
	// Create token
	FTestStructNetTokenStoreStruct StructA;
	StructA.IntA = 1;
	StructA.IntB = 2;

	FNetToken StructTokenA = CreateAndExportNetToken(StructA);

	// Verify that we can resolve the token on server.
	UE_NET_ASSERT_EQ(StructA, ServerStructTokenStore->ResolveToken(StructTokenA));

	{
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetToken, ELogVerbosity::Fatal);
		// Verify that we cannot resolve the token on the client
		UE_NET_ASSERT_NE(StructA, ClientStructTokenStore->ResolveRemoteToken(StructTokenA, *ClientRemoteNetTokenState));
	}

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify that we can resolve the token on the client
	UE_NET_ASSERT_EQ(StructA, ClientStructTokenStore->ResolveRemoteToken(StructTokenA, *ClientRemoteNetTokenState));
}

// Test that we can send a FTestStructNetTokenStoreStruct as a NetToken but this time exported as a replicated property using an Iris NetSerializer
UE_NET_TEST_FIXTURE(FTestStructNetTokensFixture, StructAsNetTokenPropertyWithIris)
{
	// Create token struct
	FTestStructNetTokenStoreStruct StructA;
	StructA.IntA = 1;
	StructA.IntB = 2;

	UTestStructAsNetTokenObject* ServerObject = Server->CreateObject<UTestStructAsNetTokenObject>();
	ServerObject->NetTokenStoreStruct = StructA;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	UTestStructAsNetTokenObject* ClientObject = Cast<UTestStructAsNetTokenObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Verify that we replicated object
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Verify that we replicated struct correctly
	UE_NET_ASSERT_EQ(StructA, ClientObject->NetTokenStoreStruct);

	// Get token on server
	FNetToken ServerToken = ServerStructTokenStore->GetOrCreateToken(StructA);

	// Get token on client
	FNetToken ClientToken = ClientStructTokenStore->GetOrCreateToken(ClientObject->NetTokenStoreStruct);

	// They should match
	UE_NET_ASSERT_EQ(ServerToken, ClientToken);
}


// Test that we can send a FTestStructNetTokenStoreStruct as a NetToken but this time exported through a property using the LastResortNetserializer, calling into ::NetSerialize
UE_NET_TEST_FIXTURE(FTestStructNetTokensFixture, StructAsNetTokenUsingLastResortNetSerializer)
{
	// Create token struct
	FTestStructNetTokenStoreStructDerived StructA;
	StructA.IntA = 1;
	StructA.IntB = 2;

	UTestStructAsNetTokenObject* ServerObject = Server->CreateObject<UTestStructAsNetTokenObject>();
	ServerObject->DerivedNetTokenStoreStruct = StructA;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	UTestStructAsNetTokenObject* ClientObject = Cast<UTestStructAsNetTokenObject>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));

	// Verify that we replicated object
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Verify that we replicated struct correctly
	UE_NET_ASSERT_EQ(StructA, ClientObject->DerivedNetTokenStoreStruct);

	// Get token on server
	FNetToken ServerToken = ServerStructTokenStore->GetOrCreateToken(StructA);

	// Get token on client
	FNetToken ClientToken = ClientStructTokenStore->GetOrCreateToken(ClientObject->DerivedNetTokenStoreStruct);

	// They should match
	UE_NET_ASSERT_EQ(ServerToken, ClientToken);
}

// Test that we can receive corrupt token data in the form a bad token index without crashing
UE_NET_TEST_FIXTURE(FTestStructNetTokensFixture, CanReceiveInvalidTokenIndexGracefully)
{
	// Create token which is fairly large since we're going to corrupt the packet and need some leeway.
	FTestStructNetTokenStoreStruct StructA;
	StructA.IntA = 1;
	StructA.IntB = 2;
	StructA.ByteArray.SetNumZeroed(64);

	FNetToken StructTokenA = CreateAndExportNetToken(StructA);

	// Send  packet
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Corrupt packet
	{
		FReplicationSystemTestNode::FConnectionInfo& ConnectionInfo = Server->GetConnectionInfo(Client->ConnectionIdOnServer);
		UE_NET_ASSERT_GT(ConnectionInfo.WrittenPackets.Count(), SIZE_T(0));

		FReplicationSystemTestNode::FPacketData& Packet = ConnectionInfo.WrittenPackets.Poke();

		FNetBitStreamWriter Writer;
		Writer.InitBytes(Packet.PacketBuffer, sizeof(Packet.PacketBuffer));

		// The bit offset for the FNetToken at the time this test was written.
		constexpr uint32 ExpectedTokenBitOffset = 9U;
		Writer.Seek(ExpectedTokenBitOffset);

		// Mimic FNetTokenStore::InternalWriteNetToken but write a bad token index
		constexpr uint32 BadTokenIndex = FNetToken::MaxNetTokenCount + 4711U;
		WritePackedUint32(&Writer, BadTokenIndex);
		Writer.WriteBool(StructTokenA.IsAssignedByAuthority());
		Writer.WriteBits(StructTokenA.GetTypeId(), FNetToken::TokenTypeIdBits);

		Writer.CommitWrites();
	}

	LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetToken, ELogVerbosity::Fatal);

	// Deliver packet
	{
		FTestEnsureScope SuppressEnsureScope;
		Server->DeliverTo(Client, DeliverPacket);
	}

	// If corruption occured we should not be able to resolve the token on the client and we'll get some sort of invalid state back which shouldn't match the server state.
	UE_NET_ASSERT_NE(StructA, ClientStructTokenStore->ResolveRemoteToken(StructTokenA, *ClientRemoteNetTokenState));
}

// Test that we can receive corrupt token data in the form a non-existing token type without crashing
UE_NET_TEST_FIXTURE(FTestStructNetTokensFixture, CanReceiveInvalidTokenTypeGracefully)
{
	// Create token which is fairly large since we're going to corrupt the packet and need some leeway.
	FTestStructNetTokenStoreStruct StructA;
	StructA.IntA = 1;
	StructA.IntB = 2;
	StructA.ByteArray.SetNumZeroed(64);

	FNetToken StructTokenA = CreateAndExportNetToken(StructA);

	// Send  packet
	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Corrupt packet
	{
		FReplicationSystemTestNode::FConnectionInfo& ConnectionInfo = Server->GetConnectionInfo(Client->ConnectionIdOnServer);
		UE_NET_ASSERT_GT(ConnectionInfo.WrittenPackets.Count(), SIZE_T(0));

		FReplicationSystemTestNode::FPacketData& Packet = ConnectionInfo.WrittenPackets.Poke();

		FNetBitStreamWriter Writer;
		Writer.InitBytes(Packet.PacketBuffer, sizeof(Packet.PacketBuffer));

		// The bit offset for the FNetToken at the time this test was written.
		constexpr uint32 ExpectedTokenBitOffset = 9U;
		Writer.Seek(ExpectedTokenBitOffset);

		// Mimic FNetTokenStore::InternalWriteNetToken but write a bad token type ID
		WritePackedUint32(&Writer, StructTokenA.GetIndex());
		Writer.WriteBool(StructTokenA.IsAssignedByAuthority());
		Writer.WriteBits(FNetToken::MaxTypeIdCount - 1U, FNetToken::TokenTypeIdBits);

		Writer.CommitWrites();
	}

	LOG_SCOPE_VERBOSITY_OVERRIDE(LogNetToken, ELogVerbosity::Fatal);

	// Deliver packet
	{
		FTestEnsureScope SuppressEnsureScope;
		Server->DeliverTo(Client, DeliverPacket);
	}

	// If corruption occured we should not be able to resolve the token on the client and we'll get some sort of invalid state back which shouldn't match the server state.
	UE_NET_ASSERT_NE(StructA, ClientStructTokenStore->ResolveRemoteToken(StructTokenA, *ClientRemoteNetTokenState));
}

} // end namespace UE::Net::Private


//////////////////////////////////////////////////////////////////////////
// Implementation for UTestStructAsNetTokenObject
//////////////////////////////////////////////////////////////////////////
UTestStructAsNetTokenObject::UTestStructAsNetTokenObject()
: UReplicatedTestObject()
{
}

void UTestStructAsNetTokenObject::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	FDoRepLifetimeParams LifetimeParams;
	LifetimeParams.bIsPushBased = false;
	DOREPLIFETIME_WITH_PARAMS(ThisClass, NetTokenStoreStruct, LifetimeParams);
}

void UTestStructAsNetTokenObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	// Base object owns the fragment in this case
	{
		this->ReplicationFragments.Reset();
		UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags, &this->ReplicationFragments);
	}
}




