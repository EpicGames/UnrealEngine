// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "ReplicatedTestObject.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Iris/ReplicationSystem/NetTokenStructDefines.h"
#include "TestStructNetTokenDataStore.generated.h"

USTRUCT()
struct FTestStructNetTokenStoreStruct
{
	GENERATED_BODY()

	UE_NET_NETTOKEN_GENERATED_BODY(TestStructNetTokenStoreStruct, REPLICATIONSYSTEMTESTPLUGIN_API)

	uint64 GetUniqueKey() const
	{
		return HashCombineFast(HashCombineFast(IntA, IntB), GetTypeHash(ByteArray));
	}

	UPROPERTY(Transient)
	int32 IntA;

	UPROPERTY(Transient)
	int32 IntB;

	UPROPERTY(Transient)
	TArray<uint8> ByteArray;
};
	
UE_NET_DECLARE_NAMED_NETTOKEN_STRUCT_SERIALIZERS(TestStructNetTokenStoreStruct, REPLICATIONSYSTEMTESTPLUGIN_API)

// We will use a LastResortNetSerializer for this struct just to test NetSerialize using util.
USTRUCT()
struct FTestStructNetTokenStoreStructDerived : public FTestStructNetTokenStoreStruct 
{
	GENERATED_BODY()
};

//Since we are only testing last resort serializer for this Derived, we only want to declare the default NetTokenStruct ops for the RepGraph version.
template<> struct TStructOpsTypeTraits<FTestStructNetTokenStoreStructDerived> : public UE::Net::TNetTokenStructOpsTypeTraits<FTestStructNetTokenStoreStructDerived> {};


/**
 *  A test class for Replication that itself uses Property based replication but also has "components" that uses a mix of property based replication and native ReplicationStates
 */
UCLASS()
class UTestStructAsNetTokenObject : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UTestStructAsNetTokenObject();

	// Network interface must be part of base.
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

public:
	
	// This will replicate using iris NetSerializer
	UPROPERTY(Transient, Replicated)
	FTestStructNetTokenStoreStruct NetTokenStoreStruct;

	// This will replicate using a LastResortNetSerializer and thus will call into old replication path but do exports
	// using iris exports.
	UPROPERTY(Transient, Replicated)
	FTestStructNetTokenStoreStructDerived DerivedNetTokenStoreStruct;

public:
	// Network data only for test
	TArray<UE::Net::FReplicationFragment*> ReplicationFragments;
};