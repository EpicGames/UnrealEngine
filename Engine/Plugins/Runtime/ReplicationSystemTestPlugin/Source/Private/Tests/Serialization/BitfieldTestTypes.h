// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "BitfieldTestTypes.generated.h"

USTRUCT()
struct FTestUint64Bitfield
{
	GENERATED_BODY()

	UPROPERTY()
	uint64 Field0 : 1;
	UPROPERTY()
	uint64 Field1 : 1;
	UPROPERTY()
	uint64 Field2 : 1;
	UPROPERTY()
	uint64 Field3 : 1;
	UPROPERTY()
	uint64 Field4 : 1;
	UPROPERTY()
	uint64 Field5 : 1;
	UPROPERTY()
	uint64 Field6 : 1;
	UPROPERTY()
	uint64 Field7: 1;
	UPROPERTY()
	uint64 Field8 : 1;
};

USTRUCT()
struct FTestUint32Bitfield
{
	GENERATED_BODY()

	UPROPERTY()
	uint32 Field0 : 1;
	UPROPERTY()
	uint32 Field1 : 1;
	UPROPERTY()
	uint32 Field2 : 1;
	UPROPERTY()
	uint32 Field3 : 1;
	UPROPERTY()
	uint32 Field4 : 1;
	UPROPERTY()
	uint32 Field5 : 1;
	UPROPERTY()
	uint32 Field6 : 1;
	UPROPERTY()
	uint32 Field7: 1;
	UPROPERTY()
	uint32 Field8 : 1;
};

USTRUCT()
struct FTestUint16Bitfield
{
	GENERATED_BODY()

	UPROPERTY()
	uint16 Field0 : 1;
	UPROPERTY()
	uint16 Field1 : 1;
	UPROPERTY()
	uint16 Field2 : 1;
	UPROPERTY()
	uint16 Field3 : 1;
	UPROPERTY()
	uint16 Field4 : 1;
	UPROPERTY()
	uint16 Field5 : 1;
	UPROPERTY()
	uint16 Field6 : 1;
	UPROPERTY()
	uint16 Field7: 1;
	UPROPERTY()
	uint16 Field8 : 1;
};

USTRUCT()
struct FTestUint8Bitfield
{
	GENERATED_BODY()

	UPROPERTY()
	uint8 Field0 : 1;
	UPROPERTY()
	uint8 Field1 : 1;
	UPROPERTY()
	uint8 Field2 : 1;
	UPROPERTY()
	uint8 Field3 : 1;
	UPROPERTY()
	uint8 Field4 : 1;
	UPROPERTY()
	uint8 Field5 : 1;
	UPROPERTY()
	uint8 Field6 : 1;
	UPROPERTY()
	uint8 Field7: 1;
	UPROPERTY()
	uint8 Field8 : 1;
};
