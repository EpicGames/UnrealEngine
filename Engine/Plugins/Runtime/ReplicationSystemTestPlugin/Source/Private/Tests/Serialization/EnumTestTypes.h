// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Engine/EngineTypes.h"
#include "EnumTestTypes.generated.h"

/**
 * The values in the enums do not necessarily represent the minimum or maximum values in the underlying type.
 */

UENUM()
enum class ETestInt8Enum : int8
{
	MinValue = -128,
	MaxValue = 63, 
};

UENUM()
enum class ETestInt16Enum : int16
{
	MinValue = -32768,
	MaxValue = 255, 
};

UENUM()
enum class ETestInt32Enum : int32
{
	MinValue = -5536,
	MaxValue = 1023, 
};

UENUM()
enum class ETestInt64Enum : int64
{
	MinValue = -1152921504606846976LL,
	MaxValue = 4294967296ULL, 
};

UENUM()
enum class ETestUint8Enum : uint8
{
	MinValue = 64,
	MaxValue = 255, 
};

UENUM()
enum class ETestUint16Enum : uint16
{
	MinValue = 256,
	MaxValue = 32768, 
};

UENUM()
enum class ETestUint32Enum : uint32
{
	MinValue = 65535,
	MaxValue = 65536, 
};

UENUM()
enum class ETestUint64Enum : uint64
{
	MinValue = 4294967296ULL,
	MaxValue = 9223372036854775808ULL,
};

USTRUCT()
struct FStructWithETestInt64Enum
{
	GENERATED_BODY()

	ETestInt64Enum Value;
};

USTRUCT()
struct FStructWithETestUint64Enum
{
	GENERATED_BODY()

	ETestUint64Enum Value;
};

UCLASS()
class UClassWithNetRoleSwapping : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Replicated, Transient)
	TEnumAsByte<ENetRole> Role;

	UPROPERTY(Replicated, Transient)
	bool JustSomeReplicatedPropertyInBetween;

	UPROPERTY(Replicated, Transient)
	TEnumAsByte<ENetRole> RemoteRole;
};
