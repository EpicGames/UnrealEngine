// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockNetSerializer.h"

namespace UE::Net
{

struct FMockNetSerializer
{
	static constexpr uint32 Version = 0;
	static constexpr bool bIsForwardingSerializer = true; // Triggers asserts if a function is missing

	typedef int SourceType; // Arbitrary type
	typedef FMockNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);
	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&);
	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);
	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&);
	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);
	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);
};
UE_NET_IMPLEMENT_SERIALIZER(FMockNetSerializer);

const FMockNetSerializer::ConfigType FMockNetSerializer::DefaultConfig;

void FMockNetSerializer::Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	++Config->CallCounter->Serialize;
}

void FMockNetSerializer::Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	++Config->CallCounter->Deserialize;
}

void FMockNetSerializer::SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	++Config->CallCounter->SerializeDelta;
}

void FMockNetSerializer::DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	++Config->CallCounter->DeserializeDelta;
}

void FMockNetSerializer::Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	++Config->CallCounter->Quantize;
}

void FMockNetSerializer::Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	++Config->CallCounter->Dequantize;
}

bool FMockNetSerializer::IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	++Config->CallCounter->IsEqual;
	return Config->ReturnValues->bIsEqual;
}

bool FMockNetSerializer::Validate(FNetSerializationContext&, const FNetValidateArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	++Config->CallCounter->Validate;
	return Config->ReturnValues->bValidate;
}

void FMockNetSerializer::CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	++Config->CallCounter->CloneDynamicState;
}

void FMockNetSerializer::FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	++Config->CallCounter->FreeDynamicState;
}

void FMockNetSerializer::CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	++Config->CallCounter->CollectNetReferences;
}

}
