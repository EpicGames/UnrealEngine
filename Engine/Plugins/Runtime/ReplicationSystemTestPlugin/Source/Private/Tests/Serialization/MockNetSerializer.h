// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/NetSerializer.h"

struct FMockNetSerializerCallCounter
{
	uint32 Serialize{};
	uint32 Deserialize{};
	uint32 SerializeDelta{};
	uint32 DeserializeDelta{};
	uint32 Quantize{};
	uint32 Dequantize{};
	uint32 IsEqual{};
	uint32 Validate{};
	uint32 CloneDynamicState{};
	uint32 FreeDynamicState{};
	uint32 CollectNetReferences{};

	void Reset() { *this = FMockNetSerializerCallCounter() = {}; }
};

struct FMockNetSerializerReturnValues
{
	bool bIsEqual = false;
	bool bValidate = false;
};

struct FMockNetSerializerConfig : public FNetSerializerConfig
{
	FMockNetSerializerCallCounter* CallCounter = nullptr;
	FMockNetSerializerReturnValues* ReturnValues = nullptr;
};

namespace UE::Net
{

#define UE_NET_MOCK_API 
UE_NET_DECLARE_SERIALIZER(FMockNetSerializer, UE_NET_MOCK_API);
#undef UE_NET_MOCK_API

}

