// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptInterfaceTestTypes.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Net/UnrealNetwork.h"

UTestScriptInterfaceReplicatedObjectWithDefaultSubobject::UTestScriptInterfaceReplicatedObjectWithDefaultSubobject(const FObjectInitializer& ObjectInitializer)
: DefaultSubobjectWithInterface(CreateDefaultSubobject<UTestScriptInterfaceSubobject>(TEXT("TestScriptInterfaceSubobject")))
{
}

void UTestObjectReferencingScriptInterface::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

void UTestObjectReferencingScriptInterface::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(ThisClass, InterfaceObject);
}

void UTestScriptInterfaceReplicatedObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	Fragments.SetIsFragmentlessNetObject(true);
}

void UTestScriptInterfaceReplicatedObjectWithDefaultSubobject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	Fragments.SetIsFragmentlessNetObject(true);
}