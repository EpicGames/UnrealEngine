// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkAutomationTest.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "ScriptInterfaceTestTypes.generated.h"

UINTERFACE()
class UIrisTestInterface : public UInterface
{
	GENERATED_BODY()
};

class IIrisTestInterface
{
	GENERATED_BODY()

public:
	virtual bool IsInterfaceFunctionOverridden() const { return false; }
};

UCLASS()
class UTestScriptInterfaceSubobject : public UObject, public IIrisTestInterface
{
	GENERATED_BODY()

protected:
	virtual bool IsFullNameStableForNetworking() const override { return true; }

	// IIrisTestInterface
	virtual bool IsInterfaceFunctionOverridden() const { return true; }
};

UCLASS()
class UTestScriptInterfaceReplicatedObject : public UReplicatedTestObject, public IIrisTestInterface
{
	GENERATED_BODY()

public:
	// IIrisTestInterface
	virtual bool IsInterfaceFunctionOverridden() const { return true; }

	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;
};

UCLASS()
class UTestScriptInterfaceReplicatedObjectWithDefaultSubobject : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UTestScriptInterfaceReplicatedObjectWithDefaultSubobject(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

public:
	TObjectPtr<UObject> DefaultSubobjectWithInterface;
};

UCLASS()
class UTestObjectReferencingScriptInterface : public UReplicatedTestObject
{
	GENERATED_BODY()

protected:
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

public:
	UPROPERTY(Transient, Replicated)
	TScriptInterface<IIrisTestInterface> InterfaceObject;
};
