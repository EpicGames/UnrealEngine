// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestArrayPropertyNetSerializer.h"
#include "TestNetSerializerFixture.h"
#include "Iris/ReplicationState/InternalReplicationStateDescriptorUtils.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Iris/Serialization/InternalNetSerializers.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Net/Core/NetBitArray.h"
#include "Net/UnrealNetwork.h"
#include "Tests/Serialization/MockNetSerializer.h"

namespace UE::Net::Private
{

class FTestArrayPropertyNetSerializerBase : public FNetworkAutomationTestSuiteFixture
{
public:
	FTestArrayPropertyNetSerializerBase() : FNetworkAutomationTestSuiteFixture(), ArrayPropertyNetSerializerConfig(nullptr) {}

protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	template<typename ArrayType> void Quantize(const ArrayType& Source, void* StateBuffer);
	template<typename ArrayType> void Dequantize(const void* StateBuffer, ArrayType& Target);
	void FreeDynamicState(void* StateBuffer);
	void Serialize(const void* StateBuffer, void* BitstreamBuffer, SIZE_T BitStreamBufferSize);
	void SerializeDelta(const void* StateBuffer, const void* PrevStateBuffer, void* BitStreamBuffer, SIZE_T BitStreamBufferSize);

protected:
	TRefCountPtr<const FReplicationStateDescriptor> ReplicationStateDescriptor;

	FNetSerializationContext NetSerializationContext;
	FInternalNetSerializationContext InternalContext;

	FMockNetSerializerCallCounter MockNetSerializerCallCounter;
	FMockNetSerializerReturnValues MockNetSerializerReturnValues;
	FMockNetSerializerConfig MockNetSerializerConfig;
	FArrayPropertyNetSerializerConfig* ArrayPropertyNetSerializerConfig;
	const FNetSerializer* ArrayPropertyNetSerializer = &UE_NET_GET_SERIALIZER(FArrayPropertyNetSerializer);
	FReplicationStateMemberSerializerDescriptor OriginalMemberSerializerDescriptor;
	FReplicationStateMemberSerializerDescriptor MockMemberSerializerDescriptor;

	FNetBitStreamReader Reader;
	FNetBitStreamWriter Writer;

	enum : uint32
	{
		BufferSize = 1024,
	};

	alignas(16) uint8 StateBuffer0[BufferSize] = {};
	alignas(16) uint8 StateBuffer1[BufferSize] = {};

	alignas(16) uint8 BitStreamBuffer0[BufferSize];
	alignas(16) uint8 BitStreamBuffer1[BufferSize];
};

class FTestSimpleArrayPropertyNetSerializer : public FTestArrayPropertyNetSerializerBase
{
protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	void TestDescriptorHasDynamicStateTrait();
	void TestQuantize();
	void TestDequantize();
	void TestCloneDynamicState();
	void TestFreeDynamicState();
	void TestSerialize();
	void TestDeserialize();
	void TestSerializeDelta();
	void TestDeserializeDelta();
	void TestIsEqual();
	void TestValidate();

	void SetupForMockSerializer();
	void SetupForOriginalSerializer();

protected:
	FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest EmptyArrayInstance0;
	FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest EmptyArrayInstance1;

	FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest NonEmptyArrayInstance0;
	FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest NonEmptyArrayInstance1;

};

class FTestComplexArrayPropertyNetSerializer : public FTestArrayPropertyNetSerializerBase
{
public:
	FTestComplexArrayPropertyNetSerializer() : FTestArrayPropertyNetSerializerBase(), InnerArrayPropertyNetSerializerConfig(nullptr) {}

protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	void TestQuantize();
	void TestDequantize();
	void TestCloneDynamicState();
	void TestFreeDynamicState();
	void TestSerialize();
	void TestDeserialize();
	void TestSerializeDelta();
	void TestDeserializeDelta();
	void TestIsEqual();
	void TestValidate();

	void SetupForMockSerializer();
	void SetupForOriginalSerializer();

protected:
	enum : uint32 { TotalNumberOfElementsInNonEmptyArray = 6 };

	FArrayPropertyNetSerializerConfig* InnerArrayPropertyNetSerializerConfig;

	FStructWithDynamicArrayOfComplexTypeForArrayPropertyNetSerializerTest EmptyArrayInstance0;
	FStructWithDynamicArrayOfComplexTypeForArrayPropertyNetSerializerTest EmptyArrayInstance1;

	FStructWithDynamicArrayOfComplexTypeForArrayPropertyNetSerializerTest NonEmptyArrayInstance0;
	FStructWithDynamicArrayOfComplexTypeForArrayPropertyNetSerializerTest NonEmptyArrayInstance1;
};

class FTestElementChangeMaskForArrayPropertyNetSerializer : public FTestArrayPropertyNetSerializerBase
{
protected:
	virtual void SetUp() override;
	virtual void TearDown() override;
};

// Basic tests
UE_NET_TEST_FIXTURE(FTestArrayPropertyNetSerializerBase, TestHasIsForwardingSerializerTrait)
{
	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(ArrayPropertyNetSerializer->Traits, ENetSerializerTraits::IsForwardingSerializer));
}

UE_NET_TEST_FIXTURE(FTestArrayPropertyNetSerializerBase, TestHasDynamicStateTrait)
{
	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(ArrayPropertyNetSerializer->Traits, ENetSerializerTraits::HasDynamicState));
}

// Tests for array of primitive type
UE_NET_TEST_FIXTURE(FTestSimpleArrayPropertyNetSerializer, TestDescriptorHasDynamicStateTrait)
{
	TestDescriptorHasDynamicStateTrait();
}

UE_NET_TEST_FIXTURE(FTestSimpleArrayPropertyNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestSimpleArrayPropertyNetSerializer, TestDequantize)
{
	TestDequantize();
}

UE_NET_TEST_FIXTURE(FTestSimpleArrayPropertyNetSerializer, TestCloneDynamicState)
{
	TestCloneDynamicState();
}

UE_NET_TEST_FIXTURE(FTestSimpleArrayPropertyNetSerializer, TestFreeDynamicState)
{
	TestFreeDynamicState();
}

UE_NET_TEST_FIXTURE(FTestSimpleArrayPropertyNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestSimpleArrayPropertyNetSerializer, TestDeserialize)
{
	TestDeserialize();
}

UE_NET_TEST_FIXTURE(FTestSimpleArrayPropertyNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestSimpleArrayPropertyNetSerializer, TestDeserializeDelta)
{
	TestDeserializeDelta();
}

UE_NET_TEST_FIXTURE(FTestSimpleArrayPropertyNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestSimpleArrayPropertyNetSerializer, TestValidate)
{
	TestValidate();
}

// Tests for nested array
UE_NET_TEST_FIXTURE(FTestComplexArrayPropertyNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestComplexArrayPropertyNetSerializer, TestDequantize)
{
	TestDequantize();
}

UE_NET_TEST_FIXTURE(FTestComplexArrayPropertyNetSerializer, TestCloneDynamicState)
{
	TestCloneDynamicState();
}

UE_NET_TEST_FIXTURE(FTestComplexArrayPropertyNetSerializer, TestFreeDynamicState)
{
	TestFreeDynamicState();
}

// Tests for array element changemask
UE_NET_TEST_FIXTURE(FTestElementChangeMaskForArrayPropertyNetSerializer, TestChangeMaskIsModifiedWhenAddingElements)
{
	FPropertyReplicationState ReplicationState(ReplicationStateDescriptor);
	TObjectPtr<UClassWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest> Source = NewObject<UClassWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest>();

	FNetBitArrayView ArrayChangeMask = UE::Net::Private::GetMemberChangeMask(ReplicationState.GetStateBuffer(), ReplicationStateDescriptor);
	ArrayChangeMask.ClearAllBits();

	// Add one element at a time until the last bit in the changemask is set and make sure the array is always marked as dirty as well as the corresponding element
	for (unsigned ElementIt = 0, ElementEndIt = FPropertyReplicationState::TArrayElementChangeMaskBits; ElementIt != ElementEndIt; ++ElementIt)
	{
		Source->ArrayOfUint.Add(0);
		ReplicationState.PollPropertyReplicationState(Source.Get());
		
		UE_NET_ASSERT_TRUE(ArrayChangeMask.GetBit(FPropertyReplicationState::TArrayPropertyChangeMaskBitIndex));
		UE_NET_ASSERT_TRUE(ArrayChangeMask.GetBit(FPropertyReplicationState::TArrayElementChangeMaskBitOffset + ElementIt));
		UE_NET_ASSERT_EQ(ArrayChangeMask.CountSetBits(), 2U);

		ArrayChangeMask.ClearAllBits();
	}
}

UE_NET_TEST_FIXTURE(FTestElementChangeMaskForArrayPropertyNetSerializer, TestChangeMaskIsModifiedWhenAddingElementsBeyondChangeMaskCapacity)
{
	FPropertyReplicationState ReplicationState(ReplicationStateDescriptor);
	TObjectPtr<UClassWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest> Source = NewObject<UClassWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest>();

	FNetBitArrayView ArrayChangeMask = UE::Net::Private::GetMemberChangeMask(ReplicationState.GetStateBuffer(), ReplicationStateDescriptor);
	ArrayChangeMask.ClearAllBits();

	// First add elements up to the max element changemask capacity.
	{
		Source->ArrayOfUint.SetNum(FPropertyReplicationState::TArrayElementChangeMaskBits);
		ReplicationState.PollPropertyReplicationState(Source.Get());

		UE_NET_ASSERT_EQ(ArrayChangeMask.CountSetBits(), ArrayChangeMask.GetNumBits());
		ArrayChangeMask.ClearAllBits();
	}

	// Double the array size by adding one element at a time until the last bit in the changemask is set and make sure the array is always marked as dirty as well as the corresponding element.
	for (unsigned ElementIt = 0, ElementEndIt = FPropertyReplicationState::TArrayElementChangeMaskBits; ElementIt != ElementEndIt; ++ElementIt)
	{
		Source->ArrayOfUint.Add(0);
		ReplicationState.PollPropertyReplicationState(Source.Get());
		
		UE_NET_ASSERT_TRUE(ArrayChangeMask.GetBit(FPropertyReplicationState::TArrayPropertyChangeMaskBitIndex));
		UE_NET_ASSERT_TRUE(ArrayChangeMask.GetBit(FPropertyReplicationState::TArrayElementChangeMaskBitOffset + ElementIt));
		UE_NET_ASSERT_EQ(ArrayChangeMask.CountSetBits(), 2U);

		ArrayChangeMask.ClearAllBits();
	}
}

UE_NET_TEST_FIXTURE(FTestElementChangeMaskForArrayPropertyNetSerializer, TestChangeMaskIsModifiedWhenModifyingElements)
{
	FPropertyReplicationState ReplicationState(ReplicationStateDescriptor);
	TObjectPtr<UClassWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest> Source = NewObject<UClassWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest>();

	FNetBitArrayView ArrayChangeMask = UE::Net::Private::GetMemberChangeMask(ReplicationState.GetStateBuffer(), ReplicationStateDescriptor);

	// First add elements up to the max element changemask capacity.
	{
		Source->ArrayOfUint.SetNum(FPropertyReplicationState::TArrayElementChangeMaskBits);
		ReplicationState.PollPropertyReplicationState(Source.Get());

		ArrayChangeMask.ClearAllBits();
	}

	// Modify one element at a time until the last bit in the changemask is set and make sure the array is always marked as dirty as well as the corresponding element
	for (unsigned ElementIt = 0, ElementEndIt = FPropertyReplicationState::TArrayElementChangeMaskBits; ElementIt != ElementEndIt; ++ElementIt)
	{
		Source->ArrayOfUint[ElementIt] ^= 7U;
		ReplicationState.PollPropertyReplicationState(Source.Get());
		
		UE_NET_ASSERT_TRUE(ArrayChangeMask.GetBit(FPropertyReplicationState::TArrayPropertyChangeMaskBitIndex));
		UE_NET_ASSERT_TRUE(ArrayChangeMask.GetBit(FPropertyReplicationState::TArrayElementChangeMaskBitOffset + ElementIt));
		UE_NET_ASSERT_EQ(ArrayChangeMask.CountSetBits(), 2U);

		ArrayChangeMask.ClearAllBits();
	}
}

UE_NET_TEST_FIXTURE(FTestElementChangeMaskForArrayPropertyNetSerializer, TestChangeMaskIsModifiedWhenModifyingElementsBeyondChangeMaskCapacity)
{
	FPropertyReplicationState ReplicationState(ReplicationStateDescriptor);
	TObjectPtr<UClassWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest> Source = NewObject<UClassWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest>();

	FNetBitArrayView ArrayChangeMask = UE::Net::Private::GetMemberChangeMask(ReplicationState.GetStateBuffer(), ReplicationStateDescriptor);
	ArrayChangeMask.ClearAllBits();

	// First add elements up to double the max element changemask capacity.
	{
		Source->ArrayOfUint.SetNum(2*FPropertyReplicationState::TArrayElementChangeMaskBits);
		ReplicationState.PollPropertyReplicationState(Source.Get());

		ArrayChangeMask.ClearAllBits();
	}

	// Double the array size by adding one element at a time until the last bit in the changemask is set and make sure the array is always marked as dirty as well as the corresponding element.
	for (unsigned ElementIt = 0, ElementEndIt = FPropertyReplicationState::TArrayElementChangeMaskBits; ElementIt != ElementEndIt; ++ElementIt)
	{
		Source->ArrayOfUint[FPropertyReplicationState::TArrayElementChangeMaskBits + ElementIt] ^= 7U;
		ReplicationState.PollPropertyReplicationState(Source.Get());
		
		UE_NET_ASSERT_TRUE(ArrayChangeMask.GetBit(FPropertyReplicationState::TArrayPropertyChangeMaskBitIndex));
		UE_NET_ASSERT_TRUE(ArrayChangeMask.GetBit(FPropertyReplicationState::TArrayElementChangeMaskBitOffset + ElementIt));
		UE_NET_ASSERT_EQ(ArrayChangeMask.CountSetBits(), 2U);

		ArrayChangeMask.ClearAllBits();
	}
}

UE_NET_TEST_FIXTURE(FTestElementChangeMaskForArrayPropertyNetSerializer, TestChangeMaskIsModifiedWhenModifyingMultipleArbitraryElements)
{
	FPropertyReplicationState ReplicationState(ReplicationStateDescriptor);
	TObjectPtr<UClassWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest> Source = NewObject<UClassWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest>();

	FNetBitArrayView ArrayChangeMask = UE::Net::Private::GetMemberChangeMask(ReplicationState.GetStateBuffer(), ReplicationStateDescriptor);
	ArrayChangeMask.ClearAllBits();

	constexpr int32 ElementCount = 17;
	// First add some elements
	{
		Source->ArrayOfUint.SetNum(ElementCount);
		ReplicationState.PollPropertyReplicationState(Source.Get());

		ArrayChangeMask.ClearAllBits();
	}

	// Modify some arbitrary elements
	const int32 ElementIndicesToModify[] = {3, 7, 11};
	for (int32 ElementIndex : ElementIndicesToModify)
	{
		Source->ArrayOfUint[ElementIndex] ^= 471147114711U;
	}

	ReplicationState.PollPropertyReplicationState(Source.Get());

	// Verify the elements were marked as dirty
	UE_NET_ASSERT_TRUE(ArrayChangeMask.GetBit(FPropertyReplicationState::TArrayPropertyChangeMaskBitIndex));
	UE_NET_ASSERT_EQ(ArrayChangeMask.CountSetBits(), uint32(UE_ARRAY_COUNT(ElementIndicesToModify) + 1U));

	for (int32 ElementIndex : ElementIndicesToModify)
	{
		UE_NET_ASSERT_TRUE(ArrayChangeMask.GetBit(FPropertyReplicationState::TArrayElementChangeMaskBitOffset + ElementIndex));
	}
}

UE_NET_TEST_FIXTURE(FTestElementChangeMaskForArrayPropertyNetSerializer, TestSerializingFewElementsUsesLessBandwidth)
{
	FPropertyReplicationState ReplicationState(ReplicationStateDescriptor);
	TObjectPtr<UClassWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest> Source = NewObject<UClassWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest>();

	FNetBitArrayView ArrayChangeMask = UE::Net::Private::GetMemberChangeMask(ReplicationState.GetStateBuffer(), ReplicationStateDescriptor);
	ArrayChangeMask.ClearAllBits();

	constexpr int32 ElementCount = 17;
	// First add some elements
	{
		Source->ArrayOfUint.SetNum(ElementCount);

		// Modify all elements
		for (uint32& Element : Source->ArrayOfUint)
		{
			Element ^= 101U;
		}

		ReplicationState.PollPropertyReplicationState(Source.Get());
	}

	NetSerializationContext.SetChangeMask(&ArrayChangeMask);

	FNetSerializeArgs SerializeArgs = {};
	{
		SerializeArgs.Version = ArrayPropertyNetSerializer->Version;
		SerializeArgs.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
		SerializeArgs.Source = NetSerializerValuePointer(StateBuffer0);
		SerializeArgs.ChangeMaskInfo = FNetSerializerChangeMaskParam{0, static_cast<uint16>(ArrayChangeMask.GetNumBits())};
	}

	uint32 BitsWrittenForFullArray = 0;
	{
		Quantize(Source->ArrayOfUint, StateBuffer0);

		Writer.InitBytes(BitStreamBuffer0, sizeof(BitStreamBuffer0));
		ArrayPropertyNetSerializer->Serialize(NetSerializationContext, SerializeArgs);
		Writer.CommitWrites();

		BitsWrittenForFullArray = Writer.GetPosBits();
		UE_NET_ASSERT_GT(BitsWrittenForFullArray, 0U);

		FreeDynamicState(StateBuffer0);
	}

	// Modify a few elements
	{
		ArrayChangeMask.ClearAllBits();

		const int32 ElementIndicesToModify[] = {3, 7, 11};
		for (int32 ElementIndex : ElementIndicesToModify)
		{
			Source->ArrayOfUint[ElementIndex] += 1U;
		}

		ReplicationState.PollPropertyReplicationState(Source.Get());
	}

	uint32 BitsWrittenForPartialArray = 0;
	{
		Quantize(Source->ArrayOfUint, StateBuffer0);

		Writer.InitBytes(BitStreamBuffer0, sizeof(BitStreamBuffer0));
		ArrayPropertyNetSerializer->Serialize(NetSerializationContext, SerializeArgs);
		Writer.CommitWrites();

		BitsWrittenForPartialArray = Writer.GetPosBits();
		UE_NET_ASSERT_GT(BitsWrittenForPartialArray, 0U);

		FreeDynamicState(StateBuffer0);
	}

	UE_NET_ASSERT_LT(BitsWrittenForPartialArray, BitsWrittenForFullArray);
}

// FTestArrayPropertyNetSerializerBase implementation
void FTestArrayPropertyNetSerializerBase::SetUp()
{
	FMockNetSerializerConfig& SerializerConfig = MockNetSerializerConfig;

	SerializerConfig.CallCounter = &MockNetSerializerCallCounter;
	SerializerConfig.ReturnValues = &MockNetSerializerReturnValues;

	MockMemberSerializerDescriptor.Serializer = &UE_NET_GET_SERIALIZER(FMockNetSerializer);
	MockMemberSerializerDescriptor.SerializerConfig = &SerializerConfig;
}

void FTestArrayPropertyNetSerializerBase::TearDown()
{
}

template<typename ArrayType>
void FTestArrayPropertyNetSerializerBase::Quantize(const ArrayType& Array, void* StateBuffer)
{
	FNetQuantizeArgs Args = {};
	Args.Version = ArrayPropertyNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
	Args.Target = NetSerializerValuePointer(StateBuffer);
	Args.Source = NetSerializerValuePointer(&Array);
	ArrayPropertyNetSerializer->Quantize(NetSerializationContext, Args);
}

template<typename ArrayType>
void FTestArrayPropertyNetSerializerBase::Dequantize(const void* StateBuffer, ArrayType& Array)
{
	FNetDequantizeArgs Args = {};
	Args.Version = ArrayPropertyNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(StateBuffer);
	Args.Target = NetSerializerValuePointer(&Array);
	ArrayPropertyNetSerializer->Dequantize(NetSerializationContext, Args);
}

void FTestArrayPropertyNetSerializerBase::FreeDynamicState(void* StateBuffer)
{
	FNetFreeDynamicStateArgs Args = {};
	Args.Version = ArrayPropertyNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(StateBuffer);
	ArrayPropertyNetSerializer->FreeDynamicState(NetSerializationContext, Args);
}

void FTestArrayPropertyNetSerializerBase::Serialize(const void* StateBuffer, void* BitStreamBuffer, SIZE_T BitStreamBufferSize)
{
	Writer.InitBytes(BitStreamBuffer, IntCastChecked<uint32>(BitStreamBufferSize));

	FNetSerializeArgs Args = {};
	Args.Version = ArrayPropertyNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(StateBuffer);
	ArrayPropertyNetSerializer->Serialize(NetSerializationContext, Args);

	Writer.CommitWrites();
}

void FTestArrayPropertyNetSerializerBase::SerializeDelta(const void* StateBuffer, const void* PrevStateBuffer, void* BitStreamBuffer, SIZE_T BitStreamBufferSize)
{
	Writer.InitBytes(BitStreamBuffer, IntCastChecked<uint32>(BitStreamBufferSize));

	FNetSerializeDeltaArgs Args = {};
	Args.Version = ArrayPropertyNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(StateBuffer);
	Args.Prev = NetSerializerValuePointer(PrevStateBuffer);
	ArrayPropertyNetSerializer->SerializeDelta(NetSerializationContext, Args);

	Writer.CommitWrites();
}

// FTestSimpleArrayPropertyNetSerializer implementation
void FTestSimpleArrayPropertyNetSerializer::SetUp()
{
	FTestArrayPropertyNetSerializerBase::SetUp();

	ReplicationStateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(StaticStruct<FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest>());
	UE_NET_ASSERT_EQ_MSG(ReplicationStateDescriptor->MemberCount, uint16(1), "Expected FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest to only contain a single array");

	ArrayPropertyNetSerializerConfig = const_cast<FArrayPropertyNetSerializerConfig*>(static_cast<const FArrayPropertyNetSerializerConfig*>(ReplicationStateDescriptor->MemberSerializerDescriptors[0].SerializerConfig));
	UE_NET_ASSERT_EQ_MSG(ArrayPropertyNetSerializerConfig->StateDescriptor->MemberCount, uint16(1), "Expected array element descriptor to only contain a single member");

	OriginalMemberSerializerDescriptor = ArrayPropertyNetSerializerConfig->StateDescriptor->MemberSerializerDescriptors[0];

	NonEmptyArrayInstance0.ArrayOfUint.SetNumZeroed(3);
	NonEmptyArrayInstance1.ArrayOfUint.SetNumZeroed(3);
	for (int32 It = 0, EndIt = 3; It != EndIt; ++It)
	{
		NonEmptyArrayInstance0.ArrayOfUint[It] = It + 1;
		NonEmptyArrayInstance1.ArrayOfUint[It] = It + 1;
	}

	// Setup a NetSerializationContext that allows memory allocations and can write to a bit stream
	{
		NetSerializationContext = FNetSerializationContext(&Reader, &Writer);
		NetSerializationContext.SetInternalContext(&InternalContext);
	}
}

void FTestSimpleArrayPropertyNetSerializer::TearDown()
{
	ArrayPropertyNetSerializerConfig = nullptr;
	ReplicationStateDescriptor.SafeRelease();

	FTestArrayPropertyNetSerializerBase::TearDown();
}

void FTestSimpleArrayPropertyNetSerializer::SetupForMockSerializer()
{
	FReplicationStateMemberSerializerDescriptor& ArrayElementSerializerDescriptor = const_cast<FReplicationStateMemberSerializerDescriptor&>(ArrayPropertyNetSerializerConfig->StateDescriptor->MemberSerializerDescriptors[0]);
	ArrayElementSerializerDescriptor = MockMemberSerializerDescriptor;
}

void FTestSimpleArrayPropertyNetSerializer::SetupForOriginalSerializer()
{
	FReplicationStateMemberSerializerDescriptor& ArrayElementSerializerDescriptor = const_cast<FReplicationStateMemberSerializerDescriptor&>(ArrayPropertyNetSerializerConfig->StateDescriptor->MemberSerializerDescriptors[0]);
	ArrayElementSerializerDescriptor = OriginalMemberSerializerDescriptor;
}

void FTestSimpleArrayPropertyNetSerializer::TestDescriptorHasDynamicStateTrait()
{
	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(ReplicationStateDescriptor->Traits, EReplicationStateTraits::HasDynamicState));
}

void FTestSimpleArrayPropertyNetSerializer::TestQuantize()
{
	SetupForMockSerializer();

	// The storage need to be cleared in order for the test to succeed.
	UE_NET_ASSERT_EQ(FMemory::Memcmp(StateBuffer0, StateBuffer1, sizeof(StateBuffer0)), 0);

	// Use a non-empty array so we can see that we actually get calls
	FNetQuantizeArgs Args = {};
	Args.Version = ArrayPropertyNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(&NonEmptyArrayInstance0.ArrayOfUint);
	Args.Target = NetSerializerValuePointer(StateBuffer0);
	ArrayPropertyNetSerializer->Quantize(NetSerializationContext, Args);

	UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.Quantize, uint32(NonEmptyArrayInstance0.ArrayOfUint.Num()));

	// While we have no knowledge about the internal quantized state we expect it to not be full of zeros for a non-empty array.
	UE_NET_ASSERT_NE(FMemory::Memcmp(StateBuffer0, StateBuffer1, sizeof(StateBuffer0)), 0);

	FreeDynamicState(StateBuffer0);
}

void FTestSimpleArrayPropertyNetSerializer::TestDequantize()
{
	SetupForOriginalSerializer();

	Quantize(NonEmptyArrayInstance0.ArrayOfUint, StateBuffer0);

	FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest TargetStruct;
	TargetStruct.ArrayOfUint.SetNumZeroed(NonEmptyArrayInstance0.ArrayOfUint.Num() + 1);

	FNetDequantizeArgs Args = {};
	Args.Version = ArrayPropertyNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(StateBuffer0);
	Args.Target = NetSerializerValuePointer(&TargetStruct.ArrayOfUint);
	ArrayPropertyNetSerializer->Dequantize(NetSerializationContext, Args);

	FreeDynamicState(StateBuffer0);

	// Test array size and contents is correct.
	UE_NET_ASSERT_EQ(NonEmptyArrayInstance0.ArrayOfUint.Num(), TargetStruct.ArrayOfUint.Num());
	for (const auto& OriginalValue : MakeArrayView(NonEmptyArrayInstance0.ArrayOfUint))
	{
		const int32 ElementIndex = static_cast<int32>(&OriginalValue - NonEmptyArrayInstance0.ArrayOfUint.GetData());
		const auto& DequantizedValue =  TargetStruct.ArrayOfUint[ElementIndex];
		UE_NET_ASSERT_EQ(OriginalValue, DequantizedValue);
	}
}

void FTestSimpleArrayPropertyNetSerializer::TestCloneDynamicState()
{
	SetupForOriginalSerializer();

	Quantize(NonEmptyArrayInstance0.ArrayOfUint, StateBuffer0);

	FNetCloneDynamicStateArgs Args = {};
	Args.Version = ArrayPropertyNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(StateBuffer0);
	Args.Target = NetSerializerValuePointer(StateBuffer1);
	ArrayPropertyNetSerializer->CloneDynamicState(NetSerializationContext, Args);

	// Dequantize and validate contents to verify that the cloning is correct
	{
		FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest TargetStruct;
		Dequantize(StateBuffer1, TargetStruct.ArrayOfUint);

		UE_NET_ASSERT_EQ(NonEmptyArrayInstance0.ArrayOfUint.Num(), TargetStruct.ArrayOfUint.Num());
		for (const auto& OriginalValue : MakeArrayView(NonEmptyArrayInstance0.ArrayOfUint))
		{
			const int32 ElementIndex = static_cast<uint32>(&OriginalValue - NonEmptyArrayInstance0.ArrayOfUint.GetData());
			const auto& DequantizedValue = TargetStruct.ArrayOfUint[ElementIndex];
			UE_NET_ASSERT_EQ(OriginalValue, DequantizedValue);
		}
	}

	FreeDynamicState(StateBuffer0);
	FreeDynamicState(StateBuffer1);
}

void FTestSimpleArrayPropertyNetSerializer::TestFreeDynamicState()
{
	SetupForMockSerializer();

	Quantize(NonEmptyArrayInstance0.ArrayOfUint, StateBuffer0);

	FNetFreeDynamicStateArgs Args = {};
	Args.Version = ArrayPropertyNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(StateBuffer0);
	ArrayPropertyNetSerializer->FreeDynamicState(NetSerializationContext, Args);

	// We don't really have a good way to test an array of simple elements. However, FreeDynamicState must be re-entrant.
	// So we test that it's working as intended by calling FreeDynamicState again and assume a bad implementation would crash.
	ArrayPropertyNetSerializer->FreeDynamicState(NetSerializationContext, Args);
}

void FTestSimpleArrayPropertyNetSerializer::TestSerialize()
{
	SetupForMockSerializer();

	Quantize(NonEmptyArrayInstance0.ArrayOfUint, StateBuffer0);

	Writer.InitBytes(BitStreamBuffer0, sizeof(BitStreamBuffer0));

	FNetSerializeArgs Args = {};
	Args.Version = ArrayPropertyNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(StateBuffer0);
	ArrayPropertyNetSerializer->Serialize(NetSerializationContext, Args);

	FreeDynamicState(StateBuffer0);

	UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.Serialize, uint32(NonEmptyArrayInstance0.ArrayOfUint.Num()));
}

void FTestSimpleArrayPropertyNetSerializer::TestDeserialize()
{
	SetupForOriginalSerializer();

	// Need to prepare for serializing the data.
	Quantize(NonEmptyArrayInstance0.ArrayOfUint, StateBuffer0);
	Serialize(StateBuffer0, BitStreamBuffer0, sizeof(BitStreamBuffer0));

	// Deserialize.
	{
		Reader.InitBits(BitStreamBuffer0, Writer.GetPosBits());

		FNetDeserializeArgs Args = {};
		Args.Version = ArrayPropertyNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
		Args.Target = NetSerializerValuePointer(StateBuffer1);
		ArrayPropertyNetSerializer->Deserialize(NetSerializationContext, Args);

		UE_NET_ASSERT_FALSE(NetSerializationContext.HasErrorOrOverflow());

		UE_NET_ASSERT_EQ(Reader.GetPosBits(), Writer.GetPosBits());
	}

	FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest TargetStruct;
	TargetStruct.ArrayOfUint.SetNumZeroed(NonEmptyArrayInstance0.ArrayOfUint.Num() + 1);
	Dequantize(StateBuffer0, TargetStruct.ArrayOfUint);

	// Verify state
	{
		UE_NET_ASSERT_EQ(NonEmptyArrayInstance0.ArrayOfUint.Num(), TargetStruct.ArrayOfUint.Num());
		for (const auto& OriginalValue : MakeArrayView(NonEmptyArrayInstance0.ArrayOfUint))
		{
			const int32 ElementIndex = static_cast<int32>(&OriginalValue - NonEmptyArrayInstance0.ArrayOfUint.GetData());
			const auto& DequantizedValue = TargetStruct.ArrayOfUint[ElementIndex];
			UE_NET_ASSERT_EQ(OriginalValue, DequantizedValue);
		}
	}

	FreeDynamicState(StateBuffer0);
	FreeDynamicState(StateBuffer1);
}

void FTestSimpleArrayPropertyNetSerializer::TestSerializeDelta()
{
	SetupForMockSerializer();

	// Delta non-empty array against empty array
	{
		Quantize(EmptyArrayInstance0.ArrayOfUint, StateBuffer0);
		Quantize(NonEmptyArrayInstance0.ArrayOfUint, StateBuffer1);

		Writer.InitBytes(BitStreamBuffer0, sizeof(BitStreamBuffer0));
		MockNetSerializerCallCounter.Reset();

		FNetSerializeDeltaArgs Args = {};
		Args.Version = ArrayPropertyNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
		Args.Source = NetSerializerValuePointer(StateBuffer1);
		Args.Prev = NetSerializerValuePointer(StateBuffer0);
		ArrayPropertyNetSerializer->SerializeDelta(NetSerializationContext, Args);

		// Since we're delta compressing against an empty buffer we do expect one serialize or serializedelta call per array element.
		UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.Serialize + MockNetSerializerCallCounter.SerializeDelta, uint32(NonEmptyArrayInstance0.ArrayOfUint.Num()));

		FreeDynamicState(StateBuffer0);
		FreeDynamicState(StateBuffer1);
	}

	// Delta empty array against non-empty array
	{
		Quantize(EmptyArrayInstance0.ArrayOfUint, StateBuffer0);
		Quantize(NonEmptyArrayInstance0.ArrayOfUint, StateBuffer1);

		Writer.InitBytes(BitStreamBuffer0, sizeof(BitStreamBuffer0));
		MockNetSerializerCallCounter.Reset();

		FNetSerializeDeltaArgs Args = {};
		Args.Version = ArrayPropertyNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
		Args.Source = NetSerializerValuePointer(StateBuffer0);
		Args.Prev = NetSerializerValuePointer(StateBuffer1);
		ArrayPropertyNetSerializer->SerializeDelta(NetSerializationContext, Args);

		// Since we're delta compressing against an empty buffer we do expect one serialize or serializedelta call per array element.
		UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.Serialize + MockNetSerializerCallCounter.SerializeDelta, 0U);

		FreeDynamicState(StateBuffer0);
		FreeDynamicState(StateBuffer1);
	}

	// Delta non-empty array against different non-empty array
	{
		FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest NonEmptyArrayInstance;
		NonEmptyArrayInstance.ArrayOfUint.SetNumZeroed(NonEmptyArrayInstance0.ArrayOfUint.Num());

		Quantize(NonEmptyArrayInstance.ArrayOfUint, StateBuffer0);
		Quantize(NonEmptyArrayInstance0.ArrayOfUint, StateBuffer1);

		Writer.InitBytes(BitStreamBuffer0, sizeof(BitStreamBuffer0));
		MockNetSerializerCallCounter.Reset();

		FNetSerializeDeltaArgs Args = {};
		Args.Version = ArrayPropertyNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
		Args.Source = NetSerializerValuePointer(StateBuffer1);
		Args.Prev = NetSerializerValuePointer(StateBuffer0);
		ArrayPropertyNetSerializer->SerializeDelta(NetSerializationContext, Args);

		// Since we're delta compressing against an empty buffer we do expect one serialize or serializedelta call per array element.
		UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.Serialize + MockNetSerializerCallCounter.SerializeDelta, uint32(NonEmptyArrayInstance0.ArrayOfUint.Num()));

		FreeDynamicState(StateBuffer0);
		FreeDynamicState(StateBuffer1);
	}
}

void FTestSimpleArrayPropertyNetSerializer::TestDeserializeDelta()
{
	SetupForOriginalSerializer();

	// Delta non-empty array against empty array
	{
		Quantize(EmptyArrayInstance0.ArrayOfUint, StateBuffer0);
		Quantize(NonEmptyArrayInstance0.ArrayOfUint, StateBuffer1);
		SerializeDelta(StateBuffer1, StateBuffer0, BitStreamBuffer0, sizeof(BitStreamBuffer0));
		FreeDynamicState(StateBuffer1);

		Reader.InitBits(BitStreamBuffer0, Writer.GetPosBits());

		FNetDeserializeDeltaArgs Args = {};
		Args.Version = ArrayPropertyNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
		Args.Target = NetSerializerValuePointer(StateBuffer1);
		Args.Prev = NetSerializerValuePointer(StateBuffer0);
		ArrayPropertyNetSerializer->DeserializeDelta(NetSerializationContext, Args);

		FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest TargetStruct;
		TargetStruct.ArrayOfUint.SetNumZeroed(NonEmptyArrayInstance0.ArrayOfUint.Num() + 1);
		Dequantize(StateBuffer1, TargetStruct.ArrayOfUint);

		FreeDynamicState(StateBuffer0);
		FreeDynamicState(StateBuffer1);

		// Verify state
		{
			UE_NET_ASSERT_EQ(NonEmptyArrayInstance0.ArrayOfUint.Num(), TargetStruct.ArrayOfUint.Num());
			for (const auto& OriginalValue : MakeArrayView(NonEmptyArrayInstance0.ArrayOfUint))
			{
				const int32 ElementIndex = static_cast<int32>(&OriginalValue - NonEmptyArrayInstance0.ArrayOfUint.GetData());
				const auto& DequantizedValue = TargetStruct.ArrayOfUint[ElementIndex];
				UE_NET_ASSERT_EQ(OriginalValue, DequantizedValue);
			}
		}
	}

	// Delta empty array against non-empty array
	{
		Quantize(EmptyArrayInstance0.ArrayOfUint, StateBuffer1);
		Quantize(NonEmptyArrayInstance0.ArrayOfUint, StateBuffer0);
		SerializeDelta(StateBuffer1, StateBuffer0, BitStreamBuffer0, sizeof(BitStreamBuffer0));
		FreeDynamicState(StateBuffer1);

		Reader.InitBits(BitStreamBuffer0, Writer.GetPosBits());

		FNetDeserializeDeltaArgs Args = {};
		Args.Version = ArrayPropertyNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
		Args.Target = NetSerializerValuePointer(StateBuffer1);
		Args.Prev = NetSerializerValuePointer(StateBuffer0);
		ArrayPropertyNetSerializer->DeserializeDelta(NetSerializationContext, Args);

		FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest TargetStruct;
		TargetStruct.ArrayOfUint.SetNumZeroed(1);
		Dequantize(StateBuffer1, TargetStruct.ArrayOfUint);

		FreeDynamicState(StateBuffer0);
		FreeDynamicState(StateBuffer1);

		UE_NET_ASSERT_EQ(TargetStruct.ArrayOfUint.Num(), 0);
	}

	// Delta non-empty array against different non-empty array
	{
		FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest NonEmptyArrayInstance;
		NonEmptyArrayInstance.ArrayOfUint.SetNumZeroed(NonEmptyArrayInstance0.ArrayOfUint.Num());

		Quantize(NonEmptyArrayInstance.ArrayOfUint, StateBuffer0);
		Quantize(NonEmptyArrayInstance0.ArrayOfUint, StateBuffer1);
		SerializeDelta(StateBuffer1, StateBuffer0, BitStreamBuffer0, sizeof(BitStreamBuffer0));
		FreeDynamicState(StateBuffer1);

		Reader.InitBits(BitStreamBuffer0, Writer.GetPosBits());

		FNetDeserializeDeltaArgs Args = {};
		Args.Version = ArrayPropertyNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
		Args.Target = NetSerializerValuePointer(StateBuffer1);
		Args.Prev = NetSerializerValuePointer(StateBuffer0);
		ArrayPropertyNetSerializer->DeserializeDelta(NetSerializationContext, Args);

		FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest TargetStruct;
		TargetStruct.ArrayOfUint.SetNumZeroed(NonEmptyArrayInstance0.ArrayOfUint.Num() + 1);
		Dequantize(StateBuffer1, TargetStruct.ArrayOfUint);

		FreeDynamicState(StateBuffer0);
		FreeDynamicState(StateBuffer1);

		// Verify state
		{
			UE_NET_ASSERT_EQ(NonEmptyArrayInstance0.ArrayOfUint.Num(), TargetStruct.ArrayOfUint.Num());
			for (const auto& OriginalValue : MakeArrayView(NonEmptyArrayInstance0.ArrayOfUint))
			{
				const int32 ElementIndex = static_cast<int32>(&OriginalValue - NonEmptyArrayInstance0.ArrayOfUint.GetData());
				const auto& DequantizedValue = TargetStruct.ArrayOfUint[ElementIndex];
				UE_NET_ASSERT_EQ(OriginalValue, DequantizedValue);
			}
		}
	}
}

void FTestSimpleArrayPropertyNetSerializer::TestIsEqual()
{
	// Check empty arrays are considered equal
	for (const bool bIsUsingMockSerializer : {false, true})
	{
		bIsUsingMockSerializer ? SetupForMockSerializer() : SetupForOriginalSerializer();

		{
			// The MockNetSerializer shouldn't be called at all but we do the call count check to detect that.
			MockNetSerializerReturnValues.bIsEqual = true;
			MockNetSerializerCallCounter.Reset();

			FNetIsEqualArgs Args = {};
			Args.Version = ArrayPropertyNetSerializer->Version;
			Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
			Args.Source0 = NetSerializerValuePointer(&EmptyArrayInstance0.ArrayOfUint);
			Args.Source1 = NetSerializerValuePointer(&EmptyArrayInstance1.ArrayOfUint);
			Args.bStateIsQuantized = false;

			const bool bEmptyArraysAreEqual = ArrayPropertyNetSerializer->IsEqual(NetSerializationContext, Args);
			UE_NET_ASSERT_TRUE(bEmptyArraysAreEqual);
			if (bIsUsingMockSerializer)
			{
				UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.IsEqual, 0U);
			}
		}

		{
			Quantize(EmptyArrayInstance0.ArrayOfUint, StateBuffer0);
			Quantize(EmptyArrayInstance1.ArrayOfUint, StateBuffer1);

			// The MockNetSerializer shouldn't be called at all but we do the call count check to detect that.
			MockNetSerializerReturnValues.bIsEqual = true;
			MockNetSerializerCallCounter.Reset();

			FNetIsEqualArgs Args = {};
			Args.Version = ArrayPropertyNetSerializer->Version;
			Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
			Args.Source0 = NetSerializerValuePointer(StateBuffer0);
			Args.Source1 = NetSerializerValuePointer(StateBuffer1);
			Args.bStateIsQuantized = true;

			const bool bQuantizedEmptyArraysAreEqual = ArrayPropertyNetSerializer->IsEqual(NetSerializationContext, Args);
			UE_NET_ASSERT_TRUE(bQuantizedEmptyArraysAreEqual);
			if (bIsUsingMockSerializer)
			{
				UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.IsEqual, 0U);
			}

			FreeDynamicState(StateBuffer0);
			FreeDynamicState(StateBuffer1);
		}
	}

	// Check non-empty identical arrays are considered equal
	for (const bool bIsUsingMockSerializer : {false, true})
	{
		bIsUsingMockSerializer ? SetupForMockSerializer() : SetupForOriginalSerializer();

		{
			FNetIsEqualArgs Args = {};
			Args.Version = ArrayPropertyNetSerializer->Version;
			Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
			Args.Source0 = NetSerializerValuePointer(&NonEmptyArrayInstance0.ArrayOfUint);
			Args.Source1 = NetSerializerValuePointer(&NonEmptyArrayInstance1.ArrayOfUint);
			Args.bStateIsQuantized = false;

			MockNetSerializerReturnValues.bIsEqual = true;
			MockNetSerializerCallCounter.Reset();

			const bool bNonEmptyArraysAreEqual = ArrayPropertyNetSerializer->IsEqual(NetSerializationContext, Args);
			UE_NET_ASSERT_TRUE(bNonEmptyArraysAreEqual);
			if (bIsUsingMockSerializer)
			{
				UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.IsEqual, uint32(NonEmptyArrayInstance0.ArrayOfUint.Num()));
			}
		}

		{
			Quantize(NonEmptyArrayInstance0.ArrayOfUint, StateBuffer0);
			Quantize(NonEmptyArrayInstance1.ArrayOfUint, StateBuffer1);

			FNetIsEqualArgs Args = {};
			Args.Version = ArrayPropertyNetSerializer->Version;
			Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
			Args.Source0 = NetSerializerValuePointer(StateBuffer0);
			Args.Source1 = NetSerializerValuePointer(StateBuffer1);
			Args.bStateIsQuantized = true;

			MockNetSerializerReturnValues.bIsEqual = true;
			MockNetSerializerCallCounter.Reset();

			const bool bQuantizedNonEmptyArraysAreEqual = ArrayPropertyNetSerializer->IsEqual(NetSerializationContext, Args);
			UE_NET_ASSERT_TRUE(bQuantizedNonEmptyArraysAreEqual);
			if (bIsUsingMockSerializer)
			{
				UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.IsEqual, uint32(NonEmptyArrayInstance0.ArrayOfUint.Num()));
			}

			FreeDynamicState(StateBuffer0);
			FreeDynamicState(StateBuffer1);
		}
	}

	// Check non-empty non-identical arrays aren't considered equal when using proper element serializer
	for (const bool bIsUsingMockSerializer : {false, true})
	{
		bIsUsingMockSerializer ? SetupForMockSerializer() : SetupForOriginalSerializer();

		FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest NonEmptyArrayInstance = NonEmptyArrayInstance0;
		NonEmptyArrayInstance.ArrayOfUint[1] = NonEmptyArrayInstance0.ArrayOfUint[1] ^ 1U;

		{
			FNetIsEqualArgs Args = {};
			Args.Version = ArrayPropertyNetSerializer->Version;
			Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
			Args.Source0 = NetSerializerValuePointer(&NonEmptyArrayInstance0.ArrayOfUint);
			Args.Source1 = NetSerializerValuePointer(&NonEmptyArrayInstance.ArrayOfUint);
			Args.bStateIsQuantized = false;

			MockNetSerializerReturnValues.bIsEqual = false;
			MockNetSerializerCallCounter.Reset();

			const bool bNonEmptyArraysAreEqual = ArrayPropertyNetSerializer->IsEqual(NetSerializationContext, Args);
			UE_NET_ASSERT_FALSE(bNonEmptyArraysAreEqual);
			if (bIsUsingMockSerializer)
			{
				UE_NET_ASSERT_GE(MockNetSerializerCallCounter.IsEqual, 1U);
			}
		}

		{
			Quantize(NonEmptyArrayInstance0.ArrayOfUint, StateBuffer0);
			Quantize(NonEmptyArrayInstance.ArrayOfUint, StateBuffer1);

			FNetIsEqualArgs Args = {};
			Args.Version = ArrayPropertyNetSerializer->Version;
			Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
			Args.Source0 = NetSerializerValuePointer(StateBuffer0);
			Args.Source1 = NetSerializerValuePointer(StateBuffer1);
			Args.bStateIsQuantized = true;

			MockNetSerializerReturnValues.bIsEqual = false;
			MockNetSerializerCallCounter.Reset();

			const bool bQuantizedNonEmptyArraysAreEqual = ArrayPropertyNetSerializer->IsEqual(NetSerializationContext, Args);
			UE_NET_ASSERT_FALSE(bQuantizedNonEmptyArraysAreEqual);
			if (bIsUsingMockSerializer)
			{
				UE_NET_ASSERT_GE(MockNetSerializerCallCounter.IsEqual, 1U);
			}

			FreeDynamicState(StateBuffer0);
			FreeDynamicState(StateBuffer1);
		}
	}
}

void FTestSimpleArrayPropertyNetSerializer::TestValidate()
{
	// Test empty array is valid
	for (const bool bIsUsingMockSerializer : {false, true})
	{
		bIsUsingMockSerializer ? SetupForMockSerializer() : SetupForOriginalSerializer();

		FNetValidateArgs Args = {};
		Args.Version = ArrayPropertyNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
		Args.Source = NetSerializerValuePointer(&EmptyArrayInstance0.ArrayOfUint);

		MockNetSerializerReturnValues.bValidate = false;
		MockNetSerializerCallCounter.Reset();

		const bool bEmptyArrayIsValid = ArrayPropertyNetSerializer->Validate(NetSerializationContext, Args);
		UE_NET_ASSERT_TRUE(bEmptyArrayIsValid);
	}

	// Test non-empty array is valid
	for (const bool bIsUsingMockSerializer : {false, true})
	{
		bIsUsingMockSerializer ? SetupForMockSerializer() : SetupForOriginalSerializer();

		FNetValidateArgs Args = {};
		Args.Version = ArrayPropertyNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
		Args.Source = NetSerializerValuePointer(&NonEmptyArrayInstance0.ArrayOfUint);

		MockNetSerializerReturnValues.bValidate = true;
		MockNetSerializerCallCounter.Reset();

		const bool bNonEmptyArrayIsValid = ArrayPropertyNetSerializer->Validate(NetSerializationContext, Args);
		UE_NET_ASSERT_TRUE(bNonEmptyArrayIsValid);
		if (bIsUsingMockSerializer)
		{
			UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.Validate, uint32(NonEmptyArrayInstance0.ArrayOfUint.Num()));
		}
	}
}

// FTestComplexArrayPropertyNetSerializer implementation
void FTestComplexArrayPropertyNetSerializer::SetUp()
{
	FTestArrayPropertyNetSerializerBase::SetUp();

	ReplicationStateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(StaticStruct<FStructWithDynamicArrayOfComplexTypeForArrayPropertyNetSerializerTest>());
	UE_NET_ASSERT_EQ_MSG(ReplicationStateDescriptor->MemberCount, uint16(1), "Expected FStructWithDynamicArrayOfComplexTypeForArrayPropertyNetSerializerTest to only contain a single array");

	ArrayPropertyNetSerializerConfig = const_cast<FArrayPropertyNetSerializerConfig*>(static_cast<const FArrayPropertyNetSerializerConfig*>(ReplicationStateDescriptor->MemberSerializerDescriptors[0].SerializerConfig));
	UE_NET_ASSERT_EQ_MSG(ArrayPropertyNetSerializerConfig->StateDescriptor->MemberCount, uint16(1), "Expected array element descriptor to only contain a single member");
	UE_NET_ASSERT_TRUE(IsUsingStructNetSerializer(ArrayPropertyNetSerializerConfig->StateDescriptor->MemberSerializerDescriptors[0]));
	UE_NET_ASSERT_TRUE(IsUsingArrayPropertyNetSerializer(static_cast<const FStructNetSerializerConfig*>(ArrayPropertyNetSerializerConfig->StateDescriptor->MemberSerializerDescriptors[0].SerializerConfig)->StateDescriptor->MemberSerializerDescriptors[0]));

	const FStructNetSerializerConfig* StructSerializerConfig = static_cast<const FStructNetSerializerConfig*>(ArrayPropertyNetSerializerConfig->StateDescriptor->MemberSerializerDescriptors[0].SerializerConfig);
	InnerArrayPropertyNetSerializerConfig = const_cast<FArrayPropertyNetSerializerConfig*>(static_cast<const FArrayPropertyNetSerializerConfig*>(StructSerializerConfig->StateDescriptor->MemberSerializerDescriptors[0].SerializerConfig));
	OriginalMemberSerializerDescriptor = InnerArrayPropertyNetSerializerConfig->StateDescriptor->MemberSerializerDescriptors[0];

	NonEmptyArrayInstance0.ArrayOfStructWithArray.Empty(3);
	NonEmptyArrayInstance0.ArrayOfStructWithArray.SetNum(3);
	NonEmptyArrayInstance1.ArrayOfStructWithArray.Empty(3);
	NonEmptyArrayInstance1.ArrayOfStructWithArray.SetNum(3);
	for (int32 It = 0, EndIt = 3; It != EndIt; ++It)
	{
		NonEmptyArrayInstance0.ArrayOfStructWithArray[It].ArrayOfUint.SetNumZeroed(It + 1);
		NonEmptyArrayInstance1.ArrayOfStructWithArray[It].ArrayOfUint.SetNumZeroed(It + 1);
	}

	// Setup a NetSerializationContext that allows memory allocations and can write to a bit stream
	{
		NetSerializationContext = FNetSerializationContext(&Reader, &Writer);
		NetSerializationContext.SetInternalContext(&InternalContext);
	}
}

void FTestComplexArrayPropertyNetSerializer::TearDown()
{
	ArrayPropertyNetSerializerConfig = nullptr;
	InnerArrayPropertyNetSerializerConfig = nullptr;
	ReplicationStateDescriptor.SafeRelease();

	FTestArrayPropertyNetSerializerBase::TearDown();
}

void FTestComplexArrayPropertyNetSerializer::SetupForMockSerializer()
{
	FReplicationStateMemberSerializerDescriptor& ArrayElementSerializerDescriptor = const_cast<FReplicationStateMemberSerializerDescriptor&>(InnerArrayPropertyNetSerializerConfig->StateDescriptor->MemberSerializerDescriptors[0]);
	ArrayElementSerializerDescriptor = MockMemberSerializerDescriptor;
}

void FTestComplexArrayPropertyNetSerializer::SetupForOriginalSerializer()
{
	FReplicationStateMemberSerializerDescriptor& ArrayElementSerializerDescriptor = const_cast<FReplicationStateMemberSerializerDescriptor&>(InnerArrayPropertyNetSerializerConfig->StateDescriptor->MemberSerializerDescriptors[0]);
	ArrayElementSerializerDescriptor = OriginalMemberSerializerDescriptor;
}

void FTestComplexArrayPropertyNetSerializer::TestQuantize()
{
	SetupForMockSerializer();

	// The storage need to be cleared in order for the test to succeed.
	UE_NET_ASSERT_EQ(FMemory::Memcmp(StateBuffer0, StateBuffer1, sizeof(StateBuffer0)), 0);

	// Use a non-empty array so we can see that we actually get calls
	FNetQuantizeArgs Args = {};
	Args.Version = ArrayPropertyNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(&NonEmptyArrayInstance0.ArrayOfStructWithArray);
	Args.Target = NetSerializerValuePointer(StateBuffer0);
	ArrayPropertyNetSerializer->Quantize(NetSerializationContext, Args);

	UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.Quantize, uint32(TotalNumberOfElementsInNonEmptyArray));

	// While we have no knowledge about the internal quantized state we expect it to not be full of zeros for a non-empty array.
	UE_NET_ASSERT_NE(FMemory::Memcmp(StateBuffer0, StateBuffer1, sizeof(StateBuffer0)), 0);

	FreeDynamicState(StateBuffer0);
}

void FTestComplexArrayPropertyNetSerializer::TestDequantize()
{
	SetupForOriginalSerializer();

	Quantize(NonEmptyArrayInstance0.ArrayOfStructWithArray, StateBuffer0);

	FStructWithDynamicArrayOfComplexTypeForArrayPropertyNetSerializerTest TargetStruct;
	TargetStruct.ArrayOfStructWithArray.SetNum(NonEmptyArrayInstance0.ArrayOfStructWithArray.Num() + 1);

	FNetDequantizeArgs Args = {};
	Args.Version = ArrayPropertyNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(StateBuffer0);
	Args.Target = NetSerializerValuePointer(&TargetStruct.ArrayOfStructWithArray);
	ArrayPropertyNetSerializer->Dequantize(NetSerializationContext, Args);

	FreeDynamicState(StateBuffer0);

	// Test array size and contents is correct.
	UE_NET_ASSERT_EQ(NonEmptyArrayInstance0.ArrayOfStructWithArray.Num(), TargetStruct.ArrayOfStructWithArray.Num());
	for (const auto& OriginalValue : MakeArrayView(NonEmptyArrayInstance0.ArrayOfStructWithArray))
	{
		const int32 ElementIndex = static_cast<int32>(&OriginalValue - NonEmptyArrayInstance0.ArrayOfStructWithArray.GetData());
		const auto& DequantizedValue =  TargetStruct.ArrayOfStructWithArray[ElementIndex];
		UE_NET_ASSERT_EQ(OriginalValue.ArrayOfUint.Num(), DequantizedValue.ArrayOfUint.Num());

		for (const auto& OriginalInnerValue : MakeArrayView(OriginalValue.ArrayOfUint))
		{
			const int32 InnerElementIndex = static_cast<int32>(&OriginalInnerValue - OriginalValue.ArrayOfUint.GetData());
			const auto& DequantizedInnerValue = DequantizedValue.ArrayOfUint[InnerElementIndex];
			UE_NET_ASSERT_EQ(OriginalInnerValue, DequantizedInnerValue);
		}
	}
}

void FTestComplexArrayPropertyNetSerializer::TestCloneDynamicState()
{
	SetupForOriginalSerializer();

	Quantize(NonEmptyArrayInstance0.ArrayOfStructWithArray, StateBuffer0);

	FNetCloneDynamicStateArgs Args = {};
	Args.Version = ArrayPropertyNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(StateBuffer0);
	Args.Target = NetSerializerValuePointer(StateBuffer1);
	ArrayPropertyNetSerializer->CloneDynamicState(NetSerializationContext, Args);

	// Dequantize and validate contents to verify that the cloning is correct
	{
		FStructWithDynamicArrayOfComplexTypeForArrayPropertyNetSerializerTest TargetStruct;
		Dequantize(StateBuffer1, TargetStruct.ArrayOfStructWithArray);

		UE_NET_ASSERT_EQ(NonEmptyArrayInstance0.ArrayOfStructWithArray.Num(), TargetStruct.ArrayOfStructWithArray.Num());
		for (const auto& OriginalValue : MakeArrayView(NonEmptyArrayInstance0.ArrayOfStructWithArray))
		{
			const int32 ElementIndex = static_cast<int32>(&OriginalValue - NonEmptyArrayInstance0.ArrayOfStructWithArray.GetData());
			const auto& DequantizedValue = TargetStruct.ArrayOfStructWithArray[ElementIndex];
			UE_NET_ASSERT_EQ(OriginalValue.ArrayOfUint.Num(), DequantizedValue.ArrayOfUint.Num());

			for (const auto& OriginalInnerValue : MakeArrayView(OriginalValue.ArrayOfUint))
			{
				const int32 InnerElementIndex = static_cast<int32>(&OriginalInnerValue - OriginalValue.ArrayOfUint.GetData());
				const auto& DequantizedInnerValue = DequantizedValue.ArrayOfUint[InnerElementIndex];
				UE_NET_ASSERT_EQ(OriginalInnerValue, DequantizedInnerValue);
			}
		}
	}

	FreeDynamicState(StateBuffer0);
	FreeDynamicState(StateBuffer1);
}

void FTestComplexArrayPropertyNetSerializer::TestFreeDynamicState()
{
	// Special setup to trap recursive calls to FreeDynamicState
	FReplicationStateMemberSerializerDescriptor& ArrayElementSerializerDescriptor = const_cast<FReplicationStateMemberSerializerDescriptor&>(ArrayPropertyNetSerializerConfig->StateDescriptor->MemberSerializerDescriptors[0]);
	FReplicationStateMemberSerializerDescriptor OriginalElementSerializerDescriptor = ArrayElementSerializerDescriptor;
	ArrayElementSerializerDescriptor = MockMemberSerializerDescriptor;

	Quantize(NonEmptyArrayInstance0.ArrayOfStructWithArray, StateBuffer0);

	FNetFreeDynamicStateArgs Args = {};
	Args.Version = ArrayPropertyNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(ArrayPropertyNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(StateBuffer0);
 	ArrayPropertyNetSerializer->FreeDynamicState(NetSerializationContext, Args);

	// Restore the original element descriptor before asserting as it may return from the function
	{
		ArrayElementSerializerDescriptor = OriginalElementSerializerDescriptor;
	}

	// As the array contains a struct with array elements we expect each array element to get a FreeDynamicState call.
	UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.FreeDynamicState, uint32(NonEmptyArrayInstance0.ArrayOfStructWithArray.Num()));
}

// FTestElementChangeMaskForArrayPropertyNetSerializer
void FTestElementChangeMaskForArrayPropertyNetSerializer::SetUp()
{
	FTestArrayPropertyNetSerializerBase::SetUp();

	FReplicationStateDescriptorBuilder::FResult Descriptors;
	FReplicationStateDescriptorBuilder::CreateDescriptorsForClass(Descriptors, UClassWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest::StaticClass());
	UE_NET_ASSERT_GT(Descriptors.Num(), 0);
	ReplicationStateDescriptor = Descriptors[0];
	ArrayPropertyNetSerializerConfig = const_cast<FArrayPropertyNetSerializerConfig*>(static_cast<const FArrayPropertyNetSerializerConfig*>(ReplicationStateDescriptor->MemberSerializerDescriptors[0].SerializerConfig));

	NetSerializationContext = FNetSerializationContext(&Reader, &Writer);
	NetSerializationContext.SetInternalContext(&InternalContext);
}

void FTestElementChangeMaskForArrayPropertyNetSerializer::TearDown()
{
	ReplicationStateDescriptor.SafeRelease();

	FTestArrayPropertyNetSerializerBase::TearDown();
}

}

void UClassWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	DOREPLIFETIME(ThisClass, ArrayOfUint);
}
