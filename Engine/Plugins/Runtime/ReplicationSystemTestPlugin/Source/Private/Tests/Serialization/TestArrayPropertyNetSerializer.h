// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/ObjectMacros.h"
#include "TestArrayPropertyNetSerializer.generated.h"

USTRUCT()
struct FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<uint32> ArrayOfUint;
};

USTRUCT()
struct FStructWithDynamicArrayOfComplexTypeForArrayPropertyNetSerializerTest
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FStructWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest> ArrayOfStructWithArray;
};

UCLASS()
class UClassWithDynamicArrayOfPrimitiveTypeForArrayPropertyNetSerializerTest : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Replicated)
	TArray<uint32> ArrayOfUint;

protected:
	virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
};
