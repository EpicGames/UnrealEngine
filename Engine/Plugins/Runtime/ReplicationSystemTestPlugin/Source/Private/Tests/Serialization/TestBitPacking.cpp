// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/Serialization/BitPacking.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include <limits>

namespace UE::Net::Private
{

template<typename SourceType, uint8 BitCount = sizeof(SourceType)*8U>
class TTestBitPacking : public FNetworkAutomationTestSuiteFixture
{
protected:
	void TestSerialize();

	void InitWriter();
	void InitReaderFromWriter();

	void SerializeDelta(const SourceType Value, const SourceType PrevValue);
	void DeserializeDelta(SourceType& OutValue, const SourceType PrevValue);

	enum : uint32
	{
		BufferSize = 64,
	};

	FNetBitStreamReader Reader;
	FNetBitStreamWriter Writer;

	alignas(16) uint8 StateBuffer[BufferSize];

	static const uint8 SmallBitCountTable[];
	static const uint32 SmallBitCountTableEntryCount;

	static const SourceType Values[];
	static const SIZE_T ValueCount;
};

template<typename SourceType, uint8 BitCount> const uint8 TTestBitPacking<SourceType, BitCount>::SmallBitCountTable[] =
{
	0, BitCount/2, BitCount - 2,
};
template<typename SourceType, uint8 BitCount> const uint32 TTestBitPacking<SourceType, BitCount>::SmallBitCountTableEntryCount = sizeof(SmallBitCountTable)/sizeof(SmallBitCountTable[0]);

template<typename SourceType, uint8 BitCount> const SourceType TTestBitPacking<SourceType, BitCount>::Values[] =
{
	TIsSigned<SourceType>::Value ? (SourceType(-(typename TSignedIntType<sizeof(SourceType)>::Type)(std::numeric_limits<SourceType>::max() >> (sizeof(SourceType)*8U - BitCount)) - SourceType(1))) : SourceType(0), // MinValue
	SourceType(Values[0] + SourceType(1)),
	std::numeric_limits<SourceType>::max() >> (sizeof(SourceType)*8U - BitCount), // Max value for desired bitcount
	SourceType(SourceType(Values[0] + Values[2])/SourceType(2)),
};
template<typename SourceType, uint8 BitCount> const SIZE_T TTestBitPacking<SourceType, BitCount>::ValueCount = sizeof(Values)/sizeof(Values[0]);

#define UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(TestClassName, SourceType, BitCount) \
class TestClassName : public TTestBitPacking<SourceType, BitCount> \
{ \
protected: \
}; \
\
UE_NET_TEST_FIXTURE(TestClassName, TestSerialize) \
{ \
	TestSerialize(); \
} \

// Signed integer
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTestInt8BitPacking, int8, 8);
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTest7BitIntBitPacking, int8, 7);
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTestInt16BitPacking, int16, 16);
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTest14BitInt16BitPacking, int16, 14);
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTestInt32BitPacking, int32, 32);
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTest29BitInt32BitPacking, int32, 29);
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTestInt64BitPacking, int64, 64);
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTest33BitInt64BitPacking, int64, 33);

// Unsigned integer
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTestUint8BitPacking, uint8, 8);
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTest7BitUintBitPacking, uint8, 7);
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTestUint16BitPacking, uint16, 16);
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTest14BitUint16BitPacking, uint16, 14);
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTestUint32BitPacking, uint32, 32);
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTest29BitUint32BitPacking, uint32, 29);
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTestUint64BitPacking, uint64, 64);
UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST(FTest33BitUint64BitPacking, uint64, 33);

#undef UE_NET_IMPLEMENT_DELTA_SERIALIZATION_TEST

//
template<typename SourceType, uint8 BitCount>
void TTestBitPacking<SourceType, BitCount>::InitWriter()
{
	Writer.InitBytes(StateBuffer, sizeof(StateBuffer));
}

template<typename SourceType, uint8 BitCount>
void TTestBitPacking<SourceType, BitCount>::InitReaderFromWriter()
{
	Reader.InitBits(StateBuffer, Writer.GetPosBits());
}

template<typename SourceType, uint8 BitCount>
void TTestBitPacking<SourceType, BitCount>::SerializeDelta(const SourceType Value, const SourceType PrevValue)
{
	typedef typename TUnsignedIntType<sizeof(SourceType)>::Type UnsignedType;
	typedef typename TSignedIntType<sizeof(SourceType)>::Type SignedType;
	if (TIsSigned<SourceType>::Value)
	{
		SerializeIntDelta(Writer, SignedType(Value), SignedType(PrevValue), SmallBitCountTable, SmallBitCountTableEntryCount, BitCount);
	}
	else
	{
		SerializeUintDelta(Writer, UnsignedType(Value), UnsignedType(PrevValue), SmallBitCountTable, SmallBitCountTableEntryCount, BitCount);
	}

	Writer.CommitWrites();
}

template<typename SourceType, uint8 BitCount>
void TTestBitPacking<SourceType, BitCount>::DeserializeDelta(SourceType& OutValue, const SourceType PrevValue)
{
	typedef typename TUnsignedIntType<sizeof(SourceType)>::Type UnsignedType;
	typedef typename TSignedIntType<sizeof(SourceType)>::Type SignedType;
	if (TIsSigned<SourceType>::Value)
	{
		DeserializeIntDelta(Reader, reinterpret_cast<SignedType&>(OutValue), SignedType(PrevValue), SmallBitCountTable, SmallBitCountTableEntryCount, BitCount);
	}
	else
	{
		DeserializeUintDelta(Reader, reinterpret_cast<UnsignedType&>(OutValue), UnsignedType(PrevValue), SmallBitCountTable, SmallBitCountTableEntryCount, BitCount);
	}
}

template<typename SourceType, uint8 BitCount>
void TTestBitPacking<SourceType, BitCount>::TestSerialize()
{
	// Try each value against all values
	for (SIZE_T ValueIt = 0, ValueEndIt = ValueCount; ValueIt != ValueEndIt; ++ValueIt)
	{
		const SourceType Value = Values[ValueIt];
		for (SIZE_T PrevValueIt = 0, PrevValueEndIt = ValueCount; PrevValueIt != ValueEndIt; ++PrevValueIt)
		{
			const SourceType PrevValue = Values[PrevValueIt];

			InitWriter();
			SerializeDelta(Value, PrevValue);
			UE_NET_ASSERT_FALSE_MSG(Writer.IsOverflown(), "Serialize delta between " << Value << " and " << PrevValue << " caused bit stream overflow");

			InitReaderFromWriter();
			SourceType DeserializedValue;
			DeserializeDelta(DeserializedValue, PrevValue);
			UE_NET_ASSERT_FALSE_MSG(Reader.IsOverflown(), "Deserialize delta between " << Value << " and " << PrevValue << " caused bit stream overflow");

			UE_NET_ASSERT_EQ_MSG(DeserializedValue, Value, "Failed serializing value " << Value << " with previous value " << PrevValue << " using " << (TIsSigned<SourceType>::Value ? "signed " : "unsigned ") << BitCount << "-bit integer");
		}
	}
}

}