// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "BitfieldTestTypes.h"
#include "Iris/Serialization/InternalNetSerializers.h"

namespace UE::Net::Private
{

static FTestMessage& operator<<(FTestMessage& Message, const FBitfieldNetSerializerConfig& Config)
{
	return Message << "BitMask: " << Config.BitMask;
}

template<typename StructType, typename SourceType>
class FTestBitfieldNetSerializer : public FTestNetSerializerFixture
{
public:
	FTestBitfieldNetSerializer() : FTestNetSerializerFixture(UE_NET_GET_SERIALIZER(FBitfieldNetSerializer)) {}

protected:
	void TestIsEqual();
	void TestValidate();
	void TestQuantize();
	void TestSerialize();
	void TestDequantize();

	virtual void SetUp() override;

	static TArray<FBitfieldNetSerializerConfig> Configs;
};

#define UE_NET_IMPLEMENT_BITFIELD_NETSERIALIZER(TestClassName, StructName) \
class TestClassName : public FTestBitfieldNetSerializer<StructName, uint8> \
{ \
}; \
UE_NET_TEST_FIXTURE(TestClassName, TestIsEqual) \
{ \
	TestIsEqual(); \
} \
UE_NET_TEST_FIXTURE(TestClassName, TestValidate) \
{ \
	TestValidate(); \
} \
UE_NET_TEST_FIXTURE(TestClassName, TestQuantize) \
{ \
	TestQuantize(); \
} \
UE_NET_TEST_FIXTURE(TestClassName, TestSerialize) \
{ \
	TestSerialize(); \
} \
UE_NET_TEST_FIXTURE(TestClassName, TestDequantize) \
{ \
	TestDequantize(); \
} \

UE_NET_IMPLEMENT_BITFIELD_NETSERIALIZER(FTestBitfield64NetSerializer, FTestUint64Bitfield);
UE_NET_IMPLEMENT_BITFIELD_NETSERIALIZER(FTestBitfield32NetSerializer, FTestUint32Bitfield);
UE_NET_IMPLEMENT_BITFIELD_NETSERIALIZER(FTestBitfield16NetSerializer, FTestUint16Bitfield);
UE_NET_IMPLEMENT_BITFIELD_NETSERIALIZER(FTestBitfield8NetSerializer, FTestUint8Bitfield);

#undef UE_NET_IMPLEMENT_BITFIELD_NETSERIALIZER

//
template<typename StructType, typename SourceType> TArray<FBitfieldNetSerializerConfig> FTestBitfieldNetSerializer<StructType, SourceType>::Configs;

template<typename StructType, typename SourceType>
void FTestBitfieldNetSerializer<StructType, SourceType>::SetUp()
{
	SetDoTaintBuffersBeforeTest(true);

	static bool bInitialized;
	if (bInitialized)
	{
		return;
	}

	const UStruct* Struct = StaticStruct<StructType>();
	for (const FProperty* Property = Struct->PropertyLink; Property != nullptr; Property = Property->PropertyLinkNext)
	{
		const FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property);
		if (BoolProperty == nullptr)
		{
			continue;
		}

		FBitfieldNetSerializerConfig Config;
		const bool bIsValidBitfield = InitBitfieldNetSerializerConfigFromProperty(Config, BoolProperty);
		UE_NET_ASSERT_TRUE_MSG(bIsValidBitfield, "Unable to initialize bitfield from " << Struct->GetFName() << "::" << BoolProperty->GetNameCPP());
		Configs.Add(Config);
	}

	bInitialized = true;
}

template<typename StructType, typename SourceType>
void FTestBitfieldNetSerializer<StructType, SourceType>::TestIsEqual()
{
	constexpr SourceType Zero = 0;

	for (const FBitfieldNetSerializerConfig& Config : Configs)
	{
		const SourceType Bit = Config.BitMask;
		bool bIsQuantized;

		bIsQuantized = false;
		const bool bIsDequantizedEqual = FTestNetSerializerFixture::TestIsEqual(Config, NetSerializerValuePointer(&Bit), NetSerializerValuePointer(&Bit), true, bIsQuantized);
		UE_NET_ASSERT_TRUE_MSG(bIsDequantizedEqual, "Set bit dequantized equality test failed with config " << Config);

		bIsQuantized = true;
		const bool bIsQuantizedEqual = FTestNetSerializerFixture::TestIsEqual(Config, NetSerializerValuePointer(&Bit), NetSerializerValuePointer(&Bit), true, bIsQuantized);
		UE_NET_ASSERT_TRUE_MSG(bIsQuantizedEqual, "Set bit quantized equality test failed with config " << Config);

		bIsQuantized = false;
		const bool bZeroIsNotDequantizedEqualToOne = FTestNetSerializerFixture::TestIsEqual(Config, NetSerializerValuePointer(&Bit), NetSerializerValuePointer(&Zero), false, bIsQuantized);
		UE_NET_ASSERT_TRUE_MSG(bZeroIsNotDequantizedEqualToOne, "Set bit was dequantized equal to zero with config " << Config);

		bIsQuantized = true;
		const bool bZeroIsNotQuantizedEqualToOne = FTestNetSerializerFixture::TestIsEqual(Config, NetSerializerValuePointer(&Bit), NetSerializerValuePointer(&Zero), false, bIsQuantized);
		UE_NET_ASSERT_TRUE_MSG(bZeroIsNotQuantizedEqualToOne, "Set bit was quantized equal to zero with config " << Config);
	}
}

template<typename StructType, typename SourceType>
void FTestBitfieldNetSerializer<StructType, SourceType>::TestValidate()
{
	constexpr SourceType Zero = 0;
	constexpr SourceType One = ~SourceType(0);
	constexpr bool bExpectedResult = true;

	for (const FBitfieldNetSerializerConfig& Config : Configs)
	{
		bool bIsValid;
		bIsValid = FTestNetSerializerFixture::TestValidate(Config, NetSerializerValuePointer(&Zero), bExpectedResult);
		UE_NET_ASSERT_TRUE_MSG(bIsValid, "Unset bit was determined invalid with config " << Config);

		bIsValid = FTestNetSerializerFixture::TestValidate(Config, NetSerializerValuePointer(&One), bExpectedResult);
		UE_NET_ASSERT_TRUE_MSG(bIsValid, "Set bit was determined invalid with config " << Config);
	}
}

template<typename StructType, typename SourceType>
void FTestBitfieldNetSerializer<StructType, SourceType>::TestQuantize()
{
	constexpr SourceType Zero = 0;
	constexpr SourceType One = ~SourceType(0);
	constexpr bool bExpectedResult = true;

	for (const FBitfieldNetSerializerConfig& Config : Configs)
	{
		bool bQuantize0Works = FTestNetSerializerFixture::TestQuantize(Config, NetSerializerValuePointer(&Zero));
		UE_NET_ASSERT_TRUE_MSG(bQuantize0Works, "0 bit could not be quantized with config " << Config);

		bool bQuantize1Works = FTestNetSerializerFixture::TestQuantize(Config, NetSerializerValuePointer(&One));
		UE_NET_ASSERT_TRUE_MSG(bQuantize1Works, " 1 bit could not be quantized with config " << Config);
	}
}

template<typename StructType, typename SourceType>
void FTestBitfieldNetSerializer<StructType, SourceType>::TestSerialize()
{
	constexpr bool bQuantizedCompare = true;

	for (const FBitfieldNetSerializerConfig& Config : Configs)
	{
		// We test four values, two of which have the relevant bit set to zero and two which have the relevant bit set to 1.
		// We make sure the serializer does not touch any of the other bits. 
		SourceType Values[] = {SourceType(~Config.BitMask), SourceType(0), Config.BitMask, SourceType(~SourceType(0))};
		for (SIZE_T ValueIt = 0, ValueEndIt = UE_ARRAY_COUNT(Values); ValueIt != ValueEndIt; ++ValueIt)
		{
			const SourceType Value = Values[ValueIt];
			const bool bSerializeWorks = FTestNetSerializerFixture::TestSerialize(Config, NetSerializerValuePointer(&Value), NetSerializerValuePointer(&Value), bQuantizedCompare);
			UE_NET_ASSERT_TRUE_MSG(bSerializeWorks, Value << " could not be serialized with config " << Config);
		}
	}
}

template<typename StructType, typename SourceType>
void FTestBitfieldNetSerializer<StructType, SourceType>::TestDequantize()
{
	const auto& EqualityFunc = [](NetSerializerValuePointer Value0, NetSerializerValuePointer Value1) -> bool { return *reinterpret_cast<SourceType*>(Value0) == *reinterpret_cast<SourceType*>(Value1); };

	for (const FBitfieldNetSerializerConfig& Config : Configs)
	{
		SourceType Value = 0;
		const SourceType OriginalTarget = ~SourceType(Config.BitMask);
		SourceType Target = OriginalTarget;

		FNetQuantizeArgs QuantizeArgs;
		QuantizeArgs.Version = SerializerVersionOverride;
		QuantizeArgs.NetSerializerConfig = &Config;
		QuantizeArgs.Source = NetSerializerValuePointer(&Value);
		QuantizeArgs.Target = NetSerializerValuePointer(QuantizedBuffer[0]);
		Serializer.Quantize(Context, QuantizeArgs);
		UE_NET_ASSERT_FALSE_MSG(Context.HasError(), "Quantize() of value reported an error. Config " << Config);

		Writer.InitBytes(BitStreamBuffer, sizeof(BitStreamBuffer));
		FNetSerializeArgs SerializeArgs;
		SerializeArgs.Version = SerializerVersionOverride;
		SerializeArgs.NetSerializerConfig = &Config;
		SerializeArgs.Source = QuantizeArgs.Target;
		Serializer.Serialize(Context, SerializeArgs);
		Writer.CommitWrites();
		UE_NET_ASSERT_FALSE_MSG(Writer.IsOverflown(), "FNetBitStreamWriter overflowed. Config " << Config);
		UE_NET_ASSERT_FALSE_MSG(Context.HasError(), "Serialize() reported an error. Config " << Config);

		Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());
		FNetDeserializeArgs DeserializeArgs;
		DeserializeArgs.Version = SerializerVersionOverride;
		DeserializeArgs.NetSerializerConfig = &Config;
		DeserializeArgs.Target = NetSerializerValuePointer(QuantizedBuffer[1]);
		Serializer.Deserialize(Context, DeserializeArgs);
		UE_NET_ASSERT_FALSE_MSG(Reader.IsOverflown(), "FNetBitStreamReader overflowed. Config " << Config);
		UE_NET_ASSERT_FALSE_MSG(Context.HasError(), "Deserialize() reported an error. Config " << Config);

		// Need to dequantize the deserialized value
		FNetDequantizeArgs DequantizeArgs;
		DequantizeArgs.Version = SerializerVersionOverride;
		DequantizeArgs.NetSerializerConfig = &Config;
		DequantizeArgs.Source = DeserializeArgs.Target;
		DequantizeArgs.Target = NetSerializerValuePointer(&Target);
		Serializer.Dequantize(Context, DequantizeArgs);
		UE_NET_ASSERT_FALSE_MSG(Context.HasError(), "Dequantize() of deserialized value reported an error. Config " << Config);

		UE_NET_ASSERT_EQ_MSG(OriginalTarget, *reinterpret_cast<const SourceType*>(DequantizeArgs.Target), "Dequantize touched bits that should not have been modified. Config " << Config);
	}
}

}
