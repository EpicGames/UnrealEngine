// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/NetSerializers.h"

namespace UE::Net::Private
{

static FTestMessage& PrintBoolNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

class FTestBoolNetSerializer : public TTestNetSerializerFixture<PrintBoolNetSerializerConfig, bool>
{
public:
	FTestBoolNetSerializer() : TTestNetSerializerFixture<PrintBoolNetSerializerConfig, bool>(UE_NET_GET_SERIALIZER(FBoolNetSerializer)) {}

	void TestIsEqual();
	void TestValidate();
	void TestSerialize();

protected:
	virtual void SetUp() override;


	static bool Values[];
	static const SIZE_T ValueCount;
};

UE_NET_TEST_FIXTURE(FTestBoolNetSerializer, HasTestValues)
{
	UE_NET_ASSERT_GT_MSG(ValueCount, SIZE_T(0), "No test values found");
}

UE_NET_TEST_FIXTURE(FTestBoolNetSerializer, TestValuesAreAsExpected)
{
	const bool* TestValue0 = &Values[0];
	const uint8 ExpectedValue0 = 0;
	UE_NET_ASSERT_EQ(memcmp(TestValue0, &ExpectedValue0, 1), 0);

	const bool* TestValue1 = &Values[1];
	const uint8 ExpectedValue1 = 1;
	UE_NET_ASSERT_EQ(memcmp(TestValue1, &ExpectedValue1, 1), 0);

	const bool* TestValue2 = &Values[2];
	const uint8 ExpectedValue2 = 128;
	UE_NET_ASSERT_EQ(memcmp(TestValue2, &ExpectedValue2, 1), 0);

	const bool* TestValue3 = &Values[3];
	const uint8 ExpectedValue3 = 255;
	UE_NET_ASSERT_EQ(memcmp(TestValue3, &ExpectedValue3, 1), 0);
}

UE_NET_TEST_FIXTURE(FTestBoolNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestBoolNetSerializer, TestValidate)
{
	TestValidate();
}

UE_NET_TEST_FIXTURE(FTestBoolNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestBoolNetSerializer, TestHasDefaultConfig)
{
	UE_NET_ASSERT_NE(Serializer.DefaultConfig, nullptr);
}

//
bool FTestBoolNetSerializer::Values[] = {false, true, static_cast<bool>(254), static_cast<bool>(255)};
const SIZE_T FTestBoolNetSerializer::ValueCount = sizeof(Values)/sizeof(Values[0]);

void FTestBoolNetSerializer::SetUp()
{
	// Setting up bool test values where some are "uninitialized" isn't trivial. Memcpy does the trick.
	const uint8 TestValues[] = {0, 1, 128, 255};
	static_assert(sizeof(TestValues) == sizeof(Values), "Test value count mismatch");

	FPlatformMemory::Memcpy(Values, TestValues, sizeof(TestValues));
}

void FTestBoolNetSerializer::TestIsEqual()
{
	bool CompareValues[2][sizeof(Values)/sizeof(Values[0])];
	bool ExpectedResults[2][sizeof(Values)/sizeof(Values[0])];

	// See comment in SetUp()
	FPlatformMemory::Memcpy(CompareValues[0], Values, sizeof(Values));
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		ExpectedResults[0][ValueIt] = true;
	}
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		uint8 Value;
		uint8 NotValue;
		FPlatformMemory::Memcpy(&Value, &Values[ValueIt], 1);
		NotValue = Value ? 0 : 1;
		FPlatformMemory::Memcpy(&CompareValues[1][ValueIt], &NotValue, 1);

		ExpectedResults[1][ValueIt] = false;
	}

	// Do two rounds of testing, one where we compare each value with itself and one where we compare against a guaranteed non-equal value.
	// We will only do an unquantized compare since we know these serializers uses a default quantize implementation.
	FBoolNetSerializerConfig Config;
	for (SIZE_T TestRoundIt = 0, TestRoundEndIt = 2; TestRoundIt != TestRoundEndIt; ++TestRoundIt)
	{
		constexpr bool bQuantizedCompare = false;
		bool bSuccess = TTestNetSerializerFixture<PrintBoolNetSerializerConfig, bool>::TestIsEqual(Values, CompareValues[TestRoundIt], ExpectedResults[TestRoundIt], ValueCount, Config, bQuantizedCompare);
		if (!bSuccess)
		{
			return;
		}
	}
}

void FTestBoolNetSerializer::TestValidate()
{
	bool ExpectedResults[sizeof(Values)/sizeof(Values[0])];
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		ExpectedResults[ValueIt] = (ValueIt <= 1); // We know the first two test values are the only "proper" bools
	}

	FBoolNetSerializerConfig Config;
	TTestNetSerializerFixture<PrintBoolNetSerializerConfig, bool>::TestValidate(Values, ExpectedResults, ValueCount, Config);
}

void FTestBoolNetSerializer::TestSerialize()
{
	bool ExpectedValues[sizeof(Values)/sizeof(Values[0])];
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		uint8 Value;
		FPlatformMemory::Memcpy(&Value, &Values[ValueIt], 1);
		ExpectedValues[ValueIt] = Value != 0;
	}

	FBoolNetSerializerConfig Config;
	constexpr bool bQuantizedCompare = false;
	auto&& CompareFunction = [](NetSerializerValuePointer Value0, NetSerializerValuePointer Value1) -> bool { return *reinterpret_cast<bool*>(Value0) == *reinterpret_cast<bool*>(Value1); };
	TFunctionRef<bool(NetSerializerValuePointer Value0, NetSerializerValuePointer Value1)> CompareFunc = CompareFunction;
	TTestNetSerializerFixture<PrintBoolNetSerializerConfig, bool>::TestSerialize(Values, ExpectedValues, ValueCount, Config, bQuantizedCompare, CompareFunc);
}

}
