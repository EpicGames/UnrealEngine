// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"

#include "TestCompressedChangeMaskData.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetBitStreamUtil.h"

namespace UE::Net::Private
{

class FCompressedChangeMaskTest : public FNetworkAutomationTestSuiteFixture
{
public:
	typedef FNetBitArrayView::StorageWordType FWordType;

	typedef void (*WriteSparseBitArrayFunc)(FNetBitStreamWriter* Writer, const uint32* Data, uint32 BitCount, ESparseBitArraySerializationHint Hint);
	typedef void (*ReadSparseBitArrayFunc)(FNetBitStreamReader* Reader, uint32* OutData, uint32 BitCount, ESparseBitArraySerializationHint Hint);

	typedef void (*WriteSparseBitArrayDeltaFunc)(FNetBitStreamWriter* Writer, const uint32* Data, const uint32* BaseData, uint32 BitCount);
	typedef void (*ReadSparseBitArrayDeltaFunc)(FNetBitStreamReader* Reader, uint32* OutData, const uint32* BaseData, uint32 BitCount);
	
protected:
	struct FChangeMaskStats
	{
		uint32 TotalChangeMaskBits = 0U;
		uint32 TotalSetBits = 0U;
		uint32 TotalWrittenBits = 0U;
		uint32 Count = 0U;
	};

	class FChangeMaskBuffer
	{
	public:
		FChangeMaskBuffer()
		{
			Reset();
		}

		inline FWordType* GetBuffer() { return &Buffer[0]; }
		uint32 GetBufferCapacity() const { return sizeof(Buffer); }

	private:
		void Reset() {}

		FWordType Buffer[1024];
	};

	template <typename T>
	bool ValidateTestData(const T& TestData)
	{
		const uint32 TestDataWordCount = UE_ARRAY_COUNT(TestData);

		bool bResult = true;
		SIZE_T Offset = 0;

		while (Offset + 2U < TestDataWordCount)
		{
			const FWordType BitCount = TestData[Offset];
			const FWordType Invert = TestData[Offset + 1];

			Offset += FNetBitArrayView::CalculateRequiredWordCount(BitCount) + 2U;
		}

		return Offset == TestDataWordCount;
	}

	template<typename FunctorT, typename T> void ForAllEntries(FunctorT&& Functor, const T& TestData) const
	{
		const uint32 TestDataWordCount = UE_ARRAY_COUNT(TestData);

		bool bResult = true;
		SIZE_T Offset = 0;

		while (Offset + 2U < TestDataWordCount)
		{
			const FWordType BitCount = TestData[Offset];
			const FWordType Invert = TestData[Offset + 1];
			const FWordType* Data = &TestData[Offset + 2];

			Functor(BitCount, Invert, Data);

			Offset += FNetBitArrayView::CalculateRequiredWordCount(BitCount) + 2U;
		}
	}

	template <typename T>
	bool ValidateDeltaTestData(const T& DeltaTestData, const T& BaseTestData)
	{
		const uint32 TestDataWordCount = UE_ARRAY_COUNT(DeltaTestData);
		if (TestDataWordCount != UE_ARRAY_COUNT(BaseTestData))
		{
			return false;
		}

		bool bResult = true;
		SIZE_T Offset = 0;

		while (Offset + 2U < TestDataWordCount)
		{
			const FWordType BitCount = DeltaTestData[Offset];
			const FWordType Invert = DeltaTestData[Offset + 1];

			if (BitCount != BaseTestData[Offset])
			{
				return false;
			}

			Offset += FNetBitArrayView::CalculateRequiredWordCount(BitCount) + 2U;
		}

		return Offset == TestDataWordCount;
	}


	template<typename FunctorT, typename T> void ForAllEntriesDelta(FunctorT&& Functor, const T& DeltaTestData, const T& BaseTestData) const
	{
		const uint32 TestDataWordCount = UE_ARRAY_COUNT(DeltaTestData);

		check(TestDataWordCount == UE_ARRAY_COUNT(BaseTestData));

		bool bResult = true;
		SIZE_T Offset = 0;

		while (Offset + 2U < TestDataWordCount)
		{
			const FWordType BitCount = DeltaTestData[Offset];
			const FWordType Invert = BaseTestData[Offset + 1];
			const FWordType* DeltaData = &DeltaTestData[Offset + 2];
			const FWordType* BaseData = &BaseTestData[Offset + 2];

			Functor(BitCount, Invert, DeltaData, BaseData);

			Offset += FNetBitArrayView::CalculateRequiredWordCount(BitCount) + 2U;
		}
	}


	FChangeMaskBuffer WriteBuffer;
	FChangeMaskBuffer TempChangeMaskData;

	FNetBitStreamWriter Writer;
	FNetBitStreamReader Reader;
};

extern const FCompressedChangeMaskTest::FWordType FullTestData[];

UE_NET_TEST_FIXTURE(FCompressedChangeMaskTest, TestValidateTestData)
{
	const FCompressedChangeMaskTest::FWordType BadChangeMaskTestData[] =
	{
		/* BitCount */ 32U, /* Invert */ 0U, /* Data */ 0x1, 0x2,
		/* BitCount */ 64U, /* Invert */ 0U, /* Data */ 0x1, 0x2
	};

	const FCompressedChangeMaskTest::FWordType GoodChangeMaskTestData[] =
	{
		/* BitCount */ 32U, /* Invert */ 0U, /* Data */ 0x1,
		/* BitCount */ 32U, /* Invert */ 1U, /* Data */ 0xFFFFFFFF,
	};
	
	UE_NET_ASSERT_FALSE(ValidateTestData(BadChangeMaskTestData));
	UE_NET_ASSERT_TRUE(ValidateTestData(GoodChangeMaskTestData));
}

UE_NET_TEST_FIXTURE(FCompressedChangeMaskTest, TestBasic)
{
	Writer.InitBytes(WriteBuffer.GetBuffer(), WriteBuffer.GetBufferCapacity());

	const uint32 StartPos = Writer.GetPosBits();
	UE_NET_ASSERT_EQ(StartPos, 0U);

	const bool bIsOverflown = Writer.IsOverflown();
	UE_NET_ASSERT_FALSE(bIsOverflown);
}

UE_NET_TEST_FIXTURE(FCompressedChangeMaskTest, TestValidateFullTestData)
{
	UE_NET_ASSERT_TRUE(ValidateTestData(FullTestData));
}

UE_NET_TEST_FIXTURE(FCompressedChangeMaskTest, WriteEmptyChangeMask)
{
	const FCompressedChangeMaskTest::FWordType TestData[] =
	{
		/* BitCount ,  Invert, Data,,, */
		32U, 0U, 0x0,
		1024U, 0U, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0, 0x0,
	};

	UE_NET_ASSERT_TRUE(ValidateTestData(TestData));

	auto&& TestWriteFunc = [this](FWordType BitCount, FWordType Invert, const FWordType* Data) 
	{
		Writer.InitBytes(WriteBuffer.GetBuffer(), WriteBuffer.GetBufferCapacity());
		WriteSparseBitArray(&Writer, Data, BitCount, Invert ? ESparseBitArraySerializationHint::ContainsMostlyOnes : ESparseBitArraySerializationHint::None);

		// Debug data
		const FNetBitArrayView ChangeMask = MakeNetBitArrayView(Data, BitCount);
		const uint32 NumSetBits = ChangeMask.CountSetBits();
		//UE_LOGF(LogTemp, Log, "WroteSparseArray with SetBits %u / %u using %u Bits", NumSetBits, BitCount, Writer.GetPosBits());
	};

	ForAllEntries(TestWriteFunc, TestData);
}

UE_NET_TEST_FIXTURE(FCompressedChangeMaskTest, WriteChangeMasks)
{
	FChangeMaskStats ChangeMaskStats;
	
	auto&& TestWriteFunc = [this, &ChangeMaskStats](FWordType BitCount, FWordType Invert, const FWordType* Data)
	{
		Writer.InitBytes(WriteBuffer.GetBuffer(), WriteBuffer.GetBufferCapacity());
		WriteSparseBitArray(&Writer, Data, BitCount, Invert ? ESparseBitArraySerializationHint::ContainsMostlyOnes : ESparseBitArraySerializationHint::None);

		// Debug data
		const FNetBitArrayView ChangeMask = MakeNetBitArrayView(Data, BitCount);
		const uint32 NumSetBits = ChangeMask.CountSetBits();
		//UE_LOGF(LogTemp, Log, "WroteSparseArray with SetBits %u / %u using %u Bits", NumSetBits, BitCount, Writer.GetPosBits());

		ChangeMaskStats.TotalChangeMaskBits += BitCount;
		ChangeMaskStats.TotalSetBits += NumSetBits;
		ChangeMaskStats.TotalWrittenBits += Writer.GetPosBits();
		++ChangeMaskStats.Count;
	};

	ForAllEntries(TestWriteFunc, FullTestData);

	// Log stats
	if (ChangeMaskStats.TotalChangeMaskBits > 0)
	{
		float Ratio = static_cast<float>(ChangeMaskStats.TotalWrittenBits) / static_cast<float>(ChangeMaskStats.TotalChangeMaskBits);
		float WrittenBitsPerDirtyBit = static_cast<float>(ChangeMaskStats.TotalWrittenBits) / static_cast<float>(ChangeMaskStats.TotalSetBits);

		UE_NET_LOG(TEXT("")) << TStringBuilder<128>().Appendf(TEXT("WriteChangeMasks::Summary SetBits %u / TotalBits %u WrittenBits %u Ratio %f BitsPerDirtyBit %f"), ChangeMaskStats.TotalSetBits, ChangeMaskStats.TotalChangeMaskBits, ChangeMaskStats.TotalWrittenBits, Ratio, WrittenBitsPerDirtyBit);
	}
}

UE_NET_TEST_FIXTURE(FCompressedChangeMaskTest, WriteAndReadChangeMasks)
{
	WriteSparseBitArrayFunc WriteFunc = WriteSparseBitArray;
	ReadSparseBitArrayFunc ReadFunc = ReadSparseBitArray;
	
	auto&& TestWriteAndReadFunc = [this, WriteFunc, ReadFunc](FWordType BitCount, FWordType Invert, const FWordType* Data)
	{		
		Writer.InitBytes(WriteBuffer.GetBuffer(), WriteBuffer.GetBufferCapacity());
		WriteFunc(&Writer, Data, BitCount, Invert ? ESparseBitArraySerializationHint::ContainsMostlyOnes : ESparseBitArraySerializationHint::None);
		Writer.CommitWrites();

		UE_NET_ASSERT_FALSE(Writer.IsOverflown());

		Reader.InitBits(WriteBuffer.GetBuffer(), Writer.GetPosBits());

		// Init changemask
		FNetBitArrayView RcvdChangeMask((FWordType*)TempChangeMaskData.GetBuffer(), BitCount, FNetBitArrayView::ResetOnInit);

		// Read changemask
		ReadFunc(&Reader, TempChangeMaskData.GetBuffer(), BitCount, Invert ? ESparseBitArraySerializationHint::ContainsMostlyOnes : ESparseBitArraySerializationHint::None);

		UE_NET_ASSERT_FALSE(Reader.IsOverflown());
		UE_NET_ASSERT_EQ(Writer.GetPosBits(), Reader.GetPosBits());

		// Verify change mask
		const FNetBitArrayView OriginalChangeMask = MakeNetBitArrayView(Data, BitCount);

		UE_NET_ASSERT_TRUE(OriginalChangeMask == RcvdChangeMask);
	};

	ForAllEntries(TestWriteAndReadFunc, SingleChangeMaskTestData);
	ForAllEntries(TestWriteAndReadFunc, FullTestData);
	ForAllEntries(TestWriteAndReadFunc, VeryLargeChangeMaskTestData);
}

UE_NET_TEST_FIXTURE(FCompressedChangeMaskTest, WriteAndReadSparseBitArrayDelta)
{
	WriteSparseBitArrayDeltaFunc WriteFunc = WriteSparseBitArrayDelta;
	ReadSparseBitArrayDeltaFunc ReadFunc = ReadSparseBitArrayDelta;

	// Test some basic scenarios for delta

	const FCompressedChangeMaskTest::FWordType BaseTestData[] =
	{
		// Set one bit in each word
		/* BitCount */ 33U, /* Invert */ 0U, /* Data */ 0x0U, 0x0U,
		// Set highest bit
		/* BitCount */ 32U, /* Invert */ 0U, /* Data */ 0x1U,
		// Set many bit
		/* BitCount */ 33U, /* Invert */ 0U, /* Data */ 0x0U, 0x0U,
		// Clear single bit
		/* BitCount */ 32U, /* Invert */ 0U, /* Data */ 0xFFFFFFFFU,
		// Test inverted clear single bit (invert does not make much sense as we delta using xor
		/* BitCount */ 32U, /* Invert */ 1U, /* Data */ 0xFFFFFFFFU,
		// Test no change
		/* BitCount */ 33U, /* Invert */ 0U, /* Data */ 0x0U, 0x0U,
		// Test no change
		/* BitCount */ 33U, /* Invert */ 0U, /* Data */ 0xFFFFFFFFU, 0x1U, 
		// Test all changed
		/* BitCount */ 33U, /* Invert */ 0U, /* Data */ 0x0U, 0x0U,
		// Test all changed
		/* BitCount */ 33U, /* Invert */ 0U, /* Data */ 0xFFFFFFFFU, 0x1U, 
	};

	const FCompressedChangeMaskTest::FWordType DeltaTestData[] =
	{
		// Set one bit in each word
		/* BitCount */ 33U, /* Invert */ 0U, /* Data */ 0x1U, 0x1U,
		// Set highest bit
		/* BitCount */ 32U, /* Invert */ 0U, /* Data */ 0x1U | 0x8U,
		// Set many bit
		/* BitCount */ 33U, /* Invert */ 0U, /* Data */ 0xFFFFFFFFU, 0x0U,
		// Clear single bit
		/* BitCount */ 32U, /* Invert */ 0U, /* Data */ 0xFFFFFFFEU,
		// Test inverted clear single bit (invert does not make much sense as we delta using xor
		/* BitCount */ 32U, /* Invert */ 1U, /* Data */ 0xFFFFFFFFU & 0xFFFFFFFEU,
		// Test no change
		/* BitCount */ 33U, /* Invert */ 0U, /* Data */ 0x0U, 0x0U,
		// Test no change
		/* BitCount */ 33U, /* Invert */ 0U, /* Data */ 0xFFFFFFFFU, 0x1U, 
		// Test all changed
		/* BitCount */ 33U, /* Invert */ 0U, /* Data */ 0xFFFFFFFFU, 0x1U,
		// Test all changed
		/* BitCount */ 33U, /* Invert */ 0U, /* Data */ 0x0U, 0x0U,
	};


	UE_NET_ASSERT_TRUE(ValidateDeltaTestData(DeltaTestData, BaseTestData));	

	auto&& TestWriteAndReadFunc = [this, WriteFunc, ReadFunc](FWordType BitCount, FWordType Invert, const FWordType* Data, const FWordType* BaseData)
	{		
		Writer.InitBytes(WriteBuffer.GetBuffer(), WriteBuffer.GetBufferCapacity());
		WriteFunc(&Writer, Data, BaseData, BitCount);
		Writer.CommitWrites();

		UE_NET_ASSERT_FALSE(Writer.IsOverflown());

		Reader.InitBits(WriteBuffer.GetBuffer(), Writer.GetPosBits());

		// Init changemask
		FNetBitArrayView RcvdChangeMask((FWordType*)TempChangeMaskData.GetBuffer(), BitCount, FNetBitArrayView::ResetOnInit);

		// Read changemask
		ReadFunc(&Reader, TempChangeMaskData.GetBuffer(), BaseData, BitCount);

		UE_NET_ASSERT_FALSE(Reader.IsOverflown());
		UE_NET_ASSERT_EQ(Writer.GetPosBits(), Reader.GetPosBits());

		// Verify change mask
		const FNetBitArrayView OriginalChangeMask = MakeNetBitArrayView(Data, BitCount);

		UE_NET_ASSERT_TRUE(OriginalChangeMask == RcvdChangeMask);
	};

	ForAllEntriesDelta(TestWriteAndReadFunc, DeltaTestData, BaseTestData);
}


}
