// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Core/BitTwiddling.h"
#include "Iris/Serialization/EnumNetSerializers.h"
#include "Iris/Serialization/InternalEnumNetSerializers.h"
#include "EnumTestTypes.h"
#include <limits>

namespace UE::Net::Private
{

template<typename SerializerConfig> static FTestMessage& PrintEnumNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	const SerializerConfig& Config = static_cast<const SerializerConfig&>(InConfig);
	Message << "LowerBound: " << Config.LowerBound << " UpperBound: " << Config.UpperBound << " BitCount: " << Config.BitCount;
	if (const UEnum* Enum = Config.Enum)
	{
		Message << " Enum: " << Enum->GetName();
	}

	return Message;
}

template<typename SerializerConfig, typename SourceType, typename EnumType>
class FTestEnumIntNetSerializer : public TTestNetSerializerFixture<PrintEnumNetSerializerConfig<SerializerConfig>, SourceType>
{
	typedef TTestNetSerializerFixture<PrintEnumNetSerializerConfig<SerializerConfig>, SourceType> Super;

public:
	FTestEnumIntNetSerializer(const FNetSerializer& Serializer) : Super(Serializer) {}

	void TestIsEqual();
	void TestValidate();
	void TestQuantize();
	void TestSerialize();
	void TestSerializeDelta();

protected:
	virtual void SetUp() override;

	static TArray<SourceType> Values;
	static TArray<SourceType> InvalidValues;
	static SerializerConfig Config;
	static const UEnum* Enum;
};

#define UE_NET_IMPLEMENT_ENUMINT_NETSERIALIZER_TEST(TestClassName, NetSerializerType, ConfigType, EnumType, BackingType) \
class TestClassName : public FTestEnumIntNetSerializer<ConfigType, BackingType, EnumType> \
{ \
public: \
	TestClassName() : FTestEnumIntNetSerializer<ConfigType, BackingType, EnumType>(UE_NET_GET_SERIALIZER(NetSerializerType)) {} \
}; \
\
UE_NET_TEST_FIXTURE(TestClassName, TestIsEqual) \
{ \
	TestIsEqual(); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestValidate) \
{ \
	TestValidate(); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestQuantize) \
{ \
	TestQuantize(); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestSerialize) \
{ \
	TestSerialize(); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestSerializeDelta) \
{ \
	TestSerializeDelta(); \
}

// EnumNetSerializer tests
UE_NET_IMPLEMENT_ENUMINT_NETSERIALIZER_TEST(FTestEnumInt8NetSerializer, FEnumInt8NetSerializer, FEnumInt8NetSerializerConfig, ETestInt8Enum, int8);
UE_NET_IMPLEMENT_ENUMINT_NETSERIALIZER_TEST(FTestEnumInt16NetSerializer, FEnumInt16NetSerializer, FEnumInt16NetSerializerConfig, ETestInt16Enum, int16);
UE_NET_IMPLEMENT_ENUMINT_NETSERIALIZER_TEST(FTestEnumInt32NetSerializer, FEnumInt32NetSerializer, FEnumInt32NetSerializerConfig, ETestInt32Enum, int32);
UE_NET_IMPLEMENT_ENUMINT_NETSERIALIZER_TEST(FTestEnumInt64NetSerializer, FEnumInt64NetSerializer, FEnumInt64NetSerializerConfig, ETestInt64Enum, int64);

UE_NET_IMPLEMENT_ENUMINT_NETSERIALIZER_TEST(FTestEnumUint8NetSerializer, FEnumUint8NetSerializer, FEnumUint8NetSerializerConfig, ETestUint8Enum, uint8);
UE_NET_IMPLEMENT_ENUMINT_NETSERIALIZER_TEST(FTestEnumUint16NetSerializer, FEnumUint16NetSerializer, FEnumUint16NetSerializerConfig, ETestUint16Enum, uint16);
UE_NET_IMPLEMENT_ENUMINT_NETSERIALIZER_TEST(FTestEnumUint32NetSerializer, FEnumUint32NetSerializer, FEnumUint32NetSerializerConfig, ETestUint32Enum, uint32);
UE_NET_IMPLEMENT_ENUMINT_NETSERIALIZER_TEST(FTestEnumUint64NetSerializer, FEnumUint64NetSerializer, FEnumUint64NetSerializerConfig, ETestUint64Enum, uint64);

#undef UE_NET_IMPLEMENT_ENUMINT_NETSERIALIZER_TEST


template<typename SerializerConfig, typename SourceType, typename EnumType> TArray<SourceType> FTestEnumIntNetSerializer<SerializerConfig, SourceType, EnumType>::Values;
template<typename SerializerConfig, typename SourceType, typename EnumType> TArray<SourceType> FTestEnumIntNetSerializer<SerializerConfig, SourceType, EnumType>::InvalidValues;
template<typename SerializerConfig, typename SourceType, typename EnumType> SerializerConfig FTestEnumIntNetSerializer<SerializerConfig, SourceType, EnumType>::Config;
template<typename SerializerConfig, typename SourceType, typename EnumType> const UEnum* FTestEnumIntNetSerializer<SerializerConfig, SourceType, EnumType>::Enum;


template<typename SerializerConfig, typename SourceType, typename EnumType>
void FTestEnumIntNetSerializer<SerializerConfig, SourceType, EnumType>::SetUp()
{
	static bool bIsInitialized;
	if (bIsInitialized)
	{
		return;
	}

	Enum = StaticEnum<EnumType>();
	InitEnumNetSerializerConfig(Config, Enum);

	// Setup test values
	{
		// NumEnums actually also contain the generated _MAX enum value which might not even be a valid value by the backed type. Skip it!
		const int32 EnumValueCount = Enum->NumEnums() - 1;

		using LargeIntegerType = std::conditional_t<TIsSigned<SourceType>::Value, int64, uint64>;

		// Valid values
		TArray<SourceType> TempValues;
		TempValues.Reserve(EnumValueCount);
		for (int32 EnumIt = 0, EnumEndIt = EnumValueCount; EnumIt != EnumEndIt; ++EnumIt)
		{
			const LargeIntegerType Value = static_cast<LargeIntegerType>(Enum->GetValueByIndex(EnumIt));
			TempValues.Add(static_cast<SourceType>(Value));
		}
		Values = MoveTemp(TempValues);

		// Invalid values
		TArray<SourceType> TempInvalidValues;
		TempInvalidValues.Reserve(3);
		if (Config.LowerBound > std::numeric_limits<SourceType>::min())
		{
			TempInvalidValues.Add(Config.LowerBound - SourceType(1));
		}

		if (Config.UpperBound < std::numeric_limits<SourceType>::max())
		{
			TempInvalidValues.Add(Config.UpperBound + SourceType(1));
		}

		// Try adding an invalid value between the smallest and largest values found
		for (SourceType Value = Config.LowerBound, UpperBound = Config.UpperBound; Value != UpperBound; ++Value)
		{
			if (!Enum->IsValidEnumValue(Value))
			{
				TempInvalidValues.Add(Value);
				break;
			}
		}
		InvalidValues = MoveTemp(TempInvalidValues);
	}

	bIsInitialized = true;
}

template<typename SerializerConfig, typename SourceType, typename EnumType>
void FTestEnumIntNetSerializer<SerializerConfig, SourceType, EnumType>::TestIsEqual()
{
	TArray<SourceType> CompareValues[2];
	TArray<bool> ExpectedResults[2];

	CompareValues[0] = Values;
	ExpectedResults[0].Reserve(Values.Num());
	for (int32 ValueIt = 0, ValueEndIt = Values.Num(); ValueIt != ValueEndIt; ++ValueIt)
	{
		ExpectedResults[0].Add(true);
	}

	CompareValues[1].Reserve(Values.Num());
	ExpectedResults[1].Reserve(Values.Num());
	for (int32 ValueIt = 0, ValueEndIt = Values.Num(); ValueIt != ValueEndIt; ++ValueIt)
	{
		CompareValues[1].Add(Values[(ValueIt + 1) % ValueEndIt]);
		ExpectedResults[1].Add(Values[ValueIt] == Values[(ValueIt + 1) % ValueEndIt]);
	}

	// Do two rounds of testing per config, one where we compare each value with itself and one where we compare against a value in range.
	for (SIZE_T TestRoundIt = 0, TestRoundEndIt = 2; TestRoundIt != TestRoundEndIt; ++TestRoundIt)
	{
		// Do both quantized and regular compares
		for (SIZE_T CompareIt = 0; CompareIt != 2; ++CompareIt)
		{
			bool bQuantizedCompare = CompareIt == 0;
			const bool bSuccess = Super::TestIsEqual(Values.GetData(), CompareValues[TestRoundIt].GetData(), ExpectedResults[TestRoundIt].GetData(), Values.Num(), Config, bQuantizedCompare);
			if (!bSuccess)
			{
				return;
			}
		}
	}
}

template<typename SerializerConfig, typename SourceType, typename EnumType>
void FTestEnumIntNetSerializer<SerializerConfig, SourceType, EnumType>::TestValidate()
{
	// Check valid values
	{
		TArray<bool> ExpectedResults;
		ExpectedResults.SetNumUninitialized(Values.Num());
		for (int32 ValueIt = 0, ValueEndIt = Values.Num(); ValueIt != ValueEndIt; ++ValueIt)
		{
			ExpectedResults[ValueIt] = true;
		}

		const bool bSuccess = Super::TestValidate(Values.GetData(), ExpectedResults.GetData(), Values.Num(), Config);
		if (!bSuccess)
		{
			return;
		}
	}

	// Check invalid values
	{
		UE_NET_EXPECT_GT_MSG(InvalidValues.Num(), 0, "Unable to test EnumIntSerializer Validate with invalid values.");

		TArray<bool> ExpectedResults;
		ExpectedResults.SetNumZeroed(InvalidValues.Num());
		const bool bSuccess = Super::TestValidate(InvalidValues.GetData(), ExpectedResults.GetData(), InvalidValues.Num(), Config);
		if (!bSuccess)
		{
			return;
		}
	}
}

template<typename SerializerConfig, typename SourceType, typename EnumType>
void FTestEnumIntNetSerializer<SerializerConfig, SourceType, EnumType>::TestQuantize()
{
	Super::TestQuantize(Values.GetData(), Values.Num(), Config);
}

template<typename SerializerConfig, typename SourceType, typename EnumType>
void FTestEnumIntNetSerializer<SerializerConfig, SourceType, EnumType>::TestSerialize()
{
	constexpr bool bQuantizedCompare = false;
	Super::TestSerialize(Values.GetData(), Values.GetData(), Values.Num(), Config, bQuantizedCompare);
}

template<typename SerializerConfig, typename SourceType, typename EnumType>
void FTestEnumIntNetSerializer<SerializerConfig, SourceType, EnumType>::TestSerializeDelta()
{
	Super::TestSerializeDelta(Values.GetData(), Values.Num(), Config);
}

}
