// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestFieldPathNetSerializer.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Iris/ReplicationSystem/NetTokenStore.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/InternalNetSerializers.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetExportContext.h"
#include "Iris/ReplicationSystem/NetExports.h"
#include "Iris/ReplicationSystem/ObjectReferenceCache.h"
#include "Iris/Serialization/NetReferenceCollector.h"

namespace UE::Net
{

FTestMessage& operator<<(FTestMessage& Message, const FFieldPath& FieldPath)
{
	return Message << FieldPath.ToString();
}

}

namespace UE::Net::Private
{

class FTestFieldPathNetSerializer : public FReplicationSystemServerClientTestFixture
{
public:
	FTestFieldPathNetSerializer()
	: Super()
	{
	}

protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	void SetUpTestValues();

	// Helpers
	void Quantize(FNetSerializationContext& Context, const FFieldPath& Source, void* StateBuffer);
	void Dequantize(FNetSerializationContext& Context, const void* StateBuffer, FFieldPath& Target);
	bool IsEqual(FNetSerializationContext& Context, const FFieldPath& FieldPath0, const FFieldPath& FieldPath1);
	void FreeDynamicState(FNetSerializationContext& Context, void* StateBuffer);
	void CollectAndAppendExports(FNetSerializationContext& Context, void* StateBuffer);

	// Tests
	void TestQuantize();
	void TestCloneDynamicState();
	void TestFreeDynamicState();
	void TestSerialize();
	void TestRoundtripSerialize();
	void TestSerializeDelta();
	void TestIsEqual();
	void TestValidate();

protected:
	typedef FReplicationSystemServerClientTestFixture Super;

	static const FNetSerializer* FieldPathNetSerializer;
	static const FFieldPathNetSerializerConfig SerializerConfig;
	static TArray<FFieldPath> TestValues;
	static TArray<FFieldPath> UnresolvableTestValues;

	FNetSerializationContext ServerNetSerializationContext;
	FInternalNetSerializationContext ServerInternalContext;

	FNetSerializationContext ClientNetSerializationContext;
	FInternalNetSerializationContext ClientInternalContext;

	FNetBitStreamReader Reader;
	FNetBitStreamWriter Writer;

	// Export context
	UE::Net::Private::FNetExports NetExports;
	const FNetSerializerChangeMaskParam InitStateChangeMaskInfo = { 0 };

	enum : uint32
	{
		BufferSize = 1024,
	};

	alignas(16) uint8 StateBuffer0[BufferSize] = {};
	alignas(16) uint8 StateBuffer1[BufferSize] = {};
	alignas(16) uint8 StateBuffer2[BufferSize] = {};

	alignas(16) uint8 BitStreamBuffer0[BufferSize];
	alignas(16) uint8 BitStreamBuffer1[BufferSize];

	FReplicationSystemTestClient* Client = nullptr;
};

const FNetSerializer* FTestFieldPathNetSerializer::FieldPathNetSerializer = &UE_NET_GET_SERIALIZER(FFieldPathNetSerializer);
const FFieldPathNetSerializerConfig FTestFieldPathNetSerializer::SerializerConfig;
TArray<FFieldPath> FTestFieldPathNetSerializer::TestValues;
TArray<FFieldPath> FTestFieldPathNetSerializer::UnresolvableTestValues;

void FTestFieldPathNetSerializer::SetUp()
{
	Super::SetUp();

	Client = CreateClient();

	// Setup NetSerializationContexts that can serialize and allows memory allocations
	{
		ServerNetSerializationContext = FNetSerializationContext(&Reader, &Writer);
		ServerNetSerializationContext.SetInternalContext(&ServerInternalContext);
		ServerNetSerializationContext.SetLocalConnectionId(Client->ConnectionIdOnServer);

		{
			const FReplicationSystemTestNode::FConnectionInfo& ConnectionInfo = Server->GetConnectionInfo(Client->ConnectionIdOnServer);
			FNetTokenStore* ServerTokenStore = Server->GetReplicationSystem()->GetNetTokenStore();

			FInternalNetSerializationContext TempServerInternalContext;
			FInternalNetSerializationContext::FInitParameters TempServerInternalContextInitParams;
			TempServerInternalContextInitParams.ReplicationSystem = Server->GetReplicationSystem();
			TempServerInternalContextInitParams.ObjectResolveContext.RemoteNetTokenStoreState = ServerTokenStore->GetRemoteNetTokenStoreState(Client->ConnectionIdOnServer);
			TempServerInternalContextInitParams.ObjectResolveContext.ConnectionId = ConnectionInfo.ConnectionId;
			TempServerInternalContext.Init(TempServerInternalContextInitParams);

			ServerInternalContext = MoveTemp(TempServerInternalContext);
		}

		{
			const FReplicationSystemTestNode::FConnectionInfo& ConnectionInfo = Client->GetConnectionInfo(Client->LocalConnectionId);

			FNetTokenStore* ClientTokenStore = Client->GetReplicationSystem()->GetNetTokenStore();

			FInternalNetSerializationContext TempClientInternalContext;
			FInternalNetSerializationContext::FInitParameters TempClientInternalContextInitParams;
			TempClientInternalContextInitParams.ReplicationSystem = Client->GetReplicationSystem();
			TempClientInternalContextInitParams.ObjectResolveContext.RemoteNetTokenStoreState = ClientTokenStore->GetRemoteNetTokenStoreState(Client->LocalConnectionId);
			TempClientInternalContextInitParams.ObjectResolveContext.ConnectionId = ConnectionInfo.ConnectionId;
			TempClientInternalContext.Init(TempClientInternalContextInitParams);

			ClientInternalContext = MoveTemp(TempClientInternalContext);
		}

		ClientNetSerializationContext = FNetSerializationContext(&Reader, &Writer);
		ClientNetSerializationContext.SetInternalContext(&ClientInternalContext);
		ClientNetSerializationContext.SetLocalConnectionId(Client->LocalConnectionId);
	}

	if (TestValues.Num() == 0)
	{
		SetUpTestValues();
	}
}

void FTestFieldPathNetSerializer::TearDown()
{
	Super::TearDown();
}

void FTestFieldPathNetSerializer::SetUpTestValues()
{
	struct FPropertyPath
	{
		const TCHAR* PropertyPath;
	};

	const FPropertyPath PropertyPaths[] = 
	{
		// Findable properties
		{TEXT("/Script/ReplicationSystemTestPlugin.SimpleStructForFieldPathNetSerializerTest.PropertyToFind")},
		{TEXT("/Script/ReplicationSystemTestPlugin.SimpleClassForFieldPathNetSerializerTest.PropertyToFind")},
		{TEXT("/Script/ReplicationSystemTestPlugin.InheritedSimpleStructForFieldPathNetSerializerTest.PropertyToFind")},
		{TEXT("/Script/ReplicationSystemTestPlugin.InheritedSimpleClassForFieldPathNetSerializerTest.PropertyToFind")},
		{TEXT("/Script/ReplicationSystemTestPlugin.DeepInheritedSimpleStructForFieldPathNetSerializerTest.PropertyToFind")},

		// Non-findable property, even though in theory the path is valid.
		{TEXT("/Script/ReplicationSystemTestPlugin.NestedSimpleStructForFieldPathNetSerializerTest.NestedStruct2.PropertyToFind")},

		// Empty
		{TEXT("")},
	};

	const FPropertyPath InvalidPropertyPaths[] = 
	{
		// Valid class, invalid path.
		{TEXT("/Script/ReplicationSystemTestPlugin.NestedSimpleStructForFieldPathNetSerializerTest.NestedStruct2.NonExistingProperty")},

		// Invalid plugin, class, and property.
		{TEXT("/Script/NonExistingPlugin.NonExistingClass.NonExistingProperty")},
	};

	for (const FPropertyPath& PropertyPath : PropertyPaths)
	{
		FFieldPath& FieldPath = TestValues.AddDefaulted_GetRef();
		FieldPath.Generate(PropertyPath.PropertyPath);
	};

	for (const FPropertyPath& PropertyPath : InvalidPropertyPaths)
	{
		FFieldPath& FieldPath = UnresolvableTestValues.AddDefaulted_GetRef();
		FieldPath.Generate(PropertyPath.PropertyPath);
	};
}

UE_NET_TEST_FIXTURE(FTestFieldPathNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestFieldPathNetSerializer, TestCloneDynamicState)
{
	TestCloneDynamicState();
}

UE_NET_TEST_FIXTURE(FTestFieldPathNetSerializer, TestFreeDynamicState)
{
	TestFreeDynamicState();
}

UE_NET_TEST_FIXTURE(FTestFieldPathNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestFieldPathNetSerializer, TestRoundtripSerialize)
{
	TestRoundtripSerialize();
}

UE_NET_TEST_FIXTURE(FTestFieldPathNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestFieldPathNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestFieldPathNetSerializer, TestValidate)
{
	TestValidate();
}

void FTestFieldPathNetSerializer::Quantize(FNetSerializationContext& Context, const FFieldPath& FieldPath, void* StateBuffer)
{
	FNetQuantizeArgs Args = {};
	Args.Version = FieldPathNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);
	Args.Target = NetSerializerValuePointer(StateBuffer);
	Args.Source = NetSerializerValuePointer(&FieldPath);
	FieldPathNetSerializer->Quantize(Context, Args);
}

void FTestFieldPathNetSerializer::Dequantize(FNetSerializationContext& Context, const void* StateBuffer, FFieldPath& FieldPath)
{
	FNetDequantizeArgs Args = {};
	Args.Version = FieldPathNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);
	Args.Source = NetSerializerValuePointer(StateBuffer);
	Args.Target = NetSerializerValuePointer(&FieldPath);
	FieldPathNetSerializer->Dequantize(Context, Args);
}

bool FTestFieldPathNetSerializer::IsEqual(FNetSerializationContext& Context, const FFieldPath& FieldPath0, const FFieldPath& FieldPath1)
{
	FNetIsEqualArgs Args = {};
	Args.Version = FieldPathNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);
	Args.Source0 = NetSerializerValuePointer(&FieldPath0);
	Args.Source1 = NetSerializerValuePointer(&FieldPath1);
	Args.bStateIsQuantized = false;
	return FieldPathNetSerializer->IsEqual(Context, Args);
}

void FTestFieldPathNetSerializer::FreeDynamicState(FNetSerializationContext& Context, void* StateBuffer)
{
	FNetFreeDynamicStateArgs Args = {};
	Args.Version = FieldPathNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);
	Args.Source = NetSerializerValuePointer(StateBuffer);
	FieldPathNetSerializer->FreeDynamicState(Context, Args);
}

void FTestFieldPathNetSerializer::CollectAndAppendExports(FNetSerializationContext& Context, void* StateBuffer)
{
	FNetExportContext* ExportContext = Context.GetExportContext();
	if (!ExportContext)
	{
		return;
	}

	FNetReferenceCollector Collector(ENetReferenceCollectorTraits::OnlyCollectReferencesThatCanBeExported);

	FNetCollectReferencesArgs Args = {};
	Args.Version = FieldPathNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);
	Args.Source = NetSerializerValuePointer(StateBuffer);
	Args.Collector = NetSerializerValuePointer(&Collector);

	FieldPathNetSerializer->CollectNetReferences(Context, Args);

	for (const FNetReferenceCollector::FReferenceInfo& Info : MakeArrayView(Collector.GetCollectedReferences()))
	{
		Context.GetInternalContext()->ObjectReferenceCache->AddPendingExport(*ExportContext, Info.Reference);
	}
}

void FTestFieldPathNetSerializer::TestQuantize()
{
	FNetQuantizeArgs QuantizeArgs = {};
	QuantizeArgs.Version = FieldPathNetSerializer->Version;
	QuantizeArgs.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);
	QuantizeArgs.Target = NetSerializerValuePointer(StateBuffer0);

	FNetIsEqualArgs IsEqualArgs = {};
	IsEqualArgs.Version = FieldPathNetSerializer->Version;
	IsEqualArgs.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);

	TArrayView<const FFieldPath> AllTheValues[] = { MakeArrayView(TestValues), MakeArrayView(UnresolvableTestValues) };
	for (const TArrayView<const FFieldPath>& Values : AllTheValues)
	{
		for (const FFieldPath& FieldPath : Values)
		{
			QuantizeArgs.Source = NetSerializerValuePointer(&FieldPath);
			FieldPathNetSerializer->Quantize(ServerNetSerializationContext, QuantizeArgs);
			UE_NET_ASSERT_FALSE(ServerNetSerializationContext.HasError());

			IsEqualArgs.bStateIsQuantized = true;
			IsEqualArgs.Source0 = QuantizeArgs.Target;
			IsEqualArgs.Source1 = QuantizeArgs.Target;
			const bool bQuantizedStateIsEqual = FieldPathNetSerializer->IsEqual(ServerNetSerializationContext, IsEqualArgs);
			UE_NET_ASSERT_TRUE(bQuantizedStateIsEqual);

			FFieldPath DequantizedFieldPath;
			Dequantize(ServerNetSerializationContext, (void*)QuantizeArgs.Target, DequantizedFieldPath);

			const bool bDequantizedStateIsEqual = IsEqual(ServerNetSerializationContext, FieldPath, DequantizedFieldPath);
			UE_NET_ASSERT_TRUE(bQuantizedStateIsEqual);

			FreeDynamicState(ServerNetSerializationContext, StateBuffer0);
		}
	}
}

void FTestFieldPathNetSerializer::TestCloneDynamicState()
{
	FNetCloneDynamicStateArgs CloneArgs = {};
	CloneArgs.Version = FieldPathNetSerializer->Version;
	CloneArgs.NetSerializerConfig = NetSerializerConfigParam(FieldPathNetSerializer);
	CloneArgs.Source = NetSerializerValuePointer(StateBuffer0);
	CloneArgs.Target = NetSerializerValuePointer(StateBuffer1);

	for (const FFieldPath& FieldPath : TestValues)
	{
		Quantize(ServerNetSerializationContext, FieldPath, StateBuffer0);

		// This serializer requires that a proper clone of the entire state as described by a ReplicationStateDescriptor happens first.
		FMemory::Memcpy(StateBuffer1, StateBuffer0, sizeof(StateBuffer0));
		FieldPathNetSerializer->CloneDynamicState(ServerNetSerializationContext, CloneArgs);

		// Free the original state.
		FreeDynamicState(ServerNetSerializationContext, StateBuffer0);

		// Dequantize and validate contents to verify that the cloning is correct
		FFieldPath DequantizedFieldPath;
		Dequantize(ServerNetSerializationContext, (void*)CloneArgs.Target, DequantizedFieldPath);

		FreeDynamicState(ServerNetSerializationContext, (void*)CloneArgs.Target);

		UE_NET_ASSERT_EQ(FieldPath, DequantizedFieldPath);
	}
}

void FTestFieldPathNetSerializer::TestFreeDynamicState()
{
	FNetFreeDynamicStateArgs FreeArgs = {};
	FreeArgs.Version = FieldPathNetSerializer->Version;
	FreeArgs.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);
	FreeArgs.Source = NetSerializerValuePointer(StateBuffer0);
	
	TArrayView<const FFieldPath> AllTheValues[] = { MakeArrayView(TestValues), MakeArrayView(UnresolvableTestValues) };
	for (const TArrayView<const FFieldPath>& Values : AllTheValues)
	{
		for (const FFieldPath& FieldPath : Values)
		{
			Quantize(ServerNetSerializationContext, FieldPath, StateBuffer0);

			FieldPathNetSerializer->FreeDynamicState(ServerNetSerializationContext, FreeArgs);

			// Make sure FreeDynamicState is re-entrant.
			FieldPathNetSerializer->FreeDynamicState(ServerNetSerializationContext, FreeArgs);
		}
	}
}

void FTestFieldPathNetSerializer::TestSerialize()
{
	FNetSerializeArgs SerializeArgs = {};
	SerializeArgs.Version = FieldPathNetSerializer->Version;
	SerializeArgs.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);
	SerializeArgs.Source = NetSerializerValuePointer(StateBuffer0);

	FNetDeserializeArgs DeserializeArgs = {};
	DeserializeArgs.Version = FieldPathNetSerializer->Version;
	DeserializeArgs.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);
	DeserializeArgs.Target = NetSerializerValuePointer(StateBuffer1);

	// Setup export context, we do not actually need to export anything as we are writing and reading serverside
	FNetExportContext::FBatchExports CurrentPacketBatchExports;
	FNetExports::FExportScope ExportScope = NetExports.MakeExportScope(ServerNetSerializationContext, CurrentPacketBatchExports);

	TArrayView<const FFieldPath> AllTheValues[] = { MakeArrayView(TestValues), MakeArrayView(UnresolvableTestValues) };
	for (const TArrayView<const FFieldPath>& Values : AllTheValues)
	{
		for (const FFieldPath& FieldPath : Values)
		{
			Quantize(ServerNetSerializationContext, FieldPath, StateBuffer0);

			ServerNetSerializationContext.GetBitStreamWriter()->InitBytes(BitStreamBuffer0, sizeof(BitStreamBuffer0));

			FieldPathNetSerializer->Serialize(ServerNetSerializationContext, SerializeArgs);
			ServerNetSerializationContext.GetBitStreamWriter()->CommitWrites();
			UE_NET_ASSERT_FALSE(ServerNetSerializationContext.HasErrorOrOverflow());

			ServerNetSerializationContext.GetBitStreamReader()->InitBits(BitStreamBuffer0, ServerNetSerializationContext.GetBitStreamWriter()->GetPosBits());
			FieldPathNetSerializer->Deserialize(ServerNetSerializationContext, DeserializeArgs);
			UE_NET_ASSERT_FALSE(ServerNetSerializationContext.HasErrorOrOverflow());

			FFieldPath DequantizedFieldPath;
			Dequantize(ServerNetSerializationContext, (void*)DeserializeArgs.Target, DequantizedFieldPath);

			const bool bPathIsEqual = IsEqual(ServerNetSerializationContext, FieldPath, DequantizedFieldPath);
			UE_NET_ASSERT_TRUE(bPathIsEqual);

			FreeDynamicState(ServerNetSerializationContext, StateBuffer0);
			FreeDynamicState(ServerNetSerializationContext, StateBuffer1);
		}
	}
}

void FTestFieldPathNetSerializer::TestRoundtripSerialize()
{
	FNetSerializeArgs SerializeArgs = {};
	SerializeArgs.Version = FieldPathNetSerializer->Version;
	SerializeArgs.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);
	SerializeArgs.Source = NetSerializerValuePointer(StateBuffer0);

	FNetDeserializeArgs DeserializeArgs = {};
	DeserializeArgs.Version = FieldPathNetSerializer->Version;
	DeserializeArgs.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);
	DeserializeArgs.Target = NetSerializerValuePointer(StateBuffer1);

	EIrisAsyncLoadingPriority IrisAsyncLoadingPriority = EIrisAsyncLoadingPriority::Default;

	TArray<FNetRefHandle> TempMustBeMappedReferences;
	TArrayView<const FFieldPath> AllTheValues[] = { MakeArrayView(TestValues), MakeArrayView(UnresolvableTestValues) };
	for (const TArrayView<const FFieldPath>& Values : AllTheValues)
	{
		for (const FFieldPath& FieldPath : Values)
		{
			// Setup export scope to capture and serialize exports
			FNetExportContext::FBatchExports CurrentPacketBatchExports;
			FNetExports::FExportScope ExportScope = NetExports.MakeExportScope(ServerNetSerializationContext, CurrentPacketBatchExports);

			Quantize(ServerNetSerializationContext, FieldPath, StateBuffer0);

			ServerNetSerializationContext.GetBitStreamWriter()->InitBytes(BitStreamBuffer0, sizeof(BitStreamBuffer0));

			// Serialize data
			FieldPathNetSerializer->Serialize(ServerNetSerializationContext, SerializeArgs);
			
			// Collect object references as well
			CollectAndAppendExports(ServerNetSerializationContext, StateBuffer0);

			// Write exports
			const uint32 ExportsPos = ServerNetSerializationContext.GetBitStreamWriter()->GetPosBits();
			ServerNetSerializationContext.GetInternalContext()->ObjectReferenceCache->WritePendingExports(ServerNetSerializationContext, 0);
			
			// Finalize
			ServerNetSerializationContext.GetBitStreamWriter()->CommitWrites();
			UE_NET_ASSERT_FALSE(ServerNetSerializationContext.HasErrorOrOverflow());

			// Init client read
			ClientNetSerializationContext.GetBitStreamReader()->InitBits(BitStreamBuffer0, ServerNetSerializationContext.GetBitStreamWriter()->GetPosBits());

			// Read exports
			ClientNetSerializationContext.GetBitStreamReader()->Seek(ExportsPos);
			TempMustBeMappedReferences.Reset();
			ClientNetSerializationContext.GetInternalContext()->ObjectReferenceCache->ReadExports(FNetRefHandle(), ClientNetSerializationContext, &TempMustBeMappedReferences, IrisAsyncLoadingPriority);
			ClientNetSerializationContext.GetBitStreamReader()->Seek(0U);

			// Deserialize data
			FieldPathNetSerializer->Deserialize(ClientNetSerializationContext, DeserializeArgs);
			UE_NET_ASSERT_FALSE(ClientNetSerializationContext.HasErrorOrOverflow());

			FFieldPath DequantizedFieldPath;
			Dequantize(ClientNetSerializationContext, (void*)DeserializeArgs.Target, DequantizedFieldPath);

			const bool bPathIsEqual = IsEqual(ClientNetSerializationContext, FieldPath, DequantizedFieldPath);
			UE_NET_ASSERT_TRUE(bPathIsEqual);

			FreeDynamicState(ServerNetSerializationContext, StateBuffer0);
			FreeDynamicState(ClientNetSerializationContext, StateBuffer1);
		}
	}
}

void FTestFieldPathNetSerializer::TestSerializeDelta()
{
	FNetSerializeDeltaArgs SerializeDeltaArgs = {};
	SerializeDeltaArgs.Version = FieldPathNetSerializer->Version;
	SerializeDeltaArgs.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);
	SerializeDeltaArgs.Source = NetSerializerValuePointer(StateBuffer1);
	SerializeDeltaArgs.Prev = NetSerializerValuePointer(StateBuffer0);

	FNetDeserializeDeltaArgs DeserializeDeltaArgs = {};
	DeserializeDeltaArgs.Version = FieldPathNetSerializer->Version;
	DeserializeDeltaArgs.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);
	DeserializeDeltaArgs.Target = NetSerializerValuePointer(StateBuffer2);
	DeserializeDeltaArgs.Prev = NetSerializerValuePointer(StateBuffer0);

	FNetIsEqualArgs IsEqualArgs = {};
	IsEqualArgs.Version = FieldPathNetSerializer->Version;
	IsEqualArgs.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);
	IsEqualArgs.Source0 = NetSerializerValuePointer(StateBuffer1);
	IsEqualArgs.Source1 = NetSerializerValuePointer(StateBuffer2);
	IsEqualArgs.bStateIsQuantized = true;

	// Setup export context, we do not actually need to export anything as we are writing and reading serverside
	FNetExportContext::FBatchExports CurrentPacketBatchExports;
	FNetExports::FExportScope ExportScope = NetExports.MakeExportScope(ServerNetSerializationContext, CurrentPacketBatchExports);

	TArrayView<const FFieldPath> AllTheValues[] = { MakeArrayView(TestValues), MakeArrayView(UnresolvableTestValues) };
	for (const TArrayView<const FFieldPath>& Values : AllTheValues)
	{
		// Test each value delta compressed against all other values.
		for (const FFieldPath& OldFieldPath : Values)
		{
			Quantize(ServerNetSerializationContext, OldFieldPath, StateBuffer0);

			for (const FFieldPath& FieldPath : Values)
			{
				Quantize(ServerNetSerializationContext, FieldPath, StateBuffer1);

				ServerNetSerializationContext.GetBitStreamWriter()->InitBytes(BitStreamBuffer0, sizeof(BitStreamBuffer0));

				FieldPathNetSerializer->SerializeDelta(ServerNetSerializationContext, SerializeDeltaArgs);	
				ServerNetSerializationContext.GetBitStreamWriter()->CommitWrites();
				UE_NET_ASSERT_FALSE(ServerNetSerializationContext.HasErrorOrOverflow());

				ServerNetSerializationContext.GetBitStreamReader()->InitBits(BitStreamBuffer0, ServerNetSerializationContext.GetBitStreamWriter()->GetPosBits());
				FieldPathNetSerializer->DeserializeDelta(ServerNetSerializationContext, DeserializeDeltaArgs);
				UE_NET_ASSERT_FALSE(ServerNetSerializationContext.HasErrorOrOverflow());

				// Make sure the quantized state after deserialization matches the original quantized state.
				const bool bQuantizedStateIsEqual = FieldPathNetSerializer->IsEqual(ServerNetSerializationContext, IsEqualArgs);
				UE_NET_ASSERT_TRUE(bQuantizedStateIsEqual);

				FreeDynamicState(ServerNetSerializationContext, StateBuffer1);
				FreeDynamicState(ServerNetSerializationContext, StateBuffer2);
			}

			FreeDynamicState(ServerNetSerializationContext, StateBuffer0);
		}
	}
}

void FTestFieldPathNetSerializer::TestIsEqual()
{
	FNetIsEqualArgs IsEqualArgs = {};
	IsEqualArgs.Version = FieldPathNetSerializer->Version;
	IsEqualArgs.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);
	IsEqualArgs.bStateIsQuantized = false;

	// Test all values are equal to themselves.
	{
		TArrayView<const FFieldPath> AllTheValues[] = { MakeArrayView(TestValues), MakeArrayView(UnresolvableTestValues) };
		for (const TArrayView<const FFieldPath>& Values : AllTheValues)
		{
			for (const FFieldPath& FieldPath : Values)
			{
				IsEqualArgs.bStateIsQuantized = false;
				IsEqualArgs.Source0 = NetSerializerValuePointer(&FieldPath);
				IsEqualArgs.Source1 = NetSerializerValuePointer(&FieldPath);
				const bool bIsEqualToSelf = FieldPathNetSerializer->IsEqual(ServerNetSerializationContext, IsEqualArgs);
				UE_NET_ASSERT_TRUE(bIsEqualToSelf);

				Quantize(ServerNetSerializationContext, FieldPath, StateBuffer0);
				IsEqualArgs.bStateIsQuantized = true;
				IsEqualArgs.Source0 = NetSerializerValuePointer(StateBuffer0);
				IsEqualArgs.Source1 = NetSerializerValuePointer(StateBuffer0);
				const bool bIsEqualToQuantizedSelf = FieldPathNetSerializer->IsEqual(ServerNetSerializationContext, IsEqualArgs);
				UE_NET_ASSERT_TRUE(bIsEqualToQuantizedSelf);
			}
		}
	}

	// Test no value is equal to any other value
	{
		for (const FFieldPath& FieldPath : TestValues)
		{
			for (const FFieldPath& OtherFieldPath : TestValues)
			{
				if (&FieldPath == &OtherFieldPath)
				{
					continue;
				}

				// Normal compare
				{
					IsEqualArgs.bStateIsQuantized = false;

					IsEqualArgs.Source0 = NetSerializerValuePointer(&FieldPath);
					IsEqualArgs.Source1 = NetSerializerValuePointer(&OtherFieldPath);
					const bool bIsEqual = FieldPathNetSerializer->IsEqual(ServerNetSerializationContext, IsEqualArgs);
					UE_NET_ASSERT_FALSE(bIsEqual);

					// Swap order of values
					Swap(IsEqualArgs.Source0, IsEqualArgs.Source1);
					const bool bIsEqual2 = FieldPathNetSerializer->IsEqual(ServerNetSerializationContext, IsEqualArgs);
					UE_NET_ASSERT_FALSE(bIsEqual2);
				}

				// Quantized compare
				{
					IsEqualArgs.bStateIsQuantized = true;

					Quantize(ServerNetSerializationContext, FieldPath, StateBuffer0);
					Quantize(ServerNetSerializationContext, OtherFieldPath, StateBuffer1);

					IsEqualArgs.Source0 = NetSerializerValuePointer(StateBuffer0);
					IsEqualArgs.Source1 = NetSerializerValuePointer(StateBuffer1);
					const bool bIsEqual = FieldPathNetSerializer->IsEqual(ServerNetSerializationContext, IsEqualArgs);
					UE_NET_ASSERT_FALSE(bIsEqual);

					// Swap order of values
					Swap(IsEqualArgs.Source0, IsEqualArgs.Source1);
					const bool bIsEqual2 = FieldPathNetSerializer->IsEqual(ServerNetSerializationContext, IsEqualArgs);
					UE_NET_ASSERT_FALSE(bIsEqual2);

					FreeDynamicState(ServerNetSerializationContext, StateBuffer0);
					FreeDynamicState(ServerNetSerializationContext, StateBuffer1);
				}

			}
		}
	}
}

void FTestFieldPathNetSerializer::TestValidate()
{
	FNetValidateArgs ValidateArgs = {};
	ValidateArgs.Version = FieldPathNetSerializer->Version;
	ValidateArgs.NetSerializerConfig = NetSerializerConfigParam(&SerializerConfig);

	TArrayView<const FFieldPath> AllTheValues[] = { MakeArrayView(TestValues), MakeArrayView(UnresolvableTestValues) };
	for (const TArrayView<const FFieldPath>& Values : AllTheValues)
	{
		for (const FFieldPath& FieldPath : Values)
		{
			ValidateArgs.Source = NetSerializerValuePointer(&FieldPath);

			const bool bPathIsValid = FieldPathNetSerializer->Validate(ServerNetSerializationContext, ValidateArgs);
			UE_NET_ASSERT_TRUE(bPathIsValid);
		}
	}
}

}
