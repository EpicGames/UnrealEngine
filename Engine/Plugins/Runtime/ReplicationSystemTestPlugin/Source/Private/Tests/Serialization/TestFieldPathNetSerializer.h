// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/ObjectMacros.h"
#include "TestFieldPathNetSerializer.generated.h"

USTRUCT()
struct FSimpleStructForFieldPathNetSerializerTest
{
	GENERATED_BODY()

	UPROPERTY()
	uint32 PropertyToFind;
};

UCLASS()
class USimpleClassForFieldPathNetSerializerTest : public UObject
{
	GENERATED_BODY()

	UPROPERTY()
	int32 PropertyToFind;
};

USTRUCT()
struct FInheritedSimpleStructForFieldPathNetSerializerTest : public FSimpleStructForFieldPathNetSerializerTest
{
	GENERATED_BODY()

	UPROPERTY()
	bool bSomeOtherProperty;
};

UCLASS()
class UInheritedSimpleClassForFieldPathNetSerializerTest : public USimpleClassForFieldPathNetSerializerTest
{
	GENERATED_BODY()

	UPROPERTY()
	bool bSomeOtherProperty;
};

USTRUCT()
struct FDeepInheritedSimpleStructForFieldPathNetSerializerTest : public FInheritedSimpleStructForFieldPathNetSerializerTest
{
	GENERATED_BODY()
};

USTRUCT()
struct FNestedSimpleStructForFieldPathNetSerializerTest
{
	GENERATED_BODY()

	UPROPERTY()
	bool bSomeOtherProperty;

	UPROPERTY()
	FSimpleStructForFieldPathNetSerializerTest NestedStruct;

	UPROPERTY()
	FSimpleStructForFieldPathNetSerializerTest NestedStruct2;
};
