// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/FloatNetSerializers.h"
#include "Iris/Core/BitTwiddling.h"
#include <limits>

namespace UE::Net::Private
{

static FTestMessage& PrintFloatNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

template<typename T>
class FTestFloatNetSerializerBase : public TTestNetSerializerFixture<PrintFloatNetSerializerConfig, T>
{
	typedef TTestNetSerializerFixture<PrintFloatNetSerializerConfig, T> Super;

public:
	FTestFloatNetSerializerBase(const FNetSerializer& Serializer) : Super(Serializer) {}

	void TestIsEqual();
	void TestSerialize();
	void TestSerializeDelta();

protected:
	static const T Values[];
	static const SIZE_T ValueCount;
};

template<typename T>
const T FTestFloatNetSerializerBase<T>::Values[] =
{
	std::numeric_limits<T>::infinity(), 
	-std::numeric_limits<T>::infinity(), 
	std::numeric_limits<T>::quiet_NaN(), 
	-0.0f, 
	+0.0f, 
	std::numeric_limits<T>::max(), 
	std::numeric_limits<T>::lowest(), 
	47.11f,
	757575.0f,
	757576.0f,
	1.26698556E-23f,
};
template<typename T>
const SIZE_T FTestFloatNetSerializerBase<T>::ValueCount = sizeof(Values)/sizeof(Values[0]);

class FTestFloatNetSerializer : public FTestFloatNetSerializerBase<float>
{
public:
	FTestFloatNetSerializer() : FTestFloatNetSerializerBase<float>(UE_NET_GET_SERIALIZER(FFloatNetSerializer)) {}
};

class FTestDoubleNetSerializer : public FTestFloatNetSerializerBase<double>
{
public:
	FTestDoubleNetSerializer() : FTestFloatNetSerializerBase<double>(UE_NET_GET_SERIALIZER(FDoubleNetSerializer)) {}
};

UE_NET_TEST_FIXTURE(FTestFloatNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestFloatNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestFloatNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestDoubleNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestDoubleNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestDoubleNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

//
template<typename T>
void FTestFloatNetSerializerBase<T>::TestIsEqual()
{
	T CompareValues[2][sizeof(Values)/sizeof(Values[0])];
	bool ExpectedResults[2][sizeof(Values)/sizeof(Values[0])];

	// As a temporary workaround for some compiler issues we copy the memory for the entire array instead of one value at a time. Revert back to per value copy when possible to detect future compiler regressions.
	FPlatformMemory::Memcpy(CompareValues[0], Values, ValueCount*sizeof(T));
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		//CompareValues[0][ValueIt] = Values[ValueIt];
		ExpectedResults[0][ValueIt] = true;
	}

	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		CompareValues[1][ValueIt] = Values[(ValueIt + 1) % ValueCount];
		ExpectedResults[1][ValueIt] = false;
	}

	const FFloatNetSerializerConfig Config;
	for (SIZE_T TestRoundIt = 0, TestRoundEndIt = 2; TestRoundIt != TestRoundEndIt; ++TestRoundIt)
	{
		constexpr bool bQuantizedCompare = false;
		bool bSuccess = Super::TestIsEqual(Values, CompareValues[TestRoundIt], ExpectedResults[TestRoundIt], ValueCount, Config, bQuantizedCompare);
		if (!bSuccess)
		{
			return;
		}
	}
}

template<typename T>
void FTestFloatNetSerializerBase<T>::TestSerialize()
{
	const FFloatNetSerializerConfig Config;
	constexpr bool bQuantizedCompare = false;
	Super::TestSerialize(Values, Values, ValueCount, Config, bQuantizedCompare);
}

template<typename T>
void FTestFloatNetSerializerBase<T>::TestSerializeDelta()
{
	const FFloatNetSerializerConfig Config;
	Super::TestSerializeDelta(Values, ValueCount, Config);
}

}
