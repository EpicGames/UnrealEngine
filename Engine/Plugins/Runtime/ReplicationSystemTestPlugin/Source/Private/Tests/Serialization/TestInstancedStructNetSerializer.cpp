// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Serialization/TestInstancedStructNetSerializer.h"
#include "Tests/Serialization/TestNetSerializerFixture.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetReferenceCollector.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Net/UnrealNetwork.h"

namespace UE::Net
{

FTestMessage& operator<<(FTestMessage& Message, const FInstancedStruct& InstancedStruct)
{
	FString String;
	InstancedStruct.ExportTextItem(String, FInstancedStruct(), nullptr, 0, nullptr);
	return Message << String;
}

struct FStructForInstancedStructTestWithCustomApplyNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Types
	typedef FStructForInstancedStructTestWithCustomApply SourceType;
	typedef int QuantizedType;
	typedef FNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	// 
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);

	static void Apply(FNetSerializationContext&, const FNetApplyArgs&);

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;

		inline static const FName StructForInstancedStructTestWithCustomApplyNetSerializer = FName("StructForInstancedStructTestWithCustomApply");
		UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(StructForInstancedStructTestWithCustomApplyNetSerializer, FStructForInstancedStructTestWithCustomApplyNetSerializer);
	};

	inline static FStructForInstancedStructTestWithCustomApplyNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
};
UE_NET_IMPLEMENT_SERIALIZER(FStructForInstancedStructTestWithCustomApplyNetSerializer);

struct FStructForInstancedStructTestWithCustomSerializerNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Types
	typedef FStructForInstancedStructTestWithCustomSerializer SourceType;
	typedef int QuantizedType;
	typedef FNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	// 
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;

		inline static const FName StructForInstancedStructTestWithCustomSerializerNetSerializer = FName("StructForInstancedStructTestWithCustomSerializer");
		UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(StructForInstancedStructTestWithCustomSerializerNetSerializer, FStructForInstancedStructTestWithCustomSerializerNetSerializer);
	};

	inline static FStructForInstancedStructTestWithCustomSerializerNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
};
UE_NET_IMPLEMENT_SERIALIZER(FStructForInstancedStructTestWithCustomSerializerNetSerializer);

}

namespace UE::Net::Private
{

FTestMessage& PrintInstancedStructNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	const FInstancedStructNetSerializerConfig& Config = static_cast<const FInstancedStructNetSerializerConfig&>(InConfig);
	for (const auto& SupportedType : Config.SupportedTypes)
	{
		Message << SupportedType.Get()->GetFullName();
	}
	return Message;
}

class FTestInstancedStructNetSerializerFixture : public FReplicationSystemServerClientTestFixture
{
public:
	FTestInstancedStructNetSerializerFixture() : FReplicationSystemServerClientTestFixture() {}

protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	// Composable operations for testing the serializer
	void Serialize();
	void Deserialize();
	void SerializeDelta();
	void DeserializeDelta();
	void Quantize();
	void QuantizeTwoStates();
	void Dequantize();
	bool IsEqual(bool bQuantized);
	void Clone();
	void Validate();
	void FreeQuantizedState();

	// Instantiates a struct with a modified property on InstancedStruct0
	void SetNonDefaultInstanceState();

	// Adds multiple elements, at least one uninitialized and at least two initialized with at least one modified property, to InstancedStructArray0
	void SetNonDefaultArrayState();

	bool AreEqual(const FInstancedStruct* Value0, const FInstancedStruct* Value1);

	const FNetSerializer* GetInstancedStructNetSerializer(bool bIsArray) const;
	const FNetSerializerConfig* GetInstancedStructNetSerializerConfig(bool bIsArray) const;

protected:
	FNetSerializationContext NetSerializationContext;
	FInternalNetSerializationContext InternalNetSerializationContext;

	FInstancedStruct InstancedStruct0;
	FInstancedStruct InstancedStruct1;
	TArray<FInstancedStruct> InstancedStructArray0;
	TArray<FInstancedStruct> InstancedStructArray1;

	FStructNetSerializerConfig InstancedStructNetSerializerConfig;
	FStructNetSerializerConfig InstancedStructArrayNetSerializerConfig;

	alignas(16) uint8 QuantizedBuffer[2][128];
	alignas(16) uint8 ClonedQuantizedBuffer[2][128];
	alignas(16) uint8 BitStreamBuffer[2048];

	bool bHasQuantizedState = false;
	bool bHasClonedQuantizedState = false;
	bool bIsTestingArray = false;

	uint32 QuantizedStateCount = 0;
	uint32 ClonedQuantizedStateCount = 0;

	FNetBitStreamWriter Writer;
	FNetBitStreamReader Reader;
};

}

namespace UE::Net::Private
{

// Instance tests
UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestQuantizeUninitialized)
{
	Quantize();
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestQuantizeInitialized)
{
	SetNonDefaultInstanceState();
	Quantize();
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestDequantizeUninitialized)
{
	Quantize();
	Dequantize();
	UE_NET_ASSERT_EQ(InstancedStruct0, InstancedStruct1);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestDequantizeInitialized)
{
	SetNonDefaultInstanceState();
	Quantize();
	Dequantize();
	UE_NET_ASSERT_EQ(InstancedStruct0, InstancedStruct1);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestSerializeUninitialized)
{
	Quantize();
	Serialize();
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestSerializeInitialized)
{
	SetNonDefaultInstanceState();
	Quantize();
	Serialize();
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestDeserializeUninitialized)
{
	Quantize();
	Serialize();
	FreeQuantizedState();
	Deserialize();
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestDeserializeInitialized)
{
	SetNonDefaultInstanceState();
	Quantize();
	Serialize();
	FreeQuantizedState();
	Deserialize();
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestDequantizeSerializedUninitializedState)
{
	Quantize();
	Serialize();
	FreeQuantizedState();
	Deserialize();
	Dequantize();
	UE_NET_ASSERT_EQ(InstancedStruct0, InstancedStruct1);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestDequantizeSerializedInitializedState)
{
	SetNonDefaultInstanceState();
	Quantize();
	Serialize();
	FreeQuantizedState();
	Deserialize();
	Dequantize();
	UE_NET_ASSERT_EQ(InstancedStruct0, InstancedStruct1);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestSerializeDeltaEqualUnsetInstances)
{
	QuantizeTwoStates();
	SerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestSerializeDeltaNonEqualStates)
{
	// Equal types but different states
	InstancedStruct0.InitializeAs<FStructForInstancedStructTestA>();
	InstancedStruct0.GetMutable<FStructForInstancedStructTestA>().SomeUint16 += 4711U;

	InstancedStruct1.InitializeAs<FStructForInstancedStructTestA>();
	InstancedStruct1.GetMutable<FStructForInstancedStructTestA>().SomeUint16 += 711U;

	QuantizeTwoStates();
	SerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestSerializeDeltaNonEqualTypes)
{
	// Different types
	InstancedStruct0.InitializeAs<FStructForInstancedStructTestWithArray>();
	InstancedStruct0.GetMutable<FStructForInstancedStructTestWithArray>().ArrayOfTestB.SetNum(17);

	InstancedStruct1.InitializeAs<FStructForInstancedStructTestA>();
	InstancedStruct1.GetMutable<FStructForInstancedStructTestA>().SomeUint16 += 711U;

	QuantizeTwoStates();
	SerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestDeserializeDeltaEqualUnsetInstances)
{
	QuantizeTwoStates();
	SerializeDelta();
	DeserializeDelta();
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestDeserializeDeltaNonEqualStates)
{
	// Equal types but different states
	InstancedStruct0.InitializeAs<FStructForInstancedStructTestA>();
	InstancedStruct0.GetMutable<FStructForInstancedStructTestA>().SomeUint16 += 4711U;

	InstancedStruct1.InitializeAs<FStructForInstancedStructTestA>();
	InstancedStruct1.GetMutable<FStructForInstancedStructTestA>().SomeUint16 += 711U;

	QuantizeTwoStates();
	SerializeDelta();
	DeserializeDelta();
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestDeserializeDeltaNonEqualTypes)
{
	// Different types
	InstancedStruct0.InitializeAs<FStructForInstancedStructTestWithArray>();
	InstancedStruct0.GetMutable<FStructForInstancedStructTestWithArray>().ArrayOfTestB.SetNum(17);

	InstancedStruct1.InitializeAs<FStructForInstancedStructTestA>();
	InstancedStruct1.GetMutable<FStructForInstancedStructTestA>().SomeUint16 += 711U;

	QuantizeTwoStates();
	SerializeDelta();
	DeserializeDelta();
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestDequantizeDeltaSerializedUnsetInstances)
{
	QuantizeTwoStates();
	SerializeDelta();
	DeserializeDelta();
	Dequantize();

	UE_NET_ASSERT_EQ(InstancedStruct0, InstancedStruct1);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestDequantizeDeltaSerializedNonEqualStates)
{
	// Equal types but different states
	InstancedStruct0.InitializeAs<FStructForInstancedStructTestA>();
	InstancedStruct0.GetMutable<FStructForInstancedStructTestA>().SomeUint16 += 4711U;

	InstancedStruct1.InitializeAs<FStructForInstancedStructTestA>();
	InstancedStruct1.GetMutable<FStructForInstancedStructTestA>().SomeUint16 += 711U;

	QuantizeTwoStates();
	SerializeDelta();
	DeserializeDelta();
	Dequantize();

	UE_NET_ASSERT_EQ(InstancedStruct0, InstancedStruct1);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestDequantizeDeltaSerializedNonEqualTypes)
{
	// Different types
	InstancedStruct0.InitializeAs<FStructForInstancedStructTestWithArray>();
	InstancedStruct0.GetMutable<FStructForInstancedStructTestWithArray>().ArrayOfTestB.SetNum(17);

	InstancedStruct1.InitializeAs<FStructForInstancedStructTestA>();
	InstancedStruct1.GetMutable<FStructForInstancedStructTestA>().SomeUint16 += 711U;

	QuantizeTwoStates();
	SerializeDelta();
	DeserializeDelta();
	Dequantize();

	UE_NET_ASSERT_EQ(InstancedStruct0, InstancedStruct1);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestCollectReferencesUninitialized)
{
	Quantize();

	FNetReferenceCollector Collector;

	FNetCollectReferencesArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetInstancedStructNetSerializerConfig(bIsTestingArray));
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Collector = NetSerializerValuePointer(&Collector);
	GetInstancedStructNetSerializer(bIsTestingArray)->CollectNetReferences(NetSerializationContext, Args);

	UE_NET_ASSERT_EQ(Collector.GetCollectedReferences().Num(), 0);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestCollectReferencesStructNoRef)
{
	InstancedStruct0.InitializeAs<FStructForInstancedStructTestD>();

	Quantize();

	FNetReferenceCollector Collector(ENetReferenceCollectorTraits::IncludeInvalidReferences);

	FNetCollectReferencesArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetInstancedStructNetSerializerConfig(bIsTestingArray));
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Collector = NetSerializerValuePointer(&Collector);

	GetInstancedStructNetSerializer(bIsTestingArray)->CollectNetReferences(NetSerializationContext, Args);

	UE_NET_ASSERT_EQ(Collector.GetCollectedReferences().Num(), 1);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestCollectReferencesStructWithRef)
{
	InstancedStruct0.InitializeAs<FStructForInstancedStructTestWithObjectReference>();
	InstancedStruct0.GetMutable<FStructForInstancedStructTestWithObjectReference>().SomeObject = FStructForInstancedStructTestWithObjectReference::StaticStruct();

	Quantize();

	FNetReferenceCollector Collector(ENetReferenceCollectorTraits::IncludeInvalidReferences);

	FNetCollectReferencesArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetInstancedStructNetSerializerConfig(bIsTestingArray));
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Collector = NetSerializerValuePointer(&Collector);

	GetInstancedStructNetSerializer(bIsTestingArray)->CollectNetReferences(NetSerializationContext, Args);

	UE_NET_ASSERT_GE(Collector.GetCollectedReferences().Num(), 2);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestIsEqualExternal)
{
	constexpr bool bUseQuantizedState = false;

	// Default state compared to default state
	InstancedStruct0.Reset();
	InstancedStruct1.Reset();
	UE_NET_ASSERT_TRUE(IsEqual(bUseQuantizedState));

	// Non-default state compared to default state
	SetNonDefaultInstanceState();
	UE_NET_ASSERT_FALSE(IsEqual(bUseQuantizedState));

	// Non-default state compared to non-default state
	InstancedStruct1 = InstancedStruct0;
	UE_NET_ASSERT_TRUE(IsEqual(bUseQuantizedState));
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestIsEqualQuantized)
{
	constexpr bool bUseQuantizedState = true;

	// Default state compared to default state
	InstancedStruct0.Reset();
	Quantize();
	Clone();
	UE_NET_ASSERT_TRUE(IsEqual(bUseQuantizedState));

	// Non-default state compared to default state
	SetNonDefaultInstanceState();
	Quantize();
	UE_NET_ASSERT_FALSE(IsEqual(bUseQuantizedState));

	// Non-default state compared to non-default state
	Clone();
	UE_NET_ASSERT_TRUE(IsEqual(bUseQuantizedState));
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestValidate)
{
	Validate();
}

// Array tests. There's no custom FInstancedStructArrayNetSerializer so we just add the one test until we require in-depth testing.
UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, TestDequantizeSerializedInitializedArrayState)
{
	bIsTestingArray = true;

	SetNonDefaultArrayState();
	Quantize();
	Serialize();
	FreeQuantizedState();
	Deserialize();
	Dequantize();
	UE_NET_ASSERT_TRUE(InstancedStruct0 == InstancedStruct1);
}

// End-to-end tests
UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, CanInitializeInstancedStructInConstructor)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UInstancedStructNetSerializerTestObject* ServerObject = Server->CreateObject<UInstancedStructNetSerializerTestObject>();
	
	// Replicate
	Server->UpdateAndSend({ Client });

	UInstancedStructNetSerializerTestObject* ClientObject = Client->GetObjectAs<UInstancedStructNetSerializerTestObject>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	CA_ASSUME(ClientObject != nullptr);
	UE_NET_ASSERT_EQ(ClientObject->ConstructorInitializedInstancedStruct.GetScriptStruct(), static_cast<const UScriptStruct*>(FStructForInstancedStructTestA::StaticStruct()));
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, CanModifyInstancedStructInitializedInConstructor)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UInstancedStructNetSerializerTestObject* ServerObject = Server->CreateObject<UInstancedStructNetSerializerTestObject>();
	ServerObject->ConstructorInitializedInstancedStruct.GetMutable<FStructForInstancedStructTestA>().SomeUint16 += 1;
	
	// Replicate
	Server->UpdateAndSend({ Client });

	UInstancedStructNetSerializerTestObject* ClientObject = Client->GetObjectAs<UInstancedStructNetSerializerTestObject>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	CA_ASSUME(ClientObject != nullptr);
	UE_NET_ASSERT_EQ(ClientObject->ConstructorInitializedInstancedStruct.GetScriptStruct(), static_cast<const UScriptStruct*>(FStructForInstancedStructTestA::StaticStruct()));
	UE_NET_ASSERT_EQ(ClientObject->ConstructorInitializedInstancedStruct.Get<FStructForInstancedStructTestA>().SomeUint16, ServerObject->ConstructorInitializedInstancedStruct.Get<FStructForInstancedStructTestA>().SomeUint16);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, ModifyInstance)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UInstancedStructNetSerializerTestObject* ServerObject = Server->CreateObject<UInstancedStructNetSerializerTestObject>();
	
	ServerObject->InstancedStruct.InitializeAs<FStructForInstancedStructTestB>();
	ServerObject->InstancedStruct.GetMutable<FStructForInstancedStructTestB>().SomeFloat = 12.0f;

	// Replicate
	Server->UpdateAndSend({ Client });

	auto ClientObject = Client->GetObjectAs<UInstancedStructNetSerializerTestObject>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	CA_ASSUME(ClientObject != nullptr);
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct, ServerObject->InstancedStruct);
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestB>().SomeFloat, ServerObject->InstancedStruct.Get<FStructForInstancedStructTestB>().SomeFloat);

	// Modify
	ServerObject->InstancedStruct.GetMutable<FStructForInstancedStructTestB>().SomeFloat += 1.0f;

	// Replicate
	Server->UpdateAndSend({ Client });

	// Verify that we detected the modification
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestB>().SomeFloat, ServerObject->InstancedStruct.Get<FStructForInstancedStructTestB>().SomeFloat);

	// Switch type
	ServerObject->InstancedStruct.InitializeAs<FStructForInstancedStructTestA>();
	ServerObject->InstancedStruct.GetMutable<FStructForInstancedStructTestA>().SomeUint16 = 100;

	// Replicate
	Server->UpdateAndSend({ Client });

	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct, ServerObject->InstancedStruct);
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestA>().SomeUint16, ServerObject->InstancedStruct.Get<FStructForInstancedStructTestA>().SomeUint16);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, ModifyArray)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UInstancedStructNetSerializerTestObject* ServerObject = Server->CreateObject<UInstancedStructNetSerializerTestObject>();

	// Add entries to the array
	ServerObject->InstancedStructArray.Add(FInstancedStruct::Make<FStructForInstancedStructTestA>());
	ServerObject->InstancedStructArray.Add(FInstancedStruct::Make<FStructForInstancedStructTestB>());
	ServerObject->InstancedStructArray.Add(FInstancedStruct::Make<FStructForInstancedStructTestC>());
	ServerObject->InstancedStructArray.Add(FInstancedStruct::Make<FStructForInstancedStructTestD>());
	ServerObject->InstancedStructArray[1].GetMutable<FStructForInstancedStructTestB>().SomeFloat = 13.0f;
	
	// Replicate
	Server->UpdateAndSend({ Client });
	
	auto ClientObject = Client->GetObjectAs<UInstancedStructNetSerializerTestObject>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	CA_ASSUME(ClientObject != nullptr);
	UE_NET_ASSERT_EQ(ClientObject->InstancedStructArray.Num(), ServerObject->InstancedStructArray.Num());
	UE_NET_ASSERT_EQ(ClientObject->InstancedStructArray[1].Get<FStructForInstancedStructTestB>().SomeFloat, 13.0f);

	// Modify value and see that it is replicated as expected
	ServerObject->InstancedStructArray[1].GetMutable<FStructForInstancedStructTestB>().SomeFloat += 2.0f;

	// Replicate
	Server->UpdateAndSend({ Client });

	// Verified that the client got the modified value
	UE_NET_ASSERT_EQ(ClientObject->InstancedStructArray[1].Get<FStructForInstancedStructTestB>().SomeFloat, ServerObject->InstancedStructArray[1].Get<FStructForInstancedStructTestB>().SomeFloat);

	// Switch type
	ServerObject->InstancedStructArray[2].InitializeAs<FStructForInstancedStructTestA>();
	ServerObject->InstancedStructArray[2].GetMutable<FStructForInstancedStructTestA>().SomeUint16 += 1;

	// Replicate
	Server->UpdateAndSend({ Client });

	// Verified that the client got the modified value
	UE_NET_ASSERT_EQ(ClientObject->InstancedStructArray[2], ClientObject->InstancedStructArray[2]);
	UE_NET_ASSERT_EQ(ClientObject->InstancedStructArray[2].Get<FStructForInstancedStructTestA>().SomeUint16, ServerObject->InstancedStructArray[2].Get<FStructForInstancedStructTestA>().SomeUint16);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, NonReplicatedPropertiesAreNotClobbered)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UInstancedStructNetSerializerTestObject* ServerObject = Server->CreateObject<UInstancedStructNetSerializerTestObject>();

	// Initialize struct with some non-replicated properties.
	ServerObject->InstancedStruct.InitializeAs<FStructForInstancedStructTestWithNonReplicatedProperties>();
	ServerObject->InstancedStruct.GetMutable<FStructForInstancedStructTestWithNonReplicatedProperties>().IntA = 1333;
	
	// Replicate
	Server->UpdateAndSend({ Client });
	
	UInstancedStructNetSerializerTestObject* ClientObject = Client->GetObjectAs<UInstancedStructNetSerializerTestObject>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	CA_ASSUME(ClientObject != nullptr);
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestWithNonReplicatedProperties>().IntA, ServerObject->InstancedStruct.Get<FStructForInstancedStructTestWithNonReplicatedProperties>().IntA);

	// Modify value and verify that it's replicated as expected
	ServerObject->InstancedStruct.GetMutable<FStructForInstancedStructTestWithNonReplicatedProperties>().IntA = 1337;

	// Modify non-replicated property on the client
	ClientObject->InstancedStruct.GetMutable<FStructForInstancedStructTestWithNonReplicatedProperties>().IntB = 1331;

	// Replicate
	Server->UpdateAndSend({ Client });

	// Verified that the client got the modified value
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestWithNonReplicatedProperties>().IntA, ServerObject->InstancedStruct.Get<FStructForInstancedStructTestWithNonReplicatedProperties>().IntA);

	// Verified that the non-replicated property remained unchanged on the client
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestWithNonReplicatedProperties>().IntB, 1331);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, NetSerializerApplyIsCalled)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UInstancedStructNetSerializerTestObject* ServerObject = Server->CreateObject<UInstancedStructNetSerializerTestObject>();

	// Initialize struct with custom apply.
	ServerObject->InstancedStruct.InitializeAs<FStructForInstancedStructTestWithCustomApply>();
	ServerObject->InstancedStruct.GetMutable<FStructForInstancedStructTestWithCustomApply>().Uint = 1333;
	
	// Replicate
	Server->UpdateAndSend({ Client });
	
	UInstancedStructNetSerializerTestObject* ClientObject = Client->GetObjectAs<UInstancedStructNetSerializerTestObject>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	CA_ASSUME(ClientObject != nullptr);
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestWithCustomApply>().Uint, ServerObject->InstancedStruct.Get<FStructForInstancedStructTestWithCustomApply>().Uint);

	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestWithCustomApply>().ApplyCallCount, 1U);

	// Modify value and verify that it's replicated as expected
	ServerObject->InstancedStruct.GetMutable<FStructForInstancedStructTestWithCustomApply>().Uint = 1337;

	// Modify Apply handled property and verify that it doesn't overwrite the client value
	ServerObject->InstancedStruct.GetMutable<FStructForInstancedStructTestWithCustomApply>().ApplyCallCount = 1337;

	// Replicate
	Server->UpdateAndSend({ Client });

	// Verified that the client got the modified value
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestWithCustomApply>().Uint, ServerObject->InstancedStruct.Get<FStructForInstancedStructTestWithCustomApply>().Uint);

	// Verified that the custom handled property in Apply is as expected
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestWithCustomApply>().ApplyCallCount, 2U);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, CustomSerializedStructIsOverwritten)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UInstancedStructNetSerializerTestObject* ServerObject = Server->CreateObject<UInstancedStructNetSerializerTestObject>();

	// Initialize struct with custom serializer.
	ServerObject->InstancedStruct.InitializeAs<FStructForInstancedStructTestWithCustomSerializer>();
	ServerObject->InstancedStruct.GetMutable<FStructForInstancedStructTestWithCustomSerializer>().Uint = 1333;
	
	// Replicate
	Server->UpdateAndSend({ Client });
	
	UInstancedStructNetSerializerTestObject* ClientObject = Client->GetObjectAs<UInstancedStructNetSerializerTestObject>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	CA_ASSUME(ClientObject != nullptr);
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestWithCustomSerializer>().Uint, ServerObject->InstancedStruct.Get<FStructForInstancedStructTestWithCustomSerializer>().Uint);

	// Modify value and verify that it's replicated as expected
	ServerObject->InstancedStruct.GetMutable<FStructForInstancedStructTestWithCustomSerializer>().Uint = 1337;

	// Modify not replicated client values and verify they're overwritten since it's a custom serializer without apply
	ClientObject->InstancedStruct.GetMutable<FStructForInstancedStructTestWithCustomSerializer>().NotReplicatedUint = 1338;
	ClientObject->InstancedStruct.GetMutable<FStructForInstancedStructTestWithCustomSerializer>().NotPropertyUint = 1339;

	// Replicate
	Server->UpdateAndSend({ Client });

	// Verified that the client got the modified value
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestWithCustomSerializer>().Uint, ServerObject->InstancedStruct.Get<FStructForInstancedStructTestWithCustomSerializer>().Uint);

	// Verified that the not replicated properties did get overwritten
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestWithCustomSerializer>().NotReplicatedUint, ServerObject->InstancedStruct.Get<FStructForInstancedStructTestWithCustomSerializer>().NotReplicatedUint);
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestWithCustomSerializer>().NotPropertyUint, ServerObject->InstancedStruct.Get<FStructForInstancedStructTestWithCustomSerializer>().NotPropertyUint);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, NotFullyReplicatedStructIsAppliedCorrectly)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UInstancedStructNetSerializerTestObject* ServerObject = Server->CreateObject<UInstancedStructNetSerializerTestObject>();

	// Initialize with a not fully replicated struct
	ServerObject->InstancedStruct.InitializeAs<FStructForInstancedStructTestWithNonReplicatedProperties>();
	
	// Replicate
	Server->UpdateAndSend({ Client });
	
	UInstancedStructNetSerializerTestObject* ClientObject = Client->GetObjectAs<UInstancedStructNetSerializerTestObject>(ServerObject->NetRefHandle);

	// Modify not replicated value on the client
	ClientObject->InstancedStruct.GetMutable<FStructForInstancedStructTestWithNonReplicatedProperties>().IntB = 1337;

	// Modify value on server to trigger replication and apply of the struct
	ServerObject->InstancedStruct.GetMutable<FStructForInstancedStructTestWithNonReplicatedProperties>().IntA += 1;

	// Replicate
	Server->UpdateAndSend({ Client });

	// Verified that the client got the modified value
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestWithNonReplicatedProperties>().IntA, ServerObject->InstancedStruct.Get<FStructForInstancedStructTestWithNonReplicatedProperties>().IntA);

	// Verify the not replicated value remained as previously set
	UE_NET_ASSERT_EQ(ClientObject->InstancedStruct.Get<FStructForInstancedStructTestWithNonReplicatedProperties>().IntB, 1337);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, NotFullyReplicatedStructInArrayIsAppliedCorrectly)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UInstancedStructNetSerializerTestObject* ServerObject = Server->CreateObject<UInstancedStructNetSerializerTestObject>();

	// Initialize an array entry with a not fully replicated struct
	ServerObject->InstancedStructArray.Add(FInstancedStruct::Make<FStructForInstancedStructTestWithNonReplicatedProperties>());
	
	// Replicate
	Server->UpdateAndSend({ Client });
	
	UInstancedStructNetSerializerTestObject* ClientObject = Client->GetObjectAs<UInstancedStructNetSerializerTestObject>(ServerObject->NetRefHandle);

	// Modify not replicated value on the client
	ClientObject->InstancedStructArray[0].GetMutable<FStructForInstancedStructTestWithNonReplicatedProperties>().IntB = 1337;

	// Modify value on server to trigger replication and apply of the struct
	ServerObject->InstancedStructArray[0].GetMutable<FStructForInstancedStructTestWithNonReplicatedProperties>().IntA += 1;

	// Replicate
	Server->UpdateAndSend({ Client });

	// Verified that the client got the modified value
	UE_NET_ASSERT_EQ(ClientObject->InstancedStructArray[0].Get<FStructForInstancedStructTestWithNonReplicatedProperties>().IntA, ServerObject->InstancedStructArray[0].Get<FStructForInstancedStructTestWithNonReplicatedProperties>().IntA);

	// Verify the not replicated value remained as previously set
	UE_NET_ASSERT_EQ(ClientObject->InstancedStructArray[0].Get<FStructForInstancedStructTestWithNonReplicatedProperties>().IntB, 1337);
}

UE_NET_TEST_FIXTURE(FTestInstancedStructNetSerializerFixture, NotFullyReplicatedStructInArrayInStructIsAppliedCorrectly)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UInstancedStructNetSerializerTestObject* ServerObject = Server->CreateObject<UInstancedStructNetSerializerTestObject>();

	// Initialize an array entry with a not fully replicated struct
	ServerObject->StructWithInstancedStructArray.InstancedStructArray.Add(FInstancedStruct::Make<FStructForInstancedStructTestWithNonReplicatedProperties>());
	
	// Replicate
	Server->UpdateAndSend({ Client });
	
	UInstancedStructNetSerializerTestObject* ClientObject = Client->GetObjectAs<UInstancedStructNetSerializerTestObject>(ServerObject->NetRefHandle);

	// Modify not replicated value on the client
	ClientObject->StructWithInstancedStructArray.InstancedStructArray[0].GetMutable<FStructForInstancedStructTestWithNonReplicatedProperties>().IntB = 1337;

	// Modify value on server to trigger replication and apply of the struct
	ServerObject->StructWithInstancedStructArray.InstancedStructArray[0].GetMutable<FStructForInstancedStructTestWithNonReplicatedProperties>().IntA += 1;

	// Replicate
	Server->UpdateAndSend({ Client });

	// Verified that the client got the modified value
	UE_NET_ASSERT_EQ(ClientObject->StructWithInstancedStructArray.InstancedStructArray[0].Get<FStructForInstancedStructTestWithNonReplicatedProperties>().IntA, ServerObject->StructWithInstancedStructArray.InstancedStructArray[0].Get<FStructForInstancedStructTestWithNonReplicatedProperties>().IntA);

	// Verify the not replicated value remained as previously set
	UE_NET_ASSERT_EQ(ClientObject->StructWithInstancedStructArray.InstancedStructArray[0].Get<FStructForInstancedStructTestWithNonReplicatedProperties>().IntB, 1337);
}

// Fixture implementation
void FTestInstancedStructNetSerializerFixture::SetUp()
{
	FReplicationSystemServerClientTestFixture::SetUp();

	// Init default serialization context
	InternalNetSerializationContext.ReplicationSystem = Server->ReplicationSystem;

	FInternalNetSerializationContext TempInternalNetSerializationContext;
	FInternalNetSerializationContext::FInitParameters TempInternalNetSerializationContextInitParams;
	TempInternalNetSerializationContextInitParams.ReplicationSystem = Server->ReplicationSystem;
	TempInternalNetSerializationContextInitParams.ObjectResolveContext.RemoteNetTokenStoreState = Server->ReplicationSystem->GetNetTokenStore()->GetLocalNetTokenStoreState();
	TempInternalNetSerializationContext.Init(TempInternalNetSerializationContextInitParams);

	InternalNetSerializationContext = MoveTemp(TempInternalNetSerializationContext);
	NetSerializationContext.SetInternalContext(&InternalNetSerializationContext);

	FMemory::Memzero(QuantizedBuffer, sizeof(QuantizedBuffer));

	bHasQuantizedState = false;
	bHasClonedQuantizedState = false;

	if (!InstancedStructNetSerializerConfig.StateDescriptor.IsValid())
	{
		FReplicationStateDescriptorBuilder::FParameters Params;
		InstancedStructNetSerializerConfig.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(FTestInstancedStruct::StaticStruct(), Params);
	}

	if (!InstancedStructArrayNetSerializerConfig.StateDescriptor.IsValid())
	{
		FReplicationStateDescriptorBuilder::FParameters Params;
		InstancedStructArrayNetSerializerConfig.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(FTestInstancedStructArray::StaticStruct(), Params);
	}
}

void FTestInstancedStructNetSerializerFixture::TearDown()
{
	InstancedStruct0.Reset();
	InstancedStruct1.Reset();
	InstancedStructArray0.Reset();
	InstancedStructArray1.Reset();

	FreeQuantizedState();

	FReplicationSystemServerClientTestFixture::TearDown();
}

void FTestInstancedStructNetSerializerFixture::Serialize()
{
	// Must have run quantize before this
	UE_NET_ASSERT_TRUE(bHasQuantizedState);

	// Serialize data
	Writer.InitBytes(BitStreamBuffer, sizeof(BitStreamBuffer));
	FNetSerializationContext Context(&Writer);
	Context.SetInternalContext(NetSerializationContext.GetInternalContext());

	FNetSerializeArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetInstancedStructNetSerializerConfig(bIsTestingArray));
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	GetInstancedStructNetSerializer(bIsTestingArray)->Serialize(Context, Args);

	Writer.CommitWrites();

	UE_NET_ASSERT_FALSE(Context.HasError());
	UE_NET_ASSERT_GT(Writer.GetPosBits(), 0U);
}

void FTestInstancedStructNetSerializerFixture::Deserialize()
{
	// Check pre-conditions
	UE_NET_ASSERT_FALSE(bHasQuantizedState);
	UE_NET_ASSERT_GT(Writer.GetPosBytes(), 0U);

	Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());

	FNetSerializationContext Context(&Reader);
	Context.SetInternalContext(NetSerializationContext.GetInternalContext());

	FNetDeserializeArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetInstancedStructNetSerializerConfig(bIsTestingArray));
	Args.Target = NetSerializerValuePointer(&QuantizedBuffer[0]);
	GetInstancedStructNetSerializer(bIsTestingArray)->Deserialize(Context, Args);

	bHasQuantizedState = true;

	UE_NET_ASSERT_FALSE(Context.HasErrorOrOverflow());
	UE_NET_ASSERT_GT(Reader.GetPosBits(), 0U);
}

void FTestInstancedStructNetSerializerFixture::SerializeDelta()
{
	// Check pre-conditions
	UE_NET_ASSERT_TRUE(bHasQuantizedState);
	UE_NET_ASSERT_EQ(QuantizedStateCount, 2U);

	// Serialize data
	Writer.InitBytes(BitStreamBuffer, sizeof(BitStreamBuffer));
	FNetSerializationContext Context(&Writer);
	Context.SetInternalContext(NetSerializationContext.GetInternalContext());

	FNetSerializeDeltaArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetInstancedStructNetSerializerConfig(bIsTestingArray));
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Prev = NetSerializerValuePointer(&QuantizedBuffer[1]);
	GetInstancedStructNetSerializer(bIsTestingArray)->SerializeDelta(Context, Args);

	Writer.CommitWrites();

	UE_NET_ASSERT_FALSE(Context.HasErrorOrOverflow());
	UE_NET_ASSERT_GT(Writer.GetPosBits(), 0U);
}

void FTestInstancedStructNetSerializerFixture::DeserializeDelta()
{
	// Check pre-conditions
	UE_NET_ASSERT_GT(Writer.GetPosBytes(), 0U);

	Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());

	FNetSerializationContext Context(&Reader);
	Context.SetInternalContext(NetSerializationContext.GetInternalContext());

	FNetDeserializeDeltaArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetInstancedStructNetSerializerConfig(bIsTestingArray));
	Args.Target = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Prev = NetSerializerValuePointer(&QuantizedBuffer[1]);
	GetInstancedStructNetSerializer(bIsTestingArray)->DeserializeDelta(Context, Args);

	bHasQuantizedState = true;
	QuantizedStateCount = 1;

	UE_NET_ASSERT_FALSE(Context.HasErrorOrOverflow());
	UE_NET_ASSERT_GT(Reader.GetPosBits(), 0U);
}

void FTestInstancedStructNetSerializerFixture::Quantize()
{
	FNetQuantizeArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetInstancedStructNetSerializerConfig(bIsTestingArray));
	Args.Target = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Source = bIsTestingArray ? NetSerializerValuePointer(&InstancedStructArray0) : NetSerializerValuePointer(&InstancedStruct0);
	GetInstancedStructNetSerializer(bIsTestingArray)->Quantize(NetSerializationContext, Args);

	bHasQuantizedState = true;
	QuantizedStateCount = 1;

	UE_NET_ASSERT_FALSE(NetSerializationContext.HasError());
}

void FTestInstancedStructNetSerializerFixture::QuantizeTwoStates()
{
	Quantize();

	FNetQuantizeArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetInstancedStructNetSerializerConfig(bIsTestingArray));
	Args.Target = NetSerializerValuePointer(&QuantizedBuffer[1]);
	Args.Source = bIsTestingArray ? NetSerializerValuePointer(&InstancedStructArray1) : NetSerializerValuePointer(&InstancedStruct1);
	GetInstancedStructNetSerializer(bIsTestingArray)->Quantize(NetSerializationContext, Args);

	bHasQuantizedState = true;
	QuantizedStateCount = 2;

	UE_NET_ASSERT_FALSE(NetSerializationContext.HasError());
}

void FTestInstancedStructNetSerializerFixture::Clone()
{
	// Check pre-conditions
	UE_NET_ASSERT_TRUE(bHasQuantizedState);

	FMemory::Memcpy(ClonedQuantizedBuffer[0], QuantizedBuffer[0], sizeof(QuantizedBuffer[0]));

	FNetCloneDynamicStateArgs Args = {};
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Target = NetSerializerValuePointer(&ClonedQuantizedBuffer[0]);
	Args.NetSerializerConfig = NetSerializerConfigParam(GetInstancedStructNetSerializerConfig(bIsTestingArray));
	GetInstancedStructNetSerializer(bIsTestingArray)->CloneDynamicState(NetSerializationContext, Args);

	bHasClonedQuantizedState = true;
	ClonedQuantizedStateCount = 1;
}

void FTestInstancedStructNetSerializerFixture::FreeQuantizedState()
{
	FNetFreeDynamicStateArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetInstancedStructNetSerializerConfig(bIsTestingArray));

	if (bHasQuantizedState)
	{
		for (uint32 StateIt = 0, StateEndIt = QuantizedStateCount; StateIt != StateEndIt; ++StateIt)
		{
			Args.Source = NetSerializerValuePointer(&QuantizedBuffer[StateIt]);
			GetInstancedStructNetSerializer(bIsTestingArray)->FreeDynamicState(NetSerializationContext, Args);

			FMemory::Memzero(&QuantizedBuffer[StateIt], sizeof(QuantizedBuffer[StateIt]));
		}
		bHasQuantizedState = false;
	}

	if (bHasClonedQuantizedState)
	{
		for (uint32 StateIt = 0, StateEndIt = ClonedQuantizedStateCount; StateIt != StateEndIt; ++StateIt)
		{
			Args.Source = NetSerializerValuePointer(&ClonedQuantizedBuffer[StateIt]);
			GetInstancedStructNetSerializer(bIsTestingArray)->FreeDynamicState(NetSerializationContext, Args);

			FMemory::Memzero(&ClonedQuantizedBuffer[StateIt], sizeof(ClonedQuantizedBuffer[StateIt]));
		}
		bHasClonedQuantizedState = false;
	}
}

void FTestInstancedStructNetSerializerFixture::Dequantize()
{
	UE_NET_ASSERT_TRUE(bHasQuantizedState);

	FNetDequantizeArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetInstancedStructNetSerializerConfig(bIsTestingArray));
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Target = bIsTestingArray ? NetSerializerValuePointer(&InstancedStructArray1) : NetSerializerValuePointer(&InstancedStruct1);
	GetInstancedStructNetSerializer(bIsTestingArray)->Dequantize(NetSerializationContext, Args);
}

bool FTestInstancedStructNetSerializerFixture::IsEqual(bool bQuantized)
{
	if (bQuantized)
	{
		UE_NET_EXPECT_TRUE(bHasQuantizedState);
		if (!bHasQuantizedState)
		{
			return false;
		}

		UE_NET_EXPECT_TRUE(bHasClonedQuantizedState);
		if (!bHasClonedQuantizedState)
		{
			return false;
		}
	}

	FNetIsEqualArgs Args = {};
	if (bQuantized)
	{
		Args.Source0 = NetSerializerValuePointer(&QuantizedBuffer[0]);
		Args.Source1 = NetSerializerValuePointer(&ClonedQuantizedBuffer[0]);
	}
	else
	{
		Args.Source0 = bIsTestingArray ? NetSerializerValuePointer(&InstancedStructArray0) : NetSerializerValuePointer(&InstancedStruct0);
		Args.Source1 = bIsTestingArray ? NetSerializerValuePointer(&InstancedStructArray1) : NetSerializerValuePointer(&InstancedStruct1);
	}
	Args.bStateIsQuantized = bQuantized;
	Args.NetSerializerConfig = NetSerializerConfigParam(GetInstancedStructNetSerializerConfig(bIsTestingArray));
	return GetInstancedStructNetSerializer(bIsTestingArray)->IsEqual(NetSerializationContext, Args);
}

void FTestInstancedStructNetSerializerFixture::Validate()
{
	FNetValidateArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetInstancedStructNetSerializerConfig(bIsTestingArray));
	Args.Source = bIsTestingArray ? NetSerializerValuePointer(&InstancedStructArray0) : NetSerializerValuePointer(&InstancedStruct0);

	GetInstancedStructNetSerializer(bIsTestingArray)->Validate(NetSerializationContext, Args);
}

void FTestInstancedStructNetSerializerFixture::SetNonDefaultInstanceState()
{
	InstancedStruct0.InitializeAs<FStructForInstancedStructTestA>();
	InstancedStruct0.GetMutable<FStructForInstancedStructTestA>().SomeUint16 += 4711U;
}

void FTestInstancedStructNetSerializerFixture::SetNonDefaultArrayState()
{
	InstancedStructArray0.Empty(7);
	InstancedStructArray0.SetNum(7);

	InstancedStructArray0[0].InitializeAs<FStructForInstancedStructTestA>();

	InstancedStructArray0[1].InitializeAs<FStructForInstancedStructTestB>();
	InstancedStructArray0[1].GetMutable<FStructForInstancedStructTestB>().SomeFloat = 1234.5f;

	InstancedStructArray0[2].Reset();

	InstancedStructArray0[3].InitializeAs<FStructForInstancedStructTestC>();
	InstancedStructArray0[3].GetMutable<FStructForInstancedStructTestC>().SomeBool ^= true;

	InstancedStructArray0[4].Reset();

	InstancedStructArray0[5].InitializeAs<FStructForInstancedStructTestWithArray>();

	InstancedStructArray0[6].InitializeAs<FStructForInstancedStructTestWithObjectReference>();
}

const FNetSerializer* FTestInstancedStructNetSerializerFixture::GetInstancedStructNetSerializer(bool bIsArray) const
{
	if (bIsArray)
	{
		if (const FReplicationStateDescriptor* Desc = InstancedStructArrayNetSerializerConfig.StateDescriptor.GetReference())
		{
			return Desc->MemberSerializerDescriptors[0].Serializer;
		}
	}
	else
	{
		if (const FReplicationStateDescriptor* Desc = InstancedStructNetSerializerConfig.StateDescriptor.GetReference())
		{
			return Desc->MemberSerializerDescriptors[0].Serializer;
		}
	}

	return nullptr;
}

const FNetSerializerConfig* FTestInstancedStructNetSerializerFixture::GetInstancedStructNetSerializerConfig(bool bIsArray) const
{
	if (bIsArray)
	{
		if (const FReplicationStateDescriptor* Desc = InstancedStructArrayNetSerializerConfig.StateDescriptor.GetReference())
		{
			return Desc->MemberSerializerDescriptors[0].SerializerConfig;
		}
	}
	else
	{
		if (const FReplicationStateDescriptor* Desc = InstancedStructNetSerializerConfig.StateDescriptor.GetReference())
		{
			return Desc->MemberSerializerDescriptors[0].SerializerConfig;
		}
	}

	return nullptr;
}

}

namespace UE::Net
{

void FStructForInstancedStructTestWithCustomApplyNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	Context.GetBitStreamWriter()->WriteBits(Value, 32);
}

void FStructForInstancedStructTestWithCustomApplyNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	Target = Context.GetBitStreamReader()->ReadBits(32);
}

void FStructForInstancedStructTestWithCustomApplyNetSerializer::Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	Target = Source.Uint;
}

void FStructForInstancedStructTestWithCustomApplyNetSerializer::Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	Target.Uint = Source;
}

bool FStructForInstancedStructTestWithCustomApplyNetSerializer::IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& QuantizedValue0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& QuantizedValue1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);
		return QuantizedValue0 == QuantizedValue1;
	}
	else
	{
		const SourceType& SourceValue0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& SourceValue1 = *reinterpret_cast<const SourceType*>(Args.Source1);
		return SourceValue0.Uint == SourceValue1.Uint;
	}
}

void FStructForInstancedStructTestWithCustomApplyNetSerializer::Apply(FNetSerializationContext& Context, const FNetApplyArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	Target.Uint = Source.Uint;
	Target.ApplyCallCount++;
}

FStructForInstancedStructTestWithCustomApplyNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(StructForInstancedStructTestWithCustomApplyNetSerializer);
}

void FStructForInstancedStructTestWithCustomApplyNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(StructForInstancedStructTestWithCustomApplyNetSerializer);
}

void FStructForInstancedStructTestWithCustomSerializerNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	Context.GetBitStreamWriter()->WriteBits(Value, 32);
}

void FStructForInstancedStructTestWithCustomSerializerNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	Target = Context.GetBitStreamReader()->ReadBits(32);
}

void FStructForInstancedStructTestWithCustomSerializerNetSerializer::Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	Target = Source.Uint;
}

void FStructForInstancedStructTestWithCustomSerializerNetSerializer::Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	Target.Uint = Source;
}

bool FStructForInstancedStructTestWithCustomSerializerNetSerializer::IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& QuantizedValue0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& QuantizedValue1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);
		return QuantizedValue0 == QuantizedValue1;
	}
	else
	{
		const SourceType& SourceValue0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& SourceValue1 = *reinterpret_cast<const SourceType*>(Args.Source1);
		return SourceValue0.Uint == SourceValue1.Uint;
	}
}

FStructForInstancedStructTestWithCustomSerializerNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(StructForInstancedStructTestWithCustomSerializerNetSerializer);
}

void FStructForInstancedStructTestWithCustomSerializerNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(StructForInstancedStructTestWithCustomSerializerNetSerializer);
}

}

//
UInstancedStructNetSerializerTestObject::UInstancedStructNetSerializerTestObject()
{
	ConstructorInitializedInstancedStruct.InitializeAs<FStructForInstancedStructTestA>();
}

void UInstancedStructNetSerializerTestObject::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = false;

	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, ConstructorInitializedInstancedStruct, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, InstancedStruct, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, InstancedStructArray, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ThisClass, StructWithInstancedStructArray, Params);
}

void UInstancedStructNetSerializerTestObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}
