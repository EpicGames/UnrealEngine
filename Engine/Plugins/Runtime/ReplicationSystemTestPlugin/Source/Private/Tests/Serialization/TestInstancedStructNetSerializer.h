// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Iris/Serialization/NetSerializer.h"
#include "StructUtils/InstancedStruct.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "TestInstancedStructNetSerializer.generated.h"

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FStructForInstancedStructTestWithCustomApplyNetSerializer, REPLICATIONSYSTEMTESTPLUGIN_API);
UE_NET_DECLARE_SERIALIZER(FStructForInstancedStructTestWithCustomSerializerNetSerializer, REPLICATIONSYSTEMTESTPLUGIN_API);

}

USTRUCT()
struct FTestInstancedStruct
{
	GENERATED_BODY()

	UPROPERTY()
	FInstancedStruct InstancedStruct;
};

USTRUCT()
struct FTestInstancedStructArray
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FInstancedStruct> InstancedStructArray;
};

USTRUCT()
struct FStructForInstancedStructTestA
{
	GENERATED_BODY()

	UPROPERTY()
	uint16 SomeUint16 = 0;
};

USTRUCT()
struct FStructForInstancedStructTestB
{
	GENERATED_BODY()

	UPROPERTY()
	float SomeFloat = 0.0f;
};

USTRUCT()
struct FStructForInstancedStructTestC
{
	GENERATED_BODY()

	UPROPERTY()
	bool SomeBool = false;
};

USTRUCT()
struct FStructForInstancedStructTestD
{
	GENERATED_BODY()

	// Intentionally has no properties
};

USTRUCT()
struct FStructForInstancedStructTestWithArray
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FStructForInstancedStructTestB> ArrayOfTestB;
};

USTRUCT()
struct FStructForInstancedStructTestWithObjectReference
{
	GENERATED_BODY()

	UPROPERTY();
	TObjectPtr<UObject> SomeObject = nullptr;
};

USTRUCT()
struct FStructForInstancedStructTestWithCustomApply
{
	GENERATED_BODY()

	UPROPERTY();
	uint32 Uint = 0;

	// Incremented by 1 in this struct's NetSerializer Apply method. It's set to replicate but Apply trumps that.
	UPROPERTY()
	uint32 ApplyCallCount = 0;
};

USTRUCT()
struct FStructForInstancedStructTestWithCustomSerializer
{
	GENERATED_BODY()

	UPROPERTY();
	uint32 Uint = 0;

	UPROPERTY(NotReplicated)
	uint32 NotReplicatedUint = 0;

	uint32 NotPropertyUint = 0;
};

USTRUCT()
struct FStructForInstancedStructTestWithNonReplicatedProperties
{
	GENERATED_BODY()

	UPROPERTY();
	int IntA = 0;

	UPROPERTY(NotReplicated);
	int IntB = 0;
};

UCLASS()
class UInstancedStructNetSerializerTestObject : public UReplicatedTestObject
{
	GENERATED_BODY()

protected:
	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

public:
	UInstancedStructNetSerializerTestObject();

	UPROPERTY(Replicated, Transient)
	FInstancedStruct ConstructorInitializedInstancedStruct;

	UPROPERTY(Replicated, Transient)
	FInstancedStruct InstancedStruct;

	UPROPERTY(Replicated, Transient)
	TArray<FInstancedStruct> InstancedStructArray;

	UPROPERTY(Replicated, Transient)
	FTestInstancedStructArray StructWithInstancedStructArray;
};

