// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/IntNetSerializers.h"
#include <limits>

namespace UE::Net::Private
{

static FTestMessage& PrintIntNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	const FIntNetSerializerConfig& Config = static_cast<const FIntNetSerializerConfig&>(InConfig);
	return Message << "BitCount: "  << Config.BitCount;
}

template<typename SourceType>
class FTestIntNetSerializer : public TTestNetSerializerFixture<PrintIntNetSerializerConfig, SourceType>
{
	typedef TTestNetSerializerFixture<PrintIntNetSerializerConfig, SourceType> Super;

public:
	FTestIntNetSerializer(const FNetSerializer& Serializer) : Super(Serializer) {}

	void TestIsEqual();
	void TestValidate();
	void TestSerialize();
	void TestSerializeDelta();
	void TestDefaultConfig();

protected:
	static const SourceType Values[];
	static const SIZE_T ValueCount;
};

#define UE_NET_IMPLEMENT_INT_NETSERIALIZER_TEST(TestClassName, NetSerializerType, SourceType) \
class TestClassName : public FTestIntNetSerializer<SourceType> \
{ \
public: \
	TestClassName() : FTestIntNetSerializer<SourceType>(UE_NET_GET_SERIALIZER(NetSerializerType)) {} \
}; \
\
UE_NET_TEST_FIXTURE(TestClassName, HasTestValues) \
{ \
	UE_NET_ASSERT_GT_MSG(ValueCount, SIZE_T(0), "No test values found"); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestIsEqual) \
{ \
	TestIsEqual(); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestValidate) \
{ \
	TestValidate(); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestSerialize) \
{ \
	TestSerialize(); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestSerializeDelta) \
{ \
	TestSerializeDelta(); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestDefaultConfig) \
{ \
	TestDefaultConfig(); \
} \

UE_NET_IMPLEMENT_INT_NETSERIALIZER_TEST(FTestInt64NetSerializer, FInt64NetSerializer, int64);
UE_NET_IMPLEMENT_INT_NETSERIALIZER_TEST(FTestInt32NetSerializer, FInt32NetSerializer, int32);
UE_NET_IMPLEMENT_INT_NETSERIALIZER_TEST(FTestInt16NetSerializer, FInt16NetSerializer, int16);
UE_NET_IMPLEMENT_INT_NETSERIALIZER_TEST(FTestInt8NetSerializer, FInt8NetSerializer, int8);

#undef UE_NET_IMPLEMENT_INT_NETSERIALIZER_TEST

//
template<typename SourceType> const SourceType FTestIntNetSerializer<SourceType>::Values[] =
{
	std::numeric_limits<SourceType>::min(), std::numeric_limits<SourceType>::max(), SourceType(0), SourceType(-1), SourceType(-16), SourceType(2048458 % std::numeric_limits<SourceType>::max())
};
template<typename SourceType> const SIZE_T FTestIntNetSerializer<SourceType>::ValueCount = sizeof(Values)/sizeof(Values[0]);

template<typename SourceType>
void FTestIntNetSerializer<SourceType>::TestIsEqual()
{
	constexpr uint8 MinBitCount = 1;
	constexpr uint8 MaxBitCount = sizeof(SourceType)*8;

	SourceType CompareValues[2][sizeof(Values)/sizeof(Values[0])];
	bool ExpectedResults[2][sizeof(Values)/sizeof(Values[0])];
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		CompareValues[0][ValueIt] = Values[ValueIt];
		ExpectedResults[0][ValueIt] = true;
	}
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		CompareValues[1][ValueIt] = ~Values[ValueIt];
		ExpectedResults[1][ValueIt] = false;
	}

	// Do two rounds of testing, one where we compare each value with itself and one where we compare against a guaranteed non-equal value.
	// We will only do an unquantized compare since we know these serializers uses a default quantize implementation.
	for (SIZE_T TestRoundIt = 0, TestRoundEndIt = 2; TestRoundIt != TestRoundEndIt; ++TestRoundIt)
	{
		for (uint8 BitCount = MinBitCount; BitCount <= MaxBitCount; ++BitCount)
		{
			FIntNetSerializerConfig Config(BitCount);

			constexpr bool bQuantizedCompare = false;
			const bool bSuccess = Super::TestIsEqual(Values, CompareValues[TestRoundIt], ExpectedResults[TestRoundIt], ValueCount, Config, bQuantizedCompare);
			if (!bSuccess)
			{
				return;
			}
		}
	}
}

template<typename SourceType>
void FTestIntNetSerializer<SourceType>::TestValidate()
{
	constexpr uint8 MinBitCount = 1;
	constexpr uint8 MaxBitCount = sizeof(SourceType)*8;

	bool ExpectedResults[sizeof(Values)/sizeof(Values[0])];
	for (uint8 BitCount = MinBitCount; BitCount <= MaxBitCount; ++BitCount)
	{
		constexpr SourceType MaxValue = std::numeric_limits<SourceType>::max();
		const SourceType MaxValueForBitCount = MaxValue >> (sizeof(SourceType)*8 - BitCount);
		const SourceType MinValueForBitCount = -MaxValueForBitCount - SourceType(1);
		for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
		{
			const SourceType Value = Values[ValueIt];
			ExpectedResults[ValueIt] = (FMath::Clamp(Value, MinValueForBitCount, MaxValueForBitCount) == Value);
			if (BitCount == sizeof(SourceType)*8)
			{
				UE_NET_ASSERT_TRUE_MSG(ExpectedResults[ValueIt], "Clamping of value with full bit precision resulted in value being clamped. This is unexpected. Make sure no undefined behavior is used in code.");
			}
		}

		FIntNetSerializerConfig Config(BitCount);

		const bool bSuccess = Super::TestValidate(Values, ExpectedResults, ValueCount, Config);
		if (!bSuccess)
		{
			return;
		}
	}
}

template<typename SourceType>
void FTestIntNetSerializer<SourceType>::TestSerialize()
{
	constexpr uint8 MinBitCount = 1;
	constexpr uint8 MaxBitCount = sizeof(SourceType)*8;

	// Setup expected results. The IntPackers should only consider the BitCount least significant bits. Our test values may have bits set outside that range.
	// But values with bits sets that won't be serialized should be considered equal as long as the least significant bits are set. This is because serializing
	// either value will result in the same value being deserialized. It may not be euqal the the original value though. The NetSerializer's Validate function
	// is there to detect that kind of situations and warn the user that the result is undefined.
	SourceType ExpectedValues[sizeof(Values) / sizeof(Values[0])];
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		ExpectedValues[ValueIt] = Values[ValueIt];
	}

	const auto& EqualityFunc = [](NetSerializerValuePointer Value0, NetSerializerValuePointer Value1) -> bool { return *reinterpret_cast<SourceType*>(Value0) == *reinterpret_cast<SourceType*>(Value1); };

	for (uint8 BitCount = MinBitCount; BitCount <= MaxBitCount; ++BitCount)
	{
		FIntNetSerializerConfig Config(BitCount);

		constexpr bool bQuantizedCompare = false;
		TOptional<TFunctionRef<bool(NetSerializerValuePointer Value0, NetSerializerValuePointer Value1)>> CompareFunc;
		if (BitCount == MaxBitCount)
		{
			CompareFunc = EqualityFunc;
		}
		const bool bSuccess = Super::TestSerialize(Values, ExpectedValues, ValueCount, Config, bQuantizedCompare, CompareFunc);
		if (!bSuccess)
		{
			return;
		}
	}
}

template<typename SourceType>
void FTestIntNetSerializer<SourceType>::TestSerializeDelta()
{
	constexpr uint8 MinBitCount = 1;
	constexpr uint8 MaxBitCount = sizeof(SourceType)*8;

	for (uint8 BitCount = MinBitCount; BitCount <= MaxBitCount; ++BitCount)
	{
		FIntNetSerializerConfig Config(BitCount);
		Super::TestSerializeDelta(Values, ValueCount, Config);
	}
}

template<typename SourceType>
void FTestIntNetSerializer<SourceType>::TestDefaultConfig()
{
	const FIntNetSerializerConfig* DefaultConfig = static_cast<const FIntNetSerializerConfig*>(FTestNetSerializerFixture::Serializer.DefaultConfig);
	UE_NET_ASSERT_NE(DefaultConfig, nullptr);

	UE_NET_ASSERT_EQ(DefaultConfig->BitCount, static_cast<uint8>(sizeof(SourceType)*8u));
}

}
