// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Core/BitTwiddling.h"
#include "Iris/Serialization/IntRangeNetSerializers.h"
#include <limits>

namespace UE::Net::Private
{

template<typename SerializerConfig> static FTestMessage& PrintIntRangeNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	const SerializerConfig& Config = static_cast<const SerializerConfig&>(InConfig);
	return Message << "SmallestValue: " << Config.LowerBound << " LargestValue: " << Config.UpperBound << " BitCount: " << Config.BitCount;
}

template<typename InSerializerConfig, typename SourceType>
class FTestIntRangeNetSerializer : public TTestNetSerializerFixture<PrintIntRangeNetSerializerConfig<InSerializerConfig>, SourceType>
{
	typedef TTestNetSerializerFixture<PrintIntRangeNetSerializerConfig<InSerializerConfig>, SourceType> Super;

public:
	typedef InSerializerConfig SerializerConfig;

	FTestIntRangeNetSerializer(const FNetSerializer& Serializer) : Super(Serializer) { SetupConfigs(); }

	void TestIsEqual();
	void TestValidate();
	void TestQuantize();
	void TestSerialize();
	void TestSerializeDelta();

private:
	void SetupConfigs();

protected:
	static const SourceType Values[];
	static const SIZE_T ValueCount;

	SerializerConfig Configs[4];
	static const SIZE_T ConfigCount;
	static constexpr SIZE_T FullRangeConfigIndex = 0;
};

// FInt8RangeNetSerializer tests
class FTestInt8RangeNetSerializer : public FTestIntRangeNetSerializer<FInt8RangeNetSerializerConfig, int8>
{
public:
	FTestInt8RangeNetSerializer() : FTestIntRangeNetSerializer<SerializerConfig, int8>(UE_NET_GET_SERIALIZER(FInt8RangeNetSerializer)) {}
};

UE_NET_TEST_FIXTURE(FTestInt8RangeNetSerializer, HasTestValues)
{
	UE_NET_ASSERT_GT_MSG(ValueCount, SIZE_T(0), "No test values found");
}

UE_NET_TEST_FIXTURE(FTestInt8RangeNetSerializer, HasTestConfigs)
{
	UE_NET_ASSERT_GT_MSG(ConfigCount, SIZE_T(0), "No configs found");
}

UE_NET_TEST_FIXTURE(FTestInt8RangeNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestInt8RangeNetSerializer, TestValidate)
{
	TestValidate();
}

UE_NET_TEST_FIXTURE(FTestInt8RangeNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestInt8RangeNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestInt8RangeNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

// FInt16RangeNetSerializer tests
class FTestInt16RangeNetSerializer : public FTestIntRangeNetSerializer<FInt16RangeNetSerializerConfig, int16>
{
public:
	FTestInt16RangeNetSerializer() : FTestIntRangeNetSerializer<SerializerConfig, int16>(UE_NET_GET_SERIALIZER(FInt16RangeNetSerializer)) {}
};

UE_NET_TEST_FIXTURE(FTestInt16RangeNetSerializer, HasTestValues)
{
	UE_NET_ASSERT_GT_MSG(ValueCount, SIZE_T(0), "No test values found");
}

UE_NET_TEST_FIXTURE(FTestInt16RangeNetSerializer, HasTestConfigs)
{
	UE_NET_ASSERT_GT_MSG(ConfigCount, SIZE_T(0), "No configs found");
}

UE_NET_TEST_FIXTURE(FTestInt16RangeNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestInt16RangeNetSerializer, TestValidate)
{
	TestValidate();
}

UE_NET_TEST_FIXTURE(FTestInt16RangeNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestInt16RangeNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestInt16RangeNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

// FInt32RangeNetSerializer tests
class FTestInt32RangeNetSerializer : public FTestIntRangeNetSerializer<FInt32RangeNetSerializerConfig, int32>
{
public:
	FTestInt32RangeNetSerializer() : FTestIntRangeNetSerializer<SerializerConfig, int32>(UE_NET_GET_SERIALIZER(FInt32RangeNetSerializer)) {}
};

UE_NET_TEST_FIXTURE(FTestInt32RangeNetSerializer, HasTestValues)
{
	UE_NET_ASSERT_GT_MSG(ValueCount, SIZE_T(0), "No test values found");
}

UE_NET_TEST_FIXTURE(FTestInt32RangeNetSerializer, HasTestConfigs)
{
	UE_NET_ASSERT_GT_MSG(ConfigCount, SIZE_T(0), "No configs found");
}

UE_NET_TEST_FIXTURE(FTestInt32RangeNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestInt32RangeNetSerializer, TestValidate)
{
	TestValidate();
}

UE_NET_TEST_FIXTURE(FTestInt32RangeNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestInt32RangeNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestInt32RangeNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

// FInt64RangeNetSerializer tests
class FTestInt64RangeNetSerializer : public FTestIntRangeNetSerializer<FInt64RangeNetSerializerConfig, int64>
{
public:
	FTestInt64RangeNetSerializer() : FTestIntRangeNetSerializer<SerializerConfig, int64>(UE_NET_GET_SERIALIZER(FInt64RangeNetSerializer)) {}
};

UE_NET_TEST_FIXTURE(FTestInt64RangeNetSerializer, HasTestValues)
{
	UE_NET_ASSERT_GT_MSG(ValueCount, SIZE_T(0), "No test values found");
}

UE_NET_TEST_FIXTURE(FTestInt64RangeNetSerializer, HasTestConfigs)
{
	UE_NET_ASSERT_GT_MSG(ConfigCount, SIZE_T(0), "No configs found");
}

UE_NET_TEST_FIXTURE(FTestInt64RangeNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestInt64RangeNetSerializer, TestValidate)
{
	TestValidate();
}

UE_NET_TEST_FIXTURE(FTestInt64RangeNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestInt64RangeNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestInt64RangeNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

//
template<typename SerializerConfig, typename SourceType> const SourceType FTestIntRangeNetSerializer<SerializerConfig, SourceType>::Values[] =
{
	std::numeric_limits<SourceType>::min(), std::numeric_limits<SourceType>::max(), SourceType(0), SourceType(-1), SourceType(-16), SourceType(2048458 % std::numeric_limits<SourceType>::max())
};
template<typename SerializerConfig, typename SourceType> const SIZE_T FTestIntRangeNetSerializer<SerializerConfig, SourceType>::ValueCount = sizeof(Values)/sizeof(Values[0]);
template<typename SerializerConfig, typename SourceType> const SIZE_T FTestIntRangeNetSerializer<SerializerConfig, SourceType>::ConfigCount = sizeof(Configs)/sizeof(Configs[0]);

template<typename SerializerConfig, typename SourceType>
void FTestIntRangeNetSerializer<SerializerConfig, SourceType>::SetupConfigs()
{
	// Full range
	{
		static_assert(FullRangeConfigIndex == 0, "Incorrect NetSerializerConfig setup");
		SerializerConfig& Config = Configs[0];
		Config.LowerBound = std::numeric_limits<SourceType>::min();
		Config.UpperBound = std::numeric_limits<SourceType>::max();
		Config.BitCount = static_cast<uint8>(GetBitsNeededForRange(Config.LowerBound, Config.UpperBound));
	}

	// Zero range
	{
		SerializerConfig& Config = Configs[1];
		Config.LowerBound = -64;
		Config.UpperBound = -64;
		Config.BitCount = static_cast<uint8>(GetBitsNeededForRange(Config.LowerBound, Config.UpperBound));
	}

	// Arbitrary ranges
	{
		SerializerConfig& Config = Configs[2];
		Config.LowerBound = std::numeric_limits<SourceType>::min()/2;
		Config.UpperBound = std::numeric_limits<SourceType>::max()/4;
		Config.BitCount = static_cast<uint8>(GetBitsNeededForRange(Config.LowerBound, Config.UpperBound));
	}

	{
		SerializerConfig& Config = Configs[3];
		Config.LowerBound = std::numeric_limits<SourceType>::max()/4;
		Config.UpperBound = std::numeric_limits<SourceType>::max()/2;
		Config.BitCount = static_cast<uint8>(GetBitsNeededForRange(Config.LowerBound, Config.UpperBound));
	}
}

template<typename SerializerConfig, typename SourceType>
void FTestIntRangeNetSerializer<SerializerConfig, SourceType>::TestIsEqual()
{
	SourceType CompareValues[2][sizeof(Values)/sizeof(Values[0])];
	bool ExpectedResults[2][sizeof(Values)/sizeof(Values[0])];
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		CompareValues[0][ValueIt] = Values[ValueIt];
		ExpectedResults[0][ValueIt] = true;
	}

	// Do two rounds of testing per config, one where we compare each value with itself and one where we compare against a value in range.
	for (SIZE_T ConfigIt = 0; ConfigIt != ConfigCount; ++ConfigIt)
	{
		const SerializerConfig& Config = Configs[ConfigIt];

		for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
		{
			CompareValues[1][ValueIt] = Config.UpperBound;
			ExpectedResults[1][ValueIt] = (FMath::Clamp(CompareValues[1][ValueIt], Config.LowerBound, Config.UpperBound) == (FMath::Clamp(Values[ValueIt], Config.LowerBound, Config.UpperBound)));
		}

		for (SIZE_T TestRoundIt = 0, TestRoundEndIt = 2; TestRoundIt != TestRoundEndIt; ++TestRoundIt)
		{
			constexpr bool bQuantizedCompare = false;
			const bool bSuccess = Super::TestIsEqual(Values, CompareValues[TestRoundIt], ExpectedResults[TestRoundIt], ValueCount, Config, bQuantizedCompare);
			if (!bSuccess)
			{
				return;
			}
		}
	}
}

template<typename SerializerConfig, typename SourceType>
void FTestIntRangeNetSerializer<SerializerConfig, SourceType>::TestValidate()
{
	bool ExpectedResults[sizeof(Values)/sizeof(Values[0])];
	for (SIZE_T ConfigIt = 0; ConfigIt != ConfigCount; ++ConfigIt)
	{
		const SerializerConfig& Config = Configs[ConfigIt];
		for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
		{
			const SourceType Value = Values[ValueIt];
			ExpectedResults[ValueIt] = (FMath::Clamp(Value, Config.LowerBound, Config.UpperBound) == Value);
			if (ConfigIt == FullRangeConfigIndex)
			{
				UE_NET_ASSERT_TRUE_MSG(ExpectedResults[ValueIt], "Clamping of value with full bit precision resulted in value being clamped. This is unexpected. Make sure no undefined behavior is used in code.");
			}
		}

		const bool bSuccess = Super::TestValidate(Values, ExpectedResults, ValueCount, Config);
		if (!bSuccess)
		{
			return;
		}
	}
}

template<typename SerializerConfig, typename SourceType>
void FTestIntRangeNetSerializer<SerializerConfig, SourceType>::TestQuantize()
{
	for (SIZE_T ConfigIt = 0; ConfigIt != ConfigCount; ++ConfigIt)
	{
		const SerializerConfig& Config = Configs[ConfigIt];

		const bool bSuccess = Super::TestQuantize(Values, ValueCount, Config);
		if (!bSuccess)
		{
			return;
		}
	}
}

template<typename SerializerConfig, typename SourceType>
void FTestIntRangeNetSerializer<SerializerConfig, SourceType>::TestSerialize()
{
	SourceType ExpectedValues[sizeof(Values) / sizeof(Values[0])];
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		ExpectedValues[ValueIt] = Values[ValueIt];
	}

	const auto& EqualityFunc = [](NetSerializerValuePointer Value0, NetSerializerValuePointer Value1) -> bool { return *reinterpret_cast<SourceType*>(Value0) == *reinterpret_cast<SourceType*>(Value1); };

	for (SIZE_T ConfigIt = 0; ConfigIt != ConfigCount; ++ConfigIt)
	{
		const SerializerConfig& Config = Configs[ConfigIt];

		constexpr bool bQuantizedCompare = false;
		TOptional<TFunctionRef<bool(NetSerializerValuePointer Value0, NetSerializerValuePointer Value1)>> CompareFunc;
		if (ConfigIt == FullRangeConfigIndex)
		{
			CompareFunc = EqualityFunc;
		}
		const bool bSuccess = Super::TestSerialize(Values, ExpectedValues, ValueCount, Config, bQuantizedCompare, CompareFunc);
		if (!bSuccess)
		{
			return;
		}
	}
}

template<typename SerializerConfig, typename SourceType>
void FTestIntRangeNetSerializer<SerializerConfig, SourceType>::TestSerializeDelta()
{
	for (SIZE_T ConfigIt = 0; ConfigIt != ConfigCount; ++ConfigIt)
	{
		const SerializerConfig& Config = Configs[ConfigIt];

		const bool bSuccess = Super::TestSerializeDelta(Values, ValueCount, Config);
		if (!bSuccess)
		{
			return;
		}
	}
}

}
