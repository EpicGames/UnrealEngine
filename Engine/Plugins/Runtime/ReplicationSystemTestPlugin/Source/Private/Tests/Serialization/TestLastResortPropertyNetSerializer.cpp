// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestLastResortPropertyNetSerializer.h"
#include "TestNetSerializerFixture.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/Serialization/InternalNetSerializers.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/IrisObjectReferencePackageMap.h"

namespace UE::Net
{

FTestMessage& operator<<(FTestMessage& Message, const FText& Text)
{
	return Message << Text.ToString();
}

}

namespace UE::Net::Private
{

static FTestMessage& PrintLastResortPropertyNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	const FLastResortPropertyNetSerializerConfig& Config = static_cast<const FLastResortPropertyNetSerializerConfig&>(InConfig);
	return Message << "Property: " << Config.Property->GetClass()->GetName();
}

class FTestLastResortPropertyNetSerializer : public TTestNetSerializerFixture<PrintLastResortPropertyNetSerializerConfig, FText>
{
public:
	FTestLastResortPropertyNetSerializer()
	: Super(UE_NET_GET_SERIALIZER(FLastResortPropertyNetSerializer))
	, SerializerConfig(nullptr) 
	{
	}

protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	void TestQuantize();
	void TestIsEqual();
	void TestSerialize();
	void TestCloneDynamicState();

	typedef TTestNetSerializerFixture<PrintLastResortPropertyNetSerializerConfig, FText> Super;

	static const FText TestValues[];
	static const SIZE_T TestValueCount;

	Private::FInternalNetSerializationContext InternalContext;
	TRefCountPtr<const FReplicationStateDescriptor> ReplicationStateDescriptor;
	FLastResortPropertyNetSerializerConfig* SerializerConfig;
};

const FText FTestLastResortPropertyNetSerializer::TestValues[] = 
{
	FText(),
	INVTEXT("Just a regular ASCII string"),
	INVTEXT("\U0001F606\U0001F41C\u20AC\u00A9"),
};

const SIZE_T FTestLastResortPropertyNetSerializer::TestValueCount = sizeof(TestValues)/sizeof(TestValues[0]);

UE_NET_TEST_FIXTURE(FTestLastResortPropertyNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestLastResortPropertyNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestLastResortPropertyNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestLastResortPropertyNetSerializer, TestCloneDynamicState)
{
	TestCloneDynamicState();
}

void FTestLastResortPropertyNetSerializer::SetUp()
{
	Super::SetUp();

	InternalContext.PackageMap = NewObject<UIrisObjectReferencePackageMap>();
	Context.SetInternalContext(&InternalContext);

	ReplicationStateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(StaticStruct<FStructForLastResortPropertyNetSerializerTest>());
	UE_NET_ASSERT_EQ_MSG(ReplicationStateDescriptor->MemberCount, uint16(1), "Expected FStructForLastResortPropertyNetSerializerTest to contain exactly one member");


	SerializerConfig = const_cast<FLastResortPropertyNetSerializerConfig*>(static_cast<const FLastResortPropertyNetSerializerConfig*>(ReplicationStateDescriptor->MemberSerializerDescriptors[0].SerializerConfig));
	UE_NET_ASSERT_NE(SerializerConfig, nullptr);
}

void FTestLastResortPropertyNetSerializer::TearDown()
{
	SerializerConfig = nullptr;
	ReplicationStateDescriptor.SafeRelease();

	Super::TearDown();
}

void FTestLastResortPropertyNetSerializer::TestQuantize()
{
	const bool bSuccess = Super::TestQuantize(TestValues, TestValueCount, *SerializerConfig);
	if (!bSuccess)
	{
		return;
	}
}

void FTestLastResortPropertyNetSerializer::TestIsEqual()
{
	TArray<FText> CompareValues[2];
	TArray<bool> ExpectedResults[2];

	CompareValues[0] = TArray<FText>(TestValues, TestValueCount);
	ExpectedResults[0].Init(true, TestValueCount);

	CompareValues[1].Reserve(TestValueCount);
	ExpectedResults[1].Reserve(TestValueCount);
	for (int32 ValueIt = 0, ValueEndIt = TestValueCount; ValueIt != ValueEndIt; ++ValueIt)
	{
		CompareValues[1].Add(TestValues[(ValueIt + 1) % ValueEndIt]);
		ExpectedResults[1].Add(TestValues[ValueIt].EqualTo(TestValues[(ValueIt + 1) % ValueEndIt], ETextComparisonLevel::Quinary));
	}

	// Do two rounds of testing per config, one where we compare each value with itself and one where we compare against a value in range.
	for (SIZE_T TestRoundIt : {0, 1})
	{
		// Do both quantized and regular compares
		for (SIZE_T CompareIt : {0, 1})
		{
			bool bQuantizedCompare = CompareIt == 0;
			const bool bSuccess = Super::TestIsEqual(TestValues, CompareValues[TestRoundIt].GetData(), ExpectedResults[TestRoundIt].GetData(), TestValueCount, *SerializerConfig, bQuantizedCompare);
			if (!bSuccess)
			{
				return;
			}
		}
	}
}

void FTestLastResortPropertyNetSerializer::TestSerialize()
{
	constexpr bool bQuantizedCompare = false;
	const bool bSuccess = Super::TestSerialize(TestValues, TestValues, TestValueCount, *SerializerConfig, bQuantizedCompare);
	if (!bSuccess)
	{
		return;
	}
}

void FTestLastResortPropertyNetSerializer::TestCloneDynamicState()
{
	const bool bSuccess = Super::TestCloneDynamicState(TestValues, TestValueCount, *SerializerConfig);
	if (!bSuccess)
	{
		return;
	}
}

}
