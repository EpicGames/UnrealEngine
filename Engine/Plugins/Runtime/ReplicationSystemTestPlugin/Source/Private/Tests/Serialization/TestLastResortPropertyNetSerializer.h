// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/ObjectMacros.h"
#include "Internationalization/Text.h"
#include "TestLastResortPropertyNetSerializer.generated.h"

USTRUCT()
struct FStructForLastResortPropertyNetSerializerTest
{
	GENERATED_BODY()

	UPROPERTY()
	FText Text;
};
