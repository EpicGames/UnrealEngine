// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNameNetSerializer.h"
#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/StringNetSerializers.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Containers/StringConv.h"
#include "Net/UnrealNetwork.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Tests/ReplicationSystem/RPC/RPCTestFixture.h"

namespace UE::Net::Private
{

static FTestMessage& PrintNameNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

class FTestNameNetSerializer : public TTestNetSerializerFixture<PrintNameNetSerializerConfig, FName>
{
public:
	FTestNameNetSerializer() : Super(UE_NET_GET_SERIALIZER(FNameNetSerializer)) {}

	void TestValidate();
	void TestQuantize();
	void TestIsEqual();
	void TestSerialize();
	void TestCloneDynamicState();

	static const FName TestNames[];
	static const SIZE_T TestNameCount;

protected:
	typedef TTestNetSerializerFixture<PrintNameNetSerializerConfig, FName> Super;

	// Serializer
	static FNameNetSerializerConfig SerializerConfig;
};

const FName FTestNameNetSerializer::TestNames[] = 
{
	// Various types of "empty" names
	FName(),
	FName(NAME_None),
	FName(""),
	// EName string
	FName(NAME_Actor),
	// EName with number
	FName(NAME_Actor, 2),
	// Pure ASCII string
	FName("Just a regular ASCII string", FNAME_Add),
	// Copy of above string, but unique!
	FName("Just a regular ASCII string", FNAME_Add),
	// Smiling face with open mouth and tightly-closed eyes, four of circles, euro, copyright
	FName(StringCast<WIDECHAR>(FUTF8ToTCHAR("\xf0\x9f\x98\x86\xf0\x9f\x80\x9c\xe2\x82\xac\xc2\xa9").Get()).Get()),
};

const SIZE_T FTestNameNetSerializer::TestNameCount = sizeof(TestNames)/sizeof(TestNames[0]);

FNameNetSerializerConfig FTestNameNetSerializer::SerializerConfig;

// does not work as we need to setup export system as well.
UE_NET_TEST_FIXTURE(FTestNameNetSerializer, TestValidate)
{
	TestValidate();
}

UE_NET_TEST_FIXTURE(FTestNameNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestNameNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestNameNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestNameNetSerializer, TestCloneDynamicState)
{
	TestCloneDynamicState();
}

void FTestNameNetSerializer::TestValidate()
{
	{
		TArray<bool> ExpectedResults;
		ExpectedResults.Init(true, TestNameCount);

		const bool bSuccess = Super::TestValidate(TestNames, ExpectedResults.GetData(), TestNameCount, SerializerConfig);
		if (!bSuccess)
		{
			return;
		}
	}
}

void FTestNameNetSerializer::TestQuantize()
{
	const bool bSuccess = Super::TestQuantize(TestNames, TestNameCount, SerializerConfig);
	if (!bSuccess)
	{
		return;
	}
}

void FTestNameNetSerializer::TestIsEqual()
{
	TArray<FName> CompareValues[2];
	TArray<bool> ExpectedResults[2];

	CompareValues[0] = TArray<FName>(TestNames, TestNameCount);
	ExpectedResults[0].Init(true, TestNameCount);

	CompareValues[1].Reserve(TestNameCount);
	ExpectedResults[1].Reserve(TestNameCount);
	for (int32 ValueIt = 0, ValueEndIt = TestNameCount; ValueIt != ValueEndIt; ++ValueIt)
	{
		CompareValues[1].Add(TestNames[(ValueIt + 1) % ValueEndIt]);
		ExpectedResults[1].Add(TestNames[ValueIt].IsEqual(TestNames[(ValueIt + 1) % ValueEndIt], ENameCase::IgnoreCase, true));
	}

	// Do two rounds of testing per config, one where we compare each value with itself and one where we compare against a value in range.
	for (SIZE_T TestRoundIt : {0, 1})
	{
		// Do both quantized and regular compares
		for (bool bQuantizedCompare : {false, true})
		{
			const bool bSuccess = Super::TestIsEqual(TestNames, CompareValues[TestRoundIt].GetData(), ExpectedResults[TestRoundIt].GetData(), TestNameCount, SerializerConfig, bQuantizedCompare);
			if (!bSuccess)
			{
				return;
			}
		}
	}
}

void FTestNameNetSerializer::TestSerialize()
{
	constexpr bool bQuantizedCompare = false;
	const bool bSuccess = Super::TestSerialize(TestNames, TestNames, TestNameCount, SerializerConfig, bQuantizedCompare);
	if (!bSuccess)
	{
		return;
	}
}

void FTestNameNetSerializer::TestCloneDynamicState()
{
	const bool bSuccess = Super::TestCloneDynamicState(TestNames, TestNameCount, SerializerConfig);
	if (!bSuccess)
	{
		return;
	}
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestFName)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestNameNetSerializer_TestObject* ServerObject = Server->CreateObject<UTestNameNetSerializer_TestObject>();

	Server->UpdateAndSend({Client});

	// Verify that object has been spawned on client
	UTestNameNetSerializer_TestObject* ClientObject = Client->GetObjectAs<UTestNameNetSerializer_TestObject>(ServerObject->NetRefHandle);

	// Verify that created server handle now also exists on client
	UE_NET_ASSERT_NE(ClientObject, nullptr);	

	// Verify that created server handle now also exists on client
	UE_NET_ASSERT_EQ(ServerObject->NameProperty, ClientObject->NameProperty);

	FName Modified("ModifiedName");

	ServerObject->NameProperty = Modified;

	Server->UpdateAndSend({Client});

	// Verify that we managed to replicate the expected name
	UE_NET_ASSERT_EQ(ServerObject->NameProperty, ClientObject->NameProperty);	
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestFName_ReplicateCommonNames)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestNameNetSerializer_TestObject* ServerObject = Server->CreateObject<UTestNameNetSerializer_TestObject>();

	Server->UpdateAndSend({Client});

	// Verify that object has been spawned on client
	UTestNameNetSerializer_TestObject* ClientObject = Client->GetObjectAs<UTestNameNetSerializer_TestObject>(ServerObject->NetRefHandle);

	// Verify that created server handle now also exists on client
	UE_NET_ASSERT_NE(ClientObject, nullptr);	

	// Send all testnames
	for (const FName& Name : MakeArrayView<const FName>(FTestNameNetSerializer::TestNames, FTestNameNetSerializer::TestNameCount))
	{
		ServerObject->NameArrayProperty.Add(Name);
	}

	// Send a bunch of packets
	for (uint32 Packets = 10; Packets > 0; Packets--)
	{
		Server->UpdateAndSend({Client});
	}

	UE_NET_ASSERT_TRUE(ServerObject->NameArrayProperty == ClientObject->NameArrayProperty);
}


UE_NET_TEST_FIXTURE(FRPCTestFixture, TestFNameClientRPCCanExportsName)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestNameNetSerializer_TestObjectWithRPC* ServerObject = Server->CreateObject<UTestNameNetSerializer_TestObjectWithRPC>();

	ServerObject->bIsServerObject = true;
	ServerObject->ReplicationSystem = Server->GetReplicationSystem();
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, 0x01);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	UTestNameNetSerializer_TestObjectWithRPC* ClientObject = Cast<UTestNameNetSerializer_TestObjectWithRPC>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		
	// Verify that created server handle now also exists on client
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	FName ExpectedName("ExpectedClientName");
	ServerObject->ClientRPCWithName(ExpectedName);
	
	// Send and deliver packet
	Server->UpdateAndSend({Client});

	// Verify that created server handle now also exists on client
	UE_NET_ASSERT_EQ(ExpectedName, ClientObject->NameFromClientRPC);
}

UE_NET_TEST_FIXTURE(FRPCTestFixture, TestFNameServerRPCCanExportsName)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestNameNetSerializer_TestObjectWithRPC* ServerObject = Server->CreateObject<UTestNameNetSerializer_TestObjectWithRPC>();

	ServerObject->bIsServerObject = true;
	ServerObject->ReplicationSystem = Server->GetReplicationSystem();
	Server->ReplicationSystem->SetOwningNetConnection(ServerObject->NetRefHandle, 0x01);

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	UTestNameNetSerializer_TestObjectWithRPC* ClientObject = Cast<UTestNameNetSerializer_TestObjectWithRPC>(Client->GetReplicationBridge()->GetReplicatedObject(ServerObject->NetRefHandle));
		
	// Verify that created server handle now also exists on client
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	ClientObject->ReplicationSystem = Client->GetReplicationSystem();

	FName ExpectedName("ExpectedClientName");
	ClientObject->ServerRPCWithName(ExpectedName);

	// Send and deliver client packet
	Client->UpdateAndSend(Server);
	
	// Verify that created server handle now also exists on client
	UE_NET_ASSERT_EQ(ExpectedName, ServerObject->NameFromServerRPC);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestFText)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTextProperty_TestObject* ServerObject = Server->CreateObject<UTextProperty_TestObject>();

	Server->UpdateAndSend({Client});

	// Verify that object has been spawned on client
	UTextProperty_TestObject* ClientObject = Client->GetObjectAs<UTextProperty_TestObject>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject, nullptr);	

	// Verify that we managed to replicate the expected name
	UE_NET_ASSERT_TRUE(ServerObject->TextProperty.EqualTo(ClientObject->TextProperty));

	FText Modified = FText::FromString(TEXT("ModifiedText"));

	ServerObject->TextProperty = Modified;

	Server->UpdateAndSend({Client});

	// Verify that we managed to replicate the expected name
	UE_NET_ASSERT_TRUE(ServerObject->TextProperty.EqualTo(ClientObject->TextProperty));	
}


}

UTestNameNetSerializer_TestObject::UTestNameNetSerializer_TestObject()
: UReplicatedTestObject()
{
}

void UTestNameNetSerializer_TestObject::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = false;

	DOREPLIFETIME_WITH_PARAMS_FAST(UTestNameNetSerializer_TestObject, NameProperty, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UTestNameNetSerializer_TestObject, NameArrayProperty, Params);
}

void UTestNameNetSerializer_TestObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

//////////////////////////////////////////////////////////////////////////
// Implementation for UTestNameNetSerializer_TestObjectWithRPC
//////////////////////////////////////////////////////////////////////////
UTestNameNetSerializer_TestObjectWithRPC::UTestNameNetSerializer_TestObjectWithRPC()
	: UReplicatedTestObject()
{
}

void UTestNameNetSerializer_TestObjectWithRPC::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	this->ReplicationFragments.Reset();
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags, &this->ReplicationFragments);
}

void UTestNameNetSerializer_TestObjectWithRPC::Init(UReplicationSystem* InRepSystem)
{
	bIsServerObject = InRepSystem->IsServer();
	ReplicationSystem = InRepSystem;
}

void UTestNameNetSerializer_TestObjectWithRPC::SetRootObject(UTestNameNetSerializer_TestObjectWithRPC* InRootObject)
{
	check(InRootObject);
	RootObject = InRootObject;
}

int32 UTestNameNetSerializer_TestObjectWithRPC::GetFunctionCallspace(UFunction* Function, FFrame* Stack)
{
	check(!(Function->FunctionFlags & FUNC_Static));
	check(Function->FunctionFlags & FUNC_Net);

	const bool bIsOnServer = bIsServerObject;

	// get the top most function
	while (Function->GetSuperFunction() != nullptr)
	{
		Function = Function->GetSuperFunction();
	}

	// Multicast RPCs
	if ((Function->FunctionFlags & FUNC_NetMulticast))
	{
		if (bIsOnServer)
		{
			// Server should execute locally and call remotely
			return (FunctionCallspace::Local | FunctionCallspace::Remote);
		}
		else
		{
			return FunctionCallspace::Local;
		}
	}

	// if we are the authority
	if (bIsOnServer)
	{
		if (Function->FunctionFlags & FUNC_NetClient)
		{
			return FunctionCallspace::Remote;
		}
		else
		{
			return FunctionCallspace::Local;
		}

	}
	// if we are not the authority
	else
	{
		if (Function->FunctionFlags & FUNC_NetServer)
		{
			return FunctionCallspace::Remote;
		}
		else
		{
			// don't replicate
			return FunctionCallspace::Local;
		}
	}
}

bool UTestNameNetSerializer_TestObjectWithRPC::CallRemoteFunction(UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack)
{
	if (bIsSubObject)
	{
		return ReplicationSystem->SendRPC(RootObject, this, Function, Parameters);
	}
	else
	{
		return ReplicationSystem->SendRPC(this, nullptr, Function, Parameters);
	}
}

void UTestNameNetSerializer_TestObjectWithRPC::ClientRPCWithName_Implementation(FName Name)
{
	NameFromClientRPC = Name;
}


void UTestNameNetSerializer_TestObjectWithRPC::ServerRPCWithName_Implementation(FName Name)
{
	NameFromServerRPC = Name;
}

//////////////////////////////////////////////////////////////////////////
// Implementation for UTextProperty_TestObject
//////////////////////////////////////////////////////////////////////////
UTextProperty_TestObject::UTextProperty_TestObject()
: UReplicatedTestObject()
{
	TextProperty = FText::FromString("DefaultText");
}

void UTextProperty_TestObject::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = false;

	DOREPLIFETIME_WITH_PARAMS_FAST(UTextProperty_TestObject, TextProperty, Params);
}

void UTextProperty_TestObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}



