// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "TestNameNetSerializer.generated.h"

UCLASS()
class UTestNameNetSerializer_TestObject : public UReplicatedTestObject
{
	GENERATED_BODY()
public:
	UTestNameNetSerializer_TestObject();

	void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	UPROPERTY(Replicated)
	FName NameProperty = FName("Initial");

	UPROPERTY(Replicated)
	TArray<FName> NameArrayProperty;
};

UCLASS()
class UTestNameNetSerializer_TestObjectWithRPC : public UReplicatedTestObject
{
	GENERATED_BODY()

public:
	UTestNameNetSerializer_TestObjectWithRPC();

	void Init(UReplicationSystem* InRepSystem);
	void SetRootObject(UTestNameNetSerializer_TestObjectWithRPC* InRootObject);

	virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	virtual int32 GetFunctionCallspace(UFunction* Function, FFrame* Stack) override;
	virtual bool CallRemoteFunction(UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack) override;

	UReplicationSystem* ReplicationSystem = nullptr;

	// Network data only for test
	TArray<UE::Net::FReplicationFragment*> ReplicationFragments;

	// To determine if this object is located on the server or client
	bool bIsServerObject = false;

	// Our owner when the object is a subobject
	UTestNameNetSerializer_TestObjectWithRPC* RootObject = nullptr;

	// RPC test functions
	UFUNCTION(Reliable, Client)
	void ClientRPCWithName(FName Name);
	FName NameFromClientRPC;

	UFUNCTION(Reliable, Server)
	void ServerRPCWithName(FName Name);
	FName NameFromServerRPC;
};

UCLASS()
class UTextProperty_TestObject : public UReplicatedTestObject
{
	GENERATED_BODY()
public:
	UTextProperty_TestObject();

	void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	UPROPERTY(Replicated)
	FText TextProperty;
};
