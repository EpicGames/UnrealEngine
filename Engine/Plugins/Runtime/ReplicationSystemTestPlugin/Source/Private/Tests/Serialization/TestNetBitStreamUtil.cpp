// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Containers/StringConv.h"
#include "CoreGlobals.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Logging/LogScopedVerbosityOverride.h"

namespace UE::Net::Private
{

class FNetBitStreamUtilTest : public FNetworkAutomationTestSuiteFixture
{
protected:
	enum : unsigned
	{
		BitStreamBufferSize = 1024,
	};

	FNetBitStreamReader Reader;
	FNetBitStreamWriter Writer;
	uint32 BitStreamBuffer[BitStreamBufferSize];
};

//const char* EmptyString = "";
//const char* ANSIString = "Just a regular ANSI string";
//const char* UTF8EncodedString = ;

UE_NET_TEST_FIXTURE(FNetBitStreamUtilTest, TestEmptyString)
{
	const FString EmptyString("");

	Writer.InitBytes(BitStreamBuffer, BitStreamBufferSize);

	WriteString(&Writer, EmptyString);
	Writer.CommitWrites();
	UE_NET_ASSERT_FALSE(Writer.IsOverflown());
	UE_NET_ASSERT_GT(Writer.GetPosBits(), 0U);

	FString String;
	Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());
	ReadString(&Reader, String);
	UE_NET_ASSERT_FALSE(Reader.IsOverflown());
	UE_NET_ASSERT_EQ(String, EmptyString);
}

UE_NET_TEST_FIXTURE(FNetBitStreamUtilTest, TestANSIString)
{
	const FString ANSIString(TEXT("An ANSI string"));

	Writer.InitBytes(BitStreamBuffer, BitStreamBufferSize);

	WriteString(&Writer, ANSIString);
	Writer.CommitWrites();
	UE_NET_ASSERT_FALSE(Writer.IsOverflown());
	UE_NET_ASSERT_GT(Writer.GetPosBits(), 0U);

	FString String;
	Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());
	ReadString(&Reader, String);
	UE_NET_ASSERT_FALSE(Reader.IsOverflown());
	UE_NET_ASSERT_EQ(String, ANSIString);
}

UE_NET_TEST_FIXTURE(FNetBitStreamUtilTest, TestWideString)
{
	const FString WideString(UTF8_TO_TCHAR("\xf0\x9f\x98\x86\xf0\x9f\x80\x9c\xe2\x82\xac\xc2\xa9"));

	Writer.InitBytes(BitStreamBuffer, BitStreamBufferSize);

	WriteString(&Writer, WideString);
	Writer.CommitWrites();
	UE_NET_ASSERT_FALSE(Writer.IsOverflown());
	UE_NET_ASSERT_GT(Writer.GetPosBits(), 0U);

	FString String;
	Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());
	ReadString(&Reader, String);
	UE_NET_ASSERT_FALSE(Reader.IsOverflown());
	UE_NET_ASSERT_EQ(String, WideString);
}

UE_NET_TEST_FIXTURE(FNetBitStreamUtilTest, TestTooLongStringWritesEmptyString)
{
	constexpr int32 VeryLongStringLength = 77777;

	FString VeryLongString;
	VeryLongString.Appendf(TEXT("%*c"), VeryLongStringLength, 'y');
	UE_NET_ASSERT_EQ(VeryLongString.Len(), VeryLongStringLength);

	TArray<uint8> VeryLargeBuffer;
	VeryLargeBuffer.SetNumUninitialized(Align(VeryLongStringLength + 1024, 4));
	Writer.InitBytes(VeryLargeBuffer.GetData(), VeryLargeBuffer.Num());
	
	// Suppress Iris internal error, since we're intentionally causing one.
	{
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogSerialization, ELogVerbosity::Fatal);
		WriteString(&Writer, VeryLongString);
	}

	Writer.CommitWrites();
	UE_NET_ASSERT_FALSE(Writer.IsOverflown());
	UE_NET_ASSERT_GT(Writer.GetPosBits(), 0U);

	FString String;
	Reader.InitBits(VeryLargeBuffer.GetData(), Writer.GetPosBits());
	ReadString(&Reader, String);
	UE_NET_ASSERT_FALSE(Reader.IsOverflown());
	UE_NET_ASSERT_EQ(String, FString());

	// Reset Writer to avoid write after free.
	Writer = FNetBitStreamWriter();
}

UE_NET_TEST_FIXTURE(FNetBitStreamUtilTest, TestReadWriteBytes)
{
	alignas(16) const char SrcBuffer[] = "012345679";

	// Test write from offset
	{
		Writer.InitBytes(BitStreamBuffer, BitStreamBufferSize);

		const char* SrcBytes = SrcBuffer + 1;
		const uint32 ByteCount = 3U;

		WriteBytes(&Writer, reinterpret_cast<const uint8*>(SrcBytes), ByteCount);
		Writer.CommitWrites();

		UE_NET_ASSERT_FALSE(Writer.IsOverflown());
		UE_NET_ASSERT_GT(Writer.GetPosBits(), 0U);

		char DstBytes[ByteCount];

		Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());
		ReadBytes(&Reader, reinterpret_cast<uint8*>(DstBytes), ByteCount);
		UE_NET_ASSERT_FALSE(Reader.IsOverflown());
		UE_NET_ASSERT_EQ(FCStringAnsi::Strncmp(SrcBytes, DstBytes, ByteCount), 0);
	}

	// Test write entire buffer
	{
		Writer.InitBytes(BitStreamBuffer, BitStreamBufferSize);

		const char* SrcBytes = SrcBuffer;
		const uint32 ByteCount = 10;

		WriteBytes(&Writer, reinterpret_cast<const uint8*>(SrcBytes), ByteCount);
		Writer.CommitWrites();

		UE_NET_ASSERT_FALSE(Writer.IsOverflown());
		UE_NET_ASSERT_GT(Writer.GetPosBits(), 0U);

		char DstBytes[ByteCount];

		Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());
		ReadBytes(&Reader, reinterpret_cast<uint8*>(DstBytes), ByteCount);
		UE_NET_ASSERT_FALSE(Reader.IsOverflown());
		UE_NET_ASSERT_EQ(FCStringAnsi::Strncmp(SrcBytes, DstBytes, ByteCount), 0);
	}

	// Test write with misaligned start
	{
		Writer.InitBytes(BitStreamBuffer, BitStreamBufferSize);

		const char* SrcBytes = SrcBuffer + 3;
		const uint32 ByteCount = 6;

		WriteBytes(&Writer, reinterpret_cast<const uint8*>(SrcBytes), ByteCount);
		Writer.CommitWrites();

		UE_NET_ASSERT_FALSE(Writer.IsOverflown());
		UE_NET_ASSERT_GT(Writer.GetPosBits(), 0U);

		char DstBytes[ByteCount];

		Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());
		ReadBytes(&Reader, reinterpret_cast<uint8*>(DstBytes), ByteCount);
		UE_NET_ASSERT_FALSE(Reader.IsOverflown());
		UE_NET_ASSERT_EQ(FCStringAnsi::Strncmp(SrcBytes, DstBytes, ByteCount), 0);
	}

	// Test read to offset
	{
		Writer.InitBytes(BitStreamBuffer, BitStreamBufferSize);

		const char* SrcBytes = SrcBuffer;
		const uint32 ByteCount = 10;
		const uint32 DstOffset = 3;

		WriteBytes(&Writer, reinterpret_cast<const uint8*>(SrcBytes), ByteCount);
		Writer.CommitWrites();

		UE_NET_ASSERT_FALSE(Writer.IsOverflown());
		UE_NET_ASSERT_GT(Writer.GetPosBits(), 0U);

		alignas(16) char DstBuffer[16] = {};

		char* DstBytes = DstBuffer + DstOffset;

		Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());
		ReadBytes(&Reader, reinterpret_cast<uint8*>(DstBytes), ByteCount);

		UE_NET_ASSERT_FALSE(Reader.IsOverflown());
		UE_NET_ASSERT_EQ(FCStringAnsi::Strncmp(SrcBytes, DstBytes, ByteCount), 0);
		// Verify byte before and after
		UE_NET_ASSERT_EQ(DstBuffer[DstOffset - 1], (char)0);
		UE_NET_ASSERT_EQ(DstBuffer[DstOffset + ByteCount], (char)0);
	}
}


}
