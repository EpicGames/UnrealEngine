// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Net/Core/Trace/NetTrace.h"

namespace UE::Net::Private
{

class FNetBitStreamTestWriteBuffer
{
public:
	FNetBitStreamTestWriteBuffer()
	{
		Reset();
	}

	inline void* GetBuffer() { return &Buffer[0];  }
	uint32 GetBufferCapacity() const { return sizeof(Buffer); }

private:
	void Reset() {}

	uint32 Buffer[16];
};

class FNetBitStreamWriterTest : public FNetworkAutomationTestSuiteFixture
{
protected:
	FNetBitStreamTestWriteBuffer Buffer;
	FNetBitStreamWriter Writer;
};

class FNetBitStreamReaderTest : public FNetworkAutomationTestSuiteFixture
{
protected:
	FNetBitStreamTestWriteBuffer Buffer;
	FNetBitStreamReader Reader;
};

class FNetBitStreamReaderWriterTest : public FNetworkAutomationTestSuiteFixture
{
protected:
	FNetBitStreamTestWriteBuffer Buffer;
	FNetBitStreamWriter Writer;
	FNetBitStreamReader Reader;
};

class FNetBitStreamWriterSubstreamTest : public FNetBitStreamWriterTest
{
};

class FNetBitStreamReaderSubstreamTest : public FNetBitStreamReaderTest
{
};

UE_NET_TEST_FIXTURE(FNetBitStreamWriterTest, TestInitState)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());

	const uint32 StartPos = Writer.GetPosBits();
	UE_NET_ASSERT_EQ(StartPos, 0U);

	const bool bIsOverflown = Writer.IsOverflown();
	UE_NET_ASSERT_FALSE(bIsOverflown);
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterTest, CanSeek)
{
	const uint32 SeekPositions[] = {0U, 47U, 11U};
	const size_t TestCount = sizeof(SeekPositions) / sizeof(SeekPositions[0]);

	for (size_t TestIt = 0; TestIt != TestCount; ++TestIt)
	{
		const uint32 SeekPosition = SeekPositions[TestIt];

		Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
		Writer.Seek(SeekPosition);
		const uint32 CurrentPosition = Writer.GetPosBits();
		UE_NET_ASSERT_EQ(CurrentPosition, SeekPosition);
	}
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterTest, WriteZeroBitsAtEndDoesNotOverflow)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
	Writer.Seek(Buffer.GetBufferCapacity()*8U);
	Writer.WriteBits(0U, 0U);
	UE_NET_ASSERT_FALSE(Writer.IsOverflown());
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterTest, WriteBitsAtEndCausesOverflow)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
	Writer.Seek(Buffer.GetBufferCapacity()*8U);
	Writer.WriteBits(0U, 1U);
	UE_NET_ASSERT_TRUE(Writer.IsOverflown());
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterTest, SeekToValidPositionAfterOverflowClearsOverflow)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
	Writer.Seek(Buffer.GetBufferCapacity()*8U);
	Writer.WriteBits(0U, 1U);
	UE_NET_EXPECT_TRUE(Writer.IsOverflown());
	Writer.Seek(Buffer.GetBufferCapacity()*8U);
	UE_NET_ASSERT_FALSE(Writer.IsOverflown());
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterTest, InitBytesAfterOverflowClearsOverflow)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
	Writer.Seek(Buffer.GetBufferCapacity() * 8U);
	Writer.WriteBits(0U, 1U);
	UE_NET_EXPECT_TRUE(Writer.IsOverflown());
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
	UE_NET_ASSERT_FALSE(Writer.IsOverflown());
}

// FNetBitStreamReader tests
UE_NET_TEST_FIXTURE(FNetBitStreamReaderTest, TestInitState)
{
	Reader.InitBits(Buffer.GetBuffer(), Buffer.GetBufferCapacity()*8);

	const uint32 StartPos = Reader.GetPosBits();
	UE_NET_ASSERT_EQ(StartPos, 0U);

	const bool bIsOverflown = Reader.IsOverflown();
	UE_NET_ASSERT_FALSE(bIsOverflown);
}

UE_NET_TEST_FIXTURE(FNetBitStreamReaderTest, CanSeek)
{
	const uint32 SeekPositions[] = { 0U, 75U, 12U };
	const size_t TestCount = sizeof(SeekPositions)/sizeof(SeekPositions[0]);

	for (size_t TestIt = 0; TestIt != TestCount; ++TestIt)
	{
		const uint32 SeekPosition = SeekPositions[TestIt];

		Reader.InitBits(Buffer.GetBuffer(), Buffer.GetBufferCapacity()*8);
		Reader.Seek(SeekPosition);
		const uint32 CurrentPosition = Reader.GetPosBits();
		UE_NET_ASSERT_EQ(CurrentPosition, SeekPosition);
	}
}

UE_NET_TEST_FIXTURE(FNetBitStreamReaderTest, ReadZeroBitsAtEndDoesNotOverflow)
{
	Reader.InitBits(Buffer.GetBuffer(), Buffer.GetBufferCapacity()*8U);
	Reader.Seek(Buffer.GetBufferCapacity()*8U);
	Reader.ReadBits(0U);
	UE_NET_ASSERT_FALSE(Reader.IsOverflown());
}

UE_NET_TEST_FIXTURE(FNetBitStreamReaderTest, ReadBitsAtEndCausesOverflow)
{
	Reader.InitBits(Buffer.GetBuffer(), Buffer.GetBufferCapacity()*8U);
	Reader.Seek(Buffer.GetBufferCapacity()*8U);
	Reader.ReadBits(1U);
	UE_NET_ASSERT_TRUE(Reader.IsOverflown());
}

UE_NET_TEST_FIXTURE(FNetBitStreamReaderTest, SeekToValidPositionAfterOverflowClearsOverflow)
{
	Reader.InitBits(Buffer.GetBuffer(), Buffer.GetBufferCapacity()*8U);
	Reader.Seek(Buffer.GetBufferCapacity()*8U);
	Reader.ReadBits(1U);
	UE_NET_EXPECT_TRUE(Reader.IsOverflown());
	Reader.Seek(Buffer.GetBufferCapacity()*8U);
	UE_NET_ASSERT_FALSE(Reader.IsOverflown());
}


// Combined reader/writer tests
UE_NET_TEST_FIXTURE(FNetBitStreamReaderWriterTest, WriteBitsAtOffset0)
{
	const uint32 Sentinel = 0xC001C0DE;
	for (uint32 BitCount = 0; BitCount <= 32; ++BitCount)
	{
		Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
		Writer.WriteBits(~0U, BitCount);
		Writer.WriteBits(Sentinel, sizeof(Sentinel)*8U);
		Writer.CommitWrites();

		Reader.InitBits(Buffer.GetBuffer(), Writer.GetPosBits());

		const uint32 ReadValue = Reader.ReadBits(BitCount);
		const uint32 ReadSentinel = Reader.ReadBits(sizeof(Sentinel)*8);

		const uint32 ExpectedValue = uint32((uint64(1) << BitCount) - uint64(1));

		UE_NET_ASSERT_EQ_MSG(ReadValue, ExpectedValue, FString::Printf(TEXT("Failed testing with %u bits"), BitCount));
		UE_NET_ASSERT_EQ_MSG(ReadSentinel, Sentinel, FString::Printf(TEXT("Failed testing with %u bits"), BitCount));
	}
}

UE_NET_TEST_FIXTURE(FNetBitStreamReaderWriterTest, WriteBitsAtArbitraryOffsets)
{
	const uint32 Sentinel = 0xC001C0DE;
	for (uint32 BitOffset = 32; BitOffset <= 64; ++BitOffset)
	{
		for (uint32 BitCount = 0; BitCount <= 32; ++BitCount)
		{
			Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
			Writer.Seek(BitOffset);
			Writer.WriteBits(~0U, BitCount);
			Writer.WriteBits(Sentinel, sizeof(Sentinel)*8U);
			Writer.CommitWrites();

			Reader.InitBits(Buffer.GetBuffer(), Writer.GetPosBits());
			Reader.Seek(BitOffset);

			const uint32 ReadValue = Reader.ReadBits(BitCount);
			const uint32 ReadSentinel = Reader.ReadBits(sizeof(Sentinel)*8);

			const uint32 ExpectedValue = uint32((uint64(1) << BitCount) - uint64(1));

			UE_NET_ASSERT_EQ_MSG(ReadValue, ExpectedValue, FString::Printf(TEXT("Failed testing with %u bits at offset %u"), BitCount, BitOffset));
			UE_NET_ASSERT_EQ_MSG(ReadSentinel, Sentinel, FString::Printf(TEXT("Failed testing with %u bits at offset %u"), BitCount, BitOffset));
		}
	}
}

// Test writing X bits at offset 32 + Y to a stream and then write that stream to a second stream at offset 32 + Z
UE_NET_TEST_FIXTURE(FNetBitStreamReaderWriterTest, WriteStreamWithBitsWrittenAtArbitraryOffsets)
{
	const uint32 ValuesAndBitCount[][2] =
	{
		{1U, 9U},
		{47U, 17U},
		{11U, 32U},
		{777777U, 32U},
		{25500U, 32U},
		{311U, 32U},
		{0xC001C0DE, 32U},
	};
	const uint32 Sentinel = 0xC0DEC0DE;

	for (uint32 BitOffset0 = 32; BitOffset0 <= 64; ++BitOffset0)
	{
		Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
		Writer.Seek(BitOffset0);
		for (size_t ValueIt = 0, ValueEndIt = sizeof(ValuesAndBitCount)/sizeof(ValuesAndBitCount[0]); ValueIt != ValueEndIt; ++ValueIt)
		{
			const uint32 Value = ValuesAndBitCount[ValueIt][0];
			const uint32 BitCount = ValuesAndBitCount[ValueIt][1];

			Writer.WriteBits(Value, BitCount);
		}
		Writer.CommitWrites();

		for (uint32 BitOffset1 = 32; BitOffset1 <= 64; ++BitOffset1)
		{
			FNetBitStreamTestWriteBuffer SecondBuffer;
			FNetBitStreamWriter SecondWriter;
			SecondWriter.InitBytes(SecondBuffer.GetBuffer(), SecondBuffer.GetBufferCapacity());
			SecondWriter.Seek(BitOffset1);
			SecondWriter.WriteBitStream(static_cast<const uint32*>(Buffer.GetBuffer()), BitOffset0, Writer.GetPosBits() - BitOffset0);
			SecondWriter.WriteBits(Sentinel, sizeof(Sentinel)*8U);
			SecondWriter.CommitWrites();

			Reader.InitBits(SecondBuffer.GetBuffer(), SecondWriter.GetPosBits());
			Reader.Seek(BitOffset1);
			for (size_t ValueIt = 0, ValueEndIt = sizeof(ValuesAndBitCount)/sizeof(ValuesAndBitCount[0]); ValueIt != ValueEndIt; ++ValueIt)
			{
				const uint32 ExpectedValue = ValuesAndBitCount[ValueIt][0];
				const uint32 BitCount = ValuesAndBitCount[ValueIt][1];

				const uint32 ReadValue = Reader.ReadBits(BitCount);
				UE_NET_ASSERT_EQ_MSG(ReadValue, ExpectedValue, FString::Printf(TEXT("Write stream with bits written at offset %u to stream at offset %u"), BitOffset0, BitOffset1));
			}
			const uint32 ReadSentinel = Reader.ReadBits(sizeof(Sentinel)*8U);
			UE_NET_ASSERT_EQ_MSG(ReadSentinel, Sentinel, FString::Printf(TEXT("Write stream with bits written at offset %u to stream at offset %u"), BitOffset0, BitOffset1));
		}
	}
}

// Test writing X bits at offset 32 + Y to a stream and then write that stream to a second stream at offset 32 + Z. 
// The resulting is then read from using ReadBitStream.
UE_NET_TEST_FIXTURE(FNetBitStreamReaderWriterTest, ReadStreamWithBitsWrittenAtArbitraryOffsets)
{
	const uint32 ValuesAndBitCount[][2] =
	{
		{1U, 9U},
		{47U, 17U},
		{11U, 32U},
		{777777U, 32U},
		{25500U, 32U},
		{311U, 32U},
		{0xC001C0DE, 32U},
	};
	const uint32 Sentinel = 0xC0DEC0DE;

	uint32 TotalBitCount = 0;
	for (size_t ValueIt = 0, ValueEndIt = sizeof(ValuesAndBitCount)/sizeof(ValuesAndBitCount[0]); ValueIt != ValueEndIt; ++ValueIt)
	{
		TotalBitCount += ValuesAndBitCount[ValueIt][1];
	}

	for (uint32 BitOffset0 = 32; BitOffset0 <= 64; ++BitOffset0)
	{
		Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
		Writer.Seek(BitOffset0);
		for (size_t ValueIt = 0, ValueEndIt = sizeof(ValuesAndBitCount)/sizeof(ValuesAndBitCount[0]); ValueIt != ValueEndIt; ++ValueIt)
		{
			const uint32 Value = ValuesAndBitCount[ValueIt][0];
			const uint32 BitCount = ValuesAndBitCount[ValueIt][1];

			Writer.WriteBits(Value, BitCount);
		}
		Writer.CommitWrites();

		for (uint32 BitOffset1 = 32; BitOffset1 <= 64; ++BitOffset1)
		{
			FNetBitStreamTestWriteBuffer SecondBuffer;
			FNetBitStreamWriter SecondWriter;
			SecondWriter.InitBytes(SecondBuffer.GetBuffer(), SecondBuffer.GetBufferCapacity());
			SecondWriter.Seek(BitOffset1);
			SecondWriter.WriteBitStream(static_cast<const uint32*>(Buffer.GetBuffer()), BitOffset0, Writer.GetPosBits() - BitOffset0);
			SecondWriter.WriteBits(Sentinel, sizeof(Sentinel)*8U);
			SecondWriter.CommitWrites();

			Reader.InitBits(SecondBuffer.GetBuffer(), SecondWriter.GetPosBits());
			Reader.Seek(BitOffset1);
			uint32 ResultBuffer[sizeof(ValuesAndBitCount)/8];
			Reader.ReadBitStream(ResultBuffer, TotalBitCount);

			const uint32 ReadSentinel = Reader.ReadBits(sizeof(Sentinel)*8U);
			UE_NET_ASSERT_EQ_MSG(ReadSentinel, Sentinel, FString::Printf(TEXT("Write stream with bits written at offset %u to stream at offset %u"), BitOffset0, BitOffset1));

			Reader.InitBits(ResultBuffer, TotalBitCount);
			for (size_t ValueIt = 0, ValueEndIt = sizeof(ValuesAndBitCount)/sizeof(ValuesAndBitCount[0]); ValueIt != ValueEndIt; ++ValueIt)
			{
				const uint32 ExpectedValue = ValuesAndBitCount[ValueIt][0];
				const uint32 BitCount = ValuesAndBitCount[ValueIt][1];

				const uint32 ReadValue = Reader.ReadBits(BitCount);
				UE_NET_ASSERT_EQ_MSG(ReadValue, ExpectedValue, FString::Printf(TEXT("Write stream with bits written at offset %u to stream at offset %u"), BitOffset0, BitOffset1));
			}
		}
	}
}

// Writer substream tests
UE_NET_TEST_FIXTURE(FNetBitStreamWriterSubstreamTest, CanCreateSubstream)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
	Writer.Seek(1U);

	FNetBitStreamWriter Substream = Writer.CreateSubstream(~0U);

	UE_NET_ASSERT_FALSE(Substream.IsOverflown());
	UE_NET_ASSERT_EQ(Substream.GetPosBits(), 0U);
	UE_NET_ASSERT_EQ(Substream.GetBitsLeft(), Writer.GetBitsLeft());

	Writer.DiscardSubstream(Substream);
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterSubstreamTest, CanCreateSmallSubstream)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
	Writer.Seek(1U);

	FNetBitStreamWriter Substream = Writer.CreateSubstream(3U);

	UE_NET_ASSERT_FALSE(Substream.IsOverflown());
	UE_NET_ASSERT_EQ(Substream.GetPosBits(), 0U);
	UE_NET_ASSERT_EQ(Substream.GetBitsLeft(), 3U);

	Writer.DiscardSubstream(Substream);
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterSubstreamTest, CanCreateEmptySubstream)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
	Writer.Seek(1U);

	FNetBitStreamWriter Substream = Writer.CreateSubstream(0U);

	UE_NET_ASSERT_FALSE(Substream.IsOverflown());
	UE_NET_ASSERT_EQ(Substream.GetPosBits(), 0U);
	UE_NET_ASSERT_EQ(Substream.GetBitsLeft(), 0U);

	Writer.DiscardSubstream(Substream);
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterSubstreamTest, CanDiscardSubstream)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());

	{
		FNetBitStreamWriter Substream = Writer.CreateSubstream();
		Substream.WriteBits(~0U, 32U);

		UE_NET_ASSERT_FALSE(Substream.IsOverflown());

		Writer.DiscardSubstream(Substream);
	}

	UE_NET_ASSERT_EQ(Writer.GetPosBits(), 0U);
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterSubstreamTest, CanCommitSubstream)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
	constexpr uint32 SeekPos = 33;
	constexpr uint32 WriteBitCount = 32;

	Writer.Seek(SeekPos);

	{
		FNetBitStreamWriter Substream = Writer.CreateSubstream();
		Substream.WriteBits(~0U, 32U);

		UE_NET_ASSERT_FALSE(Substream.IsOverflown());

		Writer.CommitSubstream(Substream);
	}

	UE_NET_ASSERT_EQ(Writer.GetPosBits(), SeekPos + WriteBitCount);
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterSubstreamTest, CanCreateSubstreamFromOverflowedStream)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
	Writer.Seek(Buffer.GetBufferCapacity()*8 + 1);


	UE_NET_ASSERT_TRUE(Writer.IsOverflown());

	const uint32 PreviousBitPos = Writer.GetPosBits();
	{
		FNetBitStreamWriter Substream = Writer.CreateSubstream();
		UE_NET_ASSERT_TRUE(Substream.IsOverflown());

		Substream.Seek(0U);
		UE_NET_ASSERT_EQ(Substream.GetBitsLeft(), 0U);
		
		Substream.WriteBits(0, 1U);
		UE_NET_ASSERT_TRUE(Substream.IsOverflown());

		Writer.CommitSubstream(Substream);
	}
	const uint32 CurrentBitPos = Writer.GetPosBits();

	UE_NET_ASSERT_EQ(CurrentBitPos, PreviousBitPos);
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterSubstreamTest, CanCreateSubSubstream)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());

	constexpr uint32 SeekCount1 = 33;
	constexpr uint32 WriteCount2 = 32;

	{
		FNetBitStreamWriter Substream1 = Writer.CreateSubstream();
		Substream1.Seek(SeekCount1);

		FNetBitStreamWriter Substream2 = Substream1.CreateSubstream();
		Substream2.WriteBits(~0U, WriteCount2);

		Substream1.CommitSubstream(Substream2);
		Writer.CommitSubstream(Substream1);
	}
	const uint32 CurrentBitPos = Writer.GetPosBits();

	UE_NET_ASSERT_EQ(CurrentBitPos, SeekCount1 + WriteCount2);
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterSubstreamTest, CanWriteToSubSubstream)
{
	uint32 Word = 0;
	Writer.InitBytes(&Word, sizeof(Word));

	Writer.WriteBits(0, 16U);
	{
		constexpr uint32 SubstreamBitCount = 15U;
		FNetBitStreamWriter Substream1 = Writer.CreateSubstream(SubstreamBitCount);
		FNetBitStreamWriter Substream2 = Substream1.CreateSubstream();
		Substream2.WriteBits(~0U, SubstreamBitCount);

		Substream1.CommitSubstream(Substream2);
		Writer.CommitSubstream(Substream1);
		Writer.WriteBits(1U, 1U);
	}

	Writer.CommitWrites();

	UE_NET_ASSERT_EQ(Word, 0xFFFF0000U);
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterSubstreamTest, CanReadDataCommittedFromSubstream)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());

	constexpr uint32 SeekCount1 = 33;
	constexpr uint32 WriteCount2 = 32;
	constexpr uint32 WriteWord = 0x01020304U;

	{
		FNetBitStreamWriter Substream1 = Writer.CreateSubstream();
		Substream1.Seek(SeekCount1);

		FNetBitStreamWriter Substream2 = Substream1.CreateSubstream();
		Substream2.WriteBits(WriteWord, WriteCount2);

		Substream1.CommitSubstream(Substream2);
		Writer.CommitSubstream(Substream1);
	}

	// Read and verify  
	{
		Writer.CommitWrites();
	
		FNetBitStreamReader Reader;
		Reader.InitBits(Buffer.GetBuffer(), Writer.GetPosBits());
		Reader.Seek(SeekCount1);
		const uint32 ReadWord = Reader.ReadBits(WriteCount2);
		const uint32 WordMask = ~0U >> (32 - WriteCount2);
		UE_NET_ASSERT_EQ((ReadWord & WordMask), (WriteWord & WordMask));
	}
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterSubstreamTest, CanWriteToEndOfSubStreamTest1)
{
	constexpr uint32 Sentinel = 0xBAAAAAADU;

	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());

	{
		FNetBitStreamWriter Substream = Writer.CreateSubstream(Writer.GetBitsLeft() - 1U);
		Substream.Seek(Substream.GetBitsLeft() - 32U);
		Substream.WriteBits(Sentinel, 32U);

		Writer.CommitSubstream(Substream);
		Writer.CommitWrites();
	}

	// Test normal reading
	{
		FNetBitStreamReader Reader;
		Reader.InitBits(Buffer.GetBuffer(), Writer.GetPosBits());
		Reader.Seek(Reader.GetBitsLeft() - 32U);

		const uint32 ReadSentinel = Reader.ReadBits(32);
		UE_NET_ASSERT_EQ(ReadSentinel, Sentinel);
	}

	// Test reading via substream
	{
		FNetBitStreamReader Reader;
		Reader.InitBits(Buffer.GetBuffer(), Buffer.GetBufferCapacity()*8U);

		FNetBitStreamReader Substream = Reader.CreateSubstream(Writer.GetPosBits());
		Substream.Seek(Substream.GetBitsLeft() - 32U);

		const uint32 ReadSentinel = Substream.ReadBits(32U);
		UE_NET_ASSERT_EQ(ReadSentinel, Sentinel);
		Reader.CommitSubstream(Substream);
	}
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterSubstreamTest, CanWriteToEndOfSubStreamTest2)
{
	constexpr uint32 Sentinel1 = 0xBAAAAAADU;
	constexpr uint32 Sentinel2 = 0xBAADF00DU;

	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());

	{
		Writer.Seek(Writer.GetBitsLeft() - 32U);
		Writer.WriteBits(Sentinel2, 32U);
		Writer.Seek(0);

		FNetBitStreamWriter Substream = Writer.CreateSubstream(Writer.GetBitsLeft() - 32U);
		Substream.Seek(Substream.GetBitsLeft() - 32U);
		Substream.WriteBits(Sentinel1, 32U);

		Writer.CommitSubstream(Substream);
		Writer.Seek(Buffer.GetBufferCapacity()*8U);
		Writer.CommitWrites();
	}

	// Test normal reading
	{
		FNetBitStreamReader Reader;
		Reader.InitBits(Buffer.GetBuffer(), Writer.GetPosBits());
		Reader.Seek(Reader.GetBitsLeft() - 64U);

		const uint32 ReadSentinel1 = Reader.ReadBits(32U);
		UE_NET_ASSERT_EQ(ReadSentinel1, Sentinel1);

		const uint32 ReadSentinel2 = Reader.ReadBits(32U);
		UE_NET_ASSERT_EQ(ReadSentinel2, Sentinel2);
	}

	// Test reading via substream
	{
		FNetBitStreamReader Reader;
		Reader.InitBits(Buffer.GetBuffer(), Writer.GetPosBits());
		Reader.Seek(Reader.GetBitsLeft() - 64U);

		FNetBitStreamReader Substream1 = Reader.CreateSubstream(32U);
		const uint32 ReadSentinel1 = Substream1.ReadBits(32U);
		UE_NET_ASSERT_EQ(ReadSentinel1, Sentinel1);
		Reader.CommitSubstream(Substream1);

		FNetBitStreamReader Substream2 = Reader.CreateSubstream();
		const uint32 ReadSentinel2 = Substream2.ReadBits(32U);
		UE_NET_ASSERT_EQ(ReadSentinel2, Sentinel2);
		Reader.CommitSubstream(Substream2);
	}
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterSubstreamTest, SubStreamSeek)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
	Writer.WriteBits(0x12345678,32);

	FNetBitStreamWriter SubStream = Writer.CreateSubstream();
	const uint32 SubStreamStartPos = SubStream.GetPosBits();
	SubStream.Seek(0);
	UE_NET_ASSERT_EQ(SubStreamStartPos, SubStream.GetPosBits());

	Writer.CommitSubstream(SubStream);
}

// Reader substream tests
UE_NET_TEST_FIXTURE(FNetBitStreamReaderSubstreamTest, CanCreateSubstream)
{
	Reader.InitBits(Buffer.GetBuffer(), Buffer.GetBufferCapacity()*8);
	Reader.Seek(1U);

	FNetBitStreamReader Substream = Reader.CreateSubstream(~0U);

	UE_NET_ASSERT_FALSE(Substream.IsOverflown());
	UE_NET_ASSERT_EQ(Substream.GetPosBits(), 0U);
	UE_NET_ASSERT_EQ(Substream.GetBitsLeft(), Reader.GetBitsLeft());

	Reader.DiscardSubstream(Substream);
}

UE_NET_TEST_FIXTURE(FNetBitStreamReaderSubstreamTest, CanCreateSmallSubstream)
{
	Reader.InitBits(Buffer.GetBuffer(), Buffer.GetBufferCapacity()*8);
	Reader.Seek(1U);

	FNetBitStreamReader Substream = Reader.CreateSubstream(3U);

	UE_NET_ASSERT_FALSE(Substream.IsOverflown());
	UE_NET_ASSERT_EQ(Substream.GetPosBits(), 0U);
	UE_NET_ASSERT_EQ(Substream.GetBitsLeft(), 3U);

	Reader.DiscardSubstream(Substream);
}

UE_NET_TEST_FIXTURE(FNetBitStreamReaderSubstreamTest, CanCreateEmptySubstream)
{
	Reader.InitBits(Buffer.GetBuffer(), Buffer.GetBufferCapacity()*8);
	Reader.Seek(1U);

	FNetBitStreamReader Substream = Reader.CreateSubstream(0U);

	UE_NET_ASSERT_FALSE(Substream.IsOverflown());
	UE_NET_ASSERT_EQ(Substream.GetPosBits(), 0U);
	UE_NET_ASSERT_EQ(Substream.GetBitsLeft(), 0U);

	Reader.DiscardSubstream(Substream);
}

UE_NET_TEST_FIXTURE(FNetBitStreamReaderSubstreamTest, CanDiscardSubstream)
{
	Reader.InitBits(Buffer.GetBuffer(), Buffer.GetBufferCapacity()*8);
	Reader.Seek(1U);

	const uint32 PreviousBitPos = Reader.GetPosBits();

	{
		FNetBitStreamReader Substream = Reader.CreateSubstream();
		Substream.ReadBits(32U);

		UE_NET_ASSERT_FALSE(Substream.IsOverflown());

		Reader.DiscardSubstream(Substream);
	}

	// A discarded substream should not affect its parent's position.
	UE_NET_ASSERT_EQ(Reader.GetPosBits(), PreviousBitPos);
}

UE_NET_TEST_FIXTURE(FNetBitStreamReaderSubstreamTest, CanCommitSubstream)
{
	Reader.InitBits(Buffer.GetBuffer(), Buffer.GetBufferCapacity()*8);

	constexpr uint32 SeekPos = 33;
	constexpr uint32 ReadBitCount = 32;

	Reader.Seek(SeekPos);

	{
		FNetBitStreamReader Substream = Reader.CreateSubstream();
		Substream.ReadBits(ReadBitCount);

		UE_NET_ASSERT_FALSE(Substream.IsOverflown());

		Reader.CommitSubstream(Substream);
	}

	UE_NET_ASSERT_EQ(Reader.GetPosBits(), SeekPos + ReadBitCount);
}

UE_NET_TEST_FIXTURE(FNetBitStreamReaderSubstreamTest, CanCreateSubstreamFromOverflowedStream)
{
	Reader.InitBits(Buffer.GetBuffer(), Buffer.GetBufferCapacity()*8);
	Reader.Seek(Reader.GetBitsLeft() + 1);

	UE_NET_ASSERT_TRUE(Reader.IsOverflown());

	const uint32 PreviousBitPos = Reader.GetPosBits();
	{
		FNetBitStreamReader Substream = Reader.CreateSubstream();
		UE_NET_ASSERT_TRUE(Substream.IsOverflown());

		Substream.Seek(0U);
		UE_NET_ASSERT_EQ(Substream.GetBitsLeft(), 0U);
		
		Substream.ReadBits(1U);
		UE_NET_ASSERT_TRUE(Substream.IsOverflown());

		// Commit overflown substream. Because of the overflow this should not affects its parent's position.
		Reader.CommitSubstream(Substream);
	}
	const uint32 CurrentBitPos = Reader.GetPosBits();

	UE_NET_ASSERT_EQ(CurrentBitPos, PreviousBitPos);
}

UE_NET_TEST_FIXTURE(FNetBitStreamReaderSubstreamTest, CanCreateSubSubstream)
{
	Reader.InitBits(Buffer.GetBuffer(), Buffer.GetBufferCapacity()*8);

	constexpr uint32 SeekPosSubstream1 = 33;
	constexpr uint32 ReadCountSubstream2 = 32;

	{
		FNetBitStreamReader Substream1 = Reader.CreateSubstream();
		Substream1.Seek(SeekPosSubstream1);

		FNetBitStreamReader Substream2 = Substream1.CreateSubstream();
		Substream2.ReadBits(ReadCountSubstream2);

		Substream1.CommitSubstream(Substream2);
		Reader.CommitSubstream(Substream1);
	}
	const uint32 CurrentBitPos = Reader.GetPosBits();

	UE_NET_ASSERT_EQ(CurrentBitPos, SeekPosSubstream1 + ReadCountSubstream2);
}

UE_NET_TEST_FIXTURE(FNetBitStreamReaderSubstreamTest, CanReadFromSubstream)
{
	const uint32 Word = 0xFFFF0000U;
	Reader.InitBits(&Word, sizeof(Word)*8);
	Reader.Seek(16);

	constexpr uint32 SubstreamBitCount = 15U;
	FNetBitStreamReader Substream = Reader.CreateSubstream(SubstreamBitCount);
	const uint32 SubStreamReadValue = Substream.ReadBits(SubstreamBitCount);
	UE_NET_ASSERT_EQ(SubStreamReadValue, 0b111111111111111U);

	Reader.CommitSubstream(Substream);
	const uint32 StreamReadValue = Reader.ReadBits(1U);
	UE_NET_ASSERT_EQ(StreamReadValue, 0b1U);
}

UE_NET_TEST_FIXTURE(FNetBitStreamReaderSubstreamTest, CanReadFromSubSubstream)
{
	const uint32 Word = 0xFFFF0000U;
	Reader.InitBits(&Word, sizeof(Word)*8);

	const uint32 FirstStreamReadValue = Reader.ReadBits(16U);
	UE_NET_ASSERT_EQ(FirstStreamReadValue, 0U);
	{
		constexpr uint32 SubstreamBitCount = 15U;
		FNetBitStreamReader Substream1 = Reader.CreateSubstream(SubstreamBitCount);
		FNetBitStreamReader Substream2 = Substream1.CreateSubstream();
		const uint32 SubSubStreamReadValue = Substream2.ReadBits(SubstreamBitCount);
		UE_NET_ASSERT_EQ(SubSubStreamReadValue, 0b111111111111111U);

		Substream1.CommitSubstream(Substream2);
		Reader.CommitSubstream(Substream1);
	}
	const uint32 SecondStreamReadValue = Reader.ReadBits(1U);
	UE_NET_ASSERT_EQ(SecondStreamReadValue, 0b1U);
}

UE_NET_TEST_FIXTURE(FNetBitStreamReaderSubstreamTest, SubStreamSeek)
{
	Reader.InitBits(Buffer.GetBuffer(), Buffer.GetBufferCapacity()*8U);
	Reader.ReadBits(32);

	FNetBitStreamReader SubStream = Reader.CreateSubstream();
	const uint32 SubStreamStartPos = SubStream.GetPosBits();
	SubStream.Seek(0);
	UE_NET_ASSERT_EQ(SubStreamStartPos, SubStream.GetPosBits());

	Reader.CommitSubstream(SubStream);
}

// Misc tests
UE_NET_TEST_FIXTURE(FNetBitStreamWriterTest, RollbackToValidPositionAfterOverflow)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());

	{
		FNetBitStreamRollbackScope Rollback(Writer);
		Writer.Seek(Buffer.GetBufferCapacity()*8U);
		Writer.WriteBits(0U, 1U);
		UE_NET_EXPECT_TRUE(Writer.IsOverflown());
	}
	UE_NET_ASSERT_EQ(0U, Writer.GetPosBits());
	UE_NET_ASSERT_FALSE(Writer.IsOverflown());
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterTest, NoRollbackIfNoOverflow)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());

	{
		FNetBitStreamRollbackScope Rollback(Writer);
		Writer.Seek(Buffer.GetBufferCapacity()*8U);
		UE_NET_EXPECT_FALSE(Writer.IsOverflown());
	}
	UE_NET_ASSERT_EQ(Buffer.GetBufferCapacity()*8U, Writer.GetPosBits());
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterTest, WriteScopeCanRewriteBits)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
	
	Writer.WriteBits(0U, 32);
	Writer.WriteBits(0U, 32);

	Writer.WriteBits(0xFFFFFFFF, 32);

	const uint32 ExpectedBitPos = Writer.GetPosBits();

	{
		// Seek back and rewrite first 32 bits
		FNetBitStreamWriteScope WriteScope(Writer, 0U);

		UE_NET_ASSERT_EQ(0U, Writer.GetPosBits());
		Writer.WriteBits(0xDEADBEEF, 32);
	}

	UE_NET_ASSERT_EQ(ExpectedBitPos, Writer.GetPosBits());

	// Read and verify  
	{
		Writer.CommitWrites();
	
		FNetBitStreamReader Reader;
		Reader.InitBits(Buffer.GetBuffer(), Writer.GetPosBits());

		UE_NET_ASSERT_EQ(0xDEADBEEF, Reader.ReadBits(32));
		UE_NET_ASSERT_EQ(0U, Reader.ReadBits(32));
		UE_NET_ASSERT_EQ(0xFFFFFFFF, Reader.ReadBits(32));
	}
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterTest, WriteScopeOverflow)
{
	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
	
	Writer.WriteBits(0U, 32);
	Writer.WriteBits(0U, 32);

	Writer.WriteBits(0xFFFFFFFF, 32);

	const uint32 ExpectedBitPos = Writer.GetPosBits();

	UE_NET_EXPECT_FALSE(Writer.IsOverflown());

	{
		// Bad write scope
		FNetBitStreamWriteScope WriteScope(Writer, Buffer.GetBufferCapacity()*8U + 1);
		UE_NET_EXPECT_TRUE(Writer.IsOverflown());
		Writer.WriteBits(0xDEADBEEF, 32);
	}

	UE_NET_ASSERT_EQ(ExpectedBitPos, Writer.GetPosBits());
	UE_NET_EXPECT_FALSE(Writer.IsOverflown());
}

UE_NET_TEST_FIXTURE(FNetBitStreamReaderWriterTest, WriteBool)
{
	static_assert(sizeof(bool) == sizeof(uint8), "Unexpected size of bool. Need to rewrite tests");

	// We can't trust the compiler not to fiddle with bools and force them to be 0 or 1.
	const uint8 TestValues[] = {0, 1, 128, 255};
	constexpr SIZE_T ValueCount = sizeof(TestValues)/sizeof(TestValues[0]);
	bool Values[ValueCount];

	FPlatformMemory::Memcpy(Values, TestValues, sizeof(TestValues));

	// Validate test values
	{
		const bool* TestValue0 = &Values[0];
		const uint8 ExpectedValue0 = 0;
		UE_NET_ASSERT_EQ(FPlatformMemory::Memcmp(TestValue0, &ExpectedValue0, 1), 0);

		const bool* TestValue1 = &Values[1];
		const uint8 ExpectedValue1 = 1;
		UE_NET_ASSERT_EQ(FPlatformMemory::Memcmp(TestValue1, &ExpectedValue1, 1), 0);

		const bool* TestValue2 = &Values[2];
		const uint8 ExpectedValue2 = 128;
		UE_NET_ASSERT_EQ(FPlatformMemory::Memcmp(TestValue2, &ExpectedValue2, 1), 0);

		const bool* TestValue3 = &Values[3];
		const uint8 ExpectedValue3 = 255;
		UE_NET_ASSERT_EQ(FPlatformMemory::Memcmp(TestValue3, &ExpectedValue3, 1), 0);
	}

	// We expect WriteBool to write a 1 if the bool isn't exactly false (zero) and write a 0 when it's false.
	// We also expect WriteBool to return the value written.
	bool ExpectedValues[sizeof(Values)/sizeof(Values[0])];
	bool ReturnValues[sizeof(Values)/sizeof(Values[0])];
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		uint8 Value;
		FPlatformMemory::Memcpy(&Value, &Values[ValueIt], 1);
		ExpectedValues[ValueIt] = Value != 0;
	}

	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());
	for (const bool& Value : Values)
	{
		const SIZE_T ValueIt = &Value - Values;
		const bool bWroteTrue = Writer.WriteBool(Value);
		ReturnValues[ValueIt] = bWroteTrue;
		// Allow undefined bool values to be written as either false or true as long as the return value matches what's actually written.
		if (TestValues[ValueIt] > 1)
		{
			ExpectedValues[&Value - Values] = bWroteTrue;
			if (!bWroteTrue)
			{
				UE_NET_LOG(FString::Printf(TEXT("Expected WriteBool of non-zero value (%u) to write true."), TestValues[ValueIt]));
			}
		}
	}
	Writer.CommitWrites();

	// Validate return values are as expected
	UE_NET_ASSERT_EQ(FPlatformMemory::Memcmp(ReturnValues, ExpectedValues, sizeof(ReturnValues)), 0);

	Reader.InitBits(Buffer.GetBuffer(), Writer.GetPosBits());
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		const bool bReadTrue = Reader.ReadBool();
		UE_NET_ASSERT_EQ_MSG(bReadTrue, ExpectedValues[ValueIt], FString::Printf(TEXT("WriteBool(%u) returned %u but ReadBool() returned %u."), TestValues[ValueIt], ReturnValues[ValueIt], bReadTrue));
	}
}

#if UE_NET_TRACE_ENABLED
static void WriteDataRecursive(FNetBitStreamWriter& Writer, FNetTraceCollector* Collector, uint32 CurrentDepth)
{
	UE_NET_TRACE_SCOPE(WrittenData, Writer, Collector, ENetTraceVerbosity::Verbose);
	Writer.WriteBits(1U, 1);
	if (CurrentDepth > 0)
	{
		WriteDataRecursive(Writer, Collector, CurrentDepth - 1U);
	}
}

UE_NET_TEST_FIXTURE(FNetBitStreamWriterTest, TraceScopeClampedNestingLevel)
{
	FNetTraceCollector* Collector = UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace);

	Writer.InitBytes(Buffer.GetBuffer(), Buffer.GetBufferCapacity());

	{
		UE_NET_TRACE_SCOPE(Payload, Writer, Collector, ENetTraceVerbosity::Verbose);
		const uint32 TestNestedDepth = FNetTraceCollector::MaxNestingLevel + 5;
	
		// Just verify we do not trigger check/crash.
		WriteDataRecursive(Writer, Collector, TestNestedDepth);
	}

	UE_NET_TRACE_DESTROY_COLLECTOR(Collector);
}
#endif

}
