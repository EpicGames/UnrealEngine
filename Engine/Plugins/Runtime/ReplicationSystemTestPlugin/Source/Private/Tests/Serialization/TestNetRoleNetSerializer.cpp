// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Core/BitTwiddling.h"
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/Serialization/InternalNetSerializers.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Engine/EngineTypes.h"
#include "EnumTestTypes.h"
#include "Net/UnrealNetwork.h"
#include "UObject/StrongObjectPtr.h"

void UClassWithNetRoleSwapping::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{
	DOREPLIFETIME(ThisClass, Role);
	DOREPLIFETIME(ThisClass, JustSomeReplicatedPropertyInBetween);
	DOREPLIFETIME(ThisClass, RemoteRole);
}

namespace UE::Net::Private
{

static FTestMessage& PrintNetRoleNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	const FNetRoleNetSerializerConfig& Config = static_cast<const FNetRoleNetSerializerConfig&>(InConfig);
	return Message << "LowerBound: " << Config.LowerBound << " UpperBound: " << Config.UpperBound << " BitCount: " << Config.BitCount;
}

class FTestNetRoleNetSerializer : public TTestNetSerializerFixture<PrintNetRoleNetSerializerConfig, uint8>
{
	typedef TTestNetSerializerFixture<PrintNetRoleNetSerializerConfig, uint8> Super;

public:
	FTestNetRoleNetSerializer()
	: Super(UE_NET_GET_SERIALIZER(FNetRoleNetSerializer))
	{
		Context.SetInternalContext(&InternalContext);
	}

	void TestIsEqual();
	void TestValidate();
	void TestQuantize();
	void TestSerialize();
	void TestSerializeDelta();
	void TestDequantizeSwapsRoles();
	void TestDowngradeRole();

protected:
	virtual void SetUp() override;

	static TArray<uint8> Values;
	static TArray<uint8> InvalidValues;
	static FNetRoleNetSerializerConfig Config;
	static const UEnum* Enum;
	FInternalNetSerializationContext InternalContext;
};

TArray<uint8> FTestNetRoleNetSerializer::Values;
TArray<uint8> FTestNetRoleNetSerializer::InvalidValues;
FNetRoleNetSerializerConfig FTestNetRoleNetSerializer::Config;
const UEnum* FTestNetRoleNetSerializer::Enum;

UE_NET_TEST_FIXTURE(FTestNetRoleNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestNetRoleNetSerializer, TestValidate)
{
	TestValidate();
}

UE_NET_TEST_FIXTURE(FTestNetRoleNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestNetRoleNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestNetRoleNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestNetRoleNetSerializer, TestDequantizeSwapsRoles)
{
	TestDequantizeSwapsRoles();
}

UE_NET_TEST_FIXTURE(FTestNetRoleNetSerializer, TestDowngradeRole)
{
	TestDowngradeRole();
}

void FTestNetRoleNetSerializer::SetUp()
{
	static bool bIsInitialized;
	if (bIsInitialized)
	{
		return;
	}

	Enum = StaticEnum<ENetRole>();

	PartialInitNetRoleSerializerConfig(Config, Enum);
	
	// NumEnums actually also contain the generated _MAX enum value which might not even be a valid value by the backed type. Skip it!
	const int32 EnumValueCount = Enum->NumEnums() - 1;

	// Setup test values
	{
		// Valid values
		TArray<uint8> TempValues;
		TempValues.Reserve(EnumValueCount);
		for (int32 EnumIt = 0, EnumEndIt = EnumValueCount; EnumIt != EnumEndIt; ++EnumIt)
		{
			const uint64 Value = static_cast<uint64>(Enum->GetValueByIndex(EnumIt));
			TempValues.Add(static_cast<uint8>(Value));
		}
		Values = MoveTemp(TempValues);

		// Invalid values
		TArray<uint8> TempInvalidValues;
		TempInvalidValues.Reserve(3);
		if (Config.LowerBound > 0)
		{
			TempInvalidValues.Add(Config.LowerBound - uint8(1));
		}

		if (Config.UpperBound < 255)
		{
			TempInvalidValues.Add(Config.UpperBound + uint8(1));
		}

		// Try adding an invalid value between the smallest and largest values found
		for (uint8 Value = Config.LowerBound, UpperBound = Config.UpperBound; Value != UpperBound; ++Value)
		{
			if (!Enum->IsValidEnumValue(Value))
			{
				TempInvalidValues.Add(Value);
				break;
			}
		}
		InvalidValues = MoveTemp(TempInvalidValues);
	}

	bIsInitialized = true;
}

void FTestNetRoleNetSerializer::TestIsEqual()
{
	TArray<uint8> CompareValues[2];
	TArray<bool> ExpectedResults[2];

	CompareValues[0] = Values;
	ExpectedResults[0].Reserve(Values.Num());
	for (int32 ValueIt = 0, ValueEndIt = Values.Num(); ValueIt != ValueEndIt; ++ValueIt)
	{
		ExpectedResults[0].Add(true);
	}

	CompareValues[1].Reserve(Values.Num());
	ExpectedResults[1].Reserve(Values.Num());
	for (int32 ValueIt = 0, ValueEndIt = Values.Num(); ValueIt != ValueEndIt; ++ValueIt)
	{
		CompareValues[1].Add(Values[(ValueIt + 1) % ValueEndIt]);
		ExpectedResults[1].Add(Values[ValueIt] == Values[(ValueIt + 1) % ValueEndIt]);
	}

	// Do two rounds of testing per config, one where we compare each value with itself and one where we compare against a value in range.
	for (const SIZE_T TestRoundIt : {0, 1})
	{
		// Do both quantized and regular compares
		for (const bool bQuantizedCompare : {false, true})
		{
			const bool bSuccess = Super::TestIsEqual(Values.GetData(), CompareValues[TestRoundIt].GetData(), ExpectedResults[TestRoundIt].GetData(), Values.Num(), Config, bQuantizedCompare);
			if (!bSuccess)
			{
				return;
			}
		}
	}
}

void FTestNetRoleNetSerializer::TestValidate()
{
	// Check valid values
	{
		TArray<bool> ExpectedResults;
		ExpectedResults.SetNumUninitialized(Values.Num());
		for (int32 ValueIt = 0, ValueEndIt = Values.Num(); ValueIt != ValueEndIt; ++ValueIt)
		{
			ExpectedResults[ValueIt] = true;
		}

		const bool bSuccess = Super::TestValidate(Values.GetData(), ExpectedResults.GetData(), Values.Num(), Config);
		if (!bSuccess)
		{
			return;
		}
	}

	// Check invalid values
	{
		UE_NET_EXPECT_GT_MSG(InvalidValues.Num(), 0, "No invalid values found to test EnumIntSerializer::Validate.");

		TArray<bool> ExpectedResults;
		ExpectedResults.SetNumZeroed(InvalidValues.Num());
		const bool bSuccess = Super::TestValidate(InvalidValues.GetData(), ExpectedResults.GetData(), InvalidValues.Num(), Config);
		if (!bSuccess)
		{
			return;
		}
	}
}

void FTestNetRoleNetSerializer::TestQuantize()
{
	Super::TestQuantize(Values.GetData(), Values.Num(), Config);
}

void FTestNetRoleNetSerializer::TestSerialize()
{
	constexpr bool bQuantizedCompare = false;
	Super::TestSerialize(Values.GetData(), Values.GetData(), Values.Num(), Config, bQuantizedCompare);
}

void FTestNetRoleNetSerializer::TestSerializeDelta()
{
	Super::TestSerializeDelta(Values.GetData(), Values.Num(), Config);
}

void FTestNetRoleNetSerializer::TestDequantizeSwapsRoles()
{
	const auto& EqualityFunc = [](NetSerializerValuePointer Value0, NetSerializerValuePointer Value1) -> bool { return *reinterpret_cast<const uint8*>(Value0) == *reinterpret_cast<const uint8*>(Value1); };

	FReplicationStateDescriptorBuilder::FResult Descriptors;
	const FReplicationStateDescriptor* Descriptor = nullptr;
	// Use cleared parameters instead of default parameters so we only get the leaf class properties in the state
	FReplicationStateDescriptorBuilder::FParameters DescriptorCreationParameters;
	FReplicationStateDescriptorBuilder::CreateDescriptorsForClass(Descriptors, UClassWithNetRoleSwapping::StaticClass(), DescriptorCreationParameters);

	UE_NET_ASSERT_EQ(Descriptors.Num(), 1);
	
	Descriptor = Descriptors[0];

	{
		FPropertyReplicationState SourceReplicationState(Descriptor);
		FPropertyReplicationState TargetReplicationState(Descriptor);
		TStrongObjectPtr<UClassWithNetRoleSwapping> Source(NewObject<UClassWithNetRoleSwapping>());
		TStrongObjectPtr<UClassWithNetRoleSwapping> Target(NewObject<UClassWithNetRoleSwapping>());

		Source->Role = ENetRole::ROLE_Authority;
		Source->RemoteRole = ENetRole::ROLE_AutonomousProxy;

		SourceReplicationState.PollPropertyReplicationState(Source.Get());
		TargetReplicationState.PollPropertyReplicationState(Target.Get());

		FReplicationStateOperations::Quantize(Context, QuantizedBuffer[0], SourceReplicationState.GetStateBuffer(), Descriptor);
		UE_NET_ASSERT_FALSE(Context.HasError());

		Writer.InitBytes(BitStreamBuffer, sizeof(BitStreamBuffer));
		FReplicationStateOperations::Serialize(Context, QuantizedBuffer[0], Descriptor);
		Writer.CommitWrites();
		UE_NET_ASSERT_FALSE(Writer.IsOverflown());
		UE_NET_ASSERT_FALSE(Context.HasError());

		Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());
		FReplicationStateOperations::Deserialize(Context, SourceBuffer[0], Descriptor);
		UE_NET_ASSERT_FALSE(Reader.IsOverflown());
		UE_NET_ASSERT_FALSE(Context.HasError());

		FReplicationStateOperations::Dequantize(Context, TargetReplicationState.GetStateBuffer(), SourceBuffer[0], Descriptor);
		UE_NET_ASSERT_FALSE(Context.HasError());

		constexpr bool bPushAll = true;
		TargetReplicationState.PushPropertyReplicationState(Target.Get(), Target.Get(), bPushAll);
		
		UE_NET_ASSERT_EQ_MSG(Target->RemoteRole, Source->Role, "Roles were not swapped");
		UE_NET_ASSERT_EQ_MSG(Target->Role, Source->RemoteRole, "Roles were not swapped");
	}
}

void FTestNetRoleNetSerializer::TestDowngradeRole()
{
	const auto& EqualityFunc = [](NetSerializerValuePointer Value0, NetSerializerValuePointer Value1) -> bool { return *reinterpret_cast<const uint8*>(Value0) == *reinterpret_cast<const uint8*>(Value1); };

	FReplicationStateDescriptorBuilder::FResult Descriptors;
	const FReplicationStateDescriptor* Descriptor = nullptr;
	// Use cleared parameters instead of default parameters so we only get the leaf class properties in the state
	FReplicationStateDescriptorBuilder::FParameters DescriptorCreationParameters;
	FReplicationStateDescriptorBuilder::CreateDescriptorsForClass(Descriptors, UClassWithNetRoleSwapping::StaticClass(), DescriptorCreationParameters);
	
	Descriptor = Descriptors[0];

	InternalContext.bDowngradeAutonomousProxyRole = 1;
	{
		FPropertyReplicationState SourceReplicationState(Descriptor);
		FPropertyReplicationState TargetReplicationState(Descriptor);
		TStrongObjectPtr<UClassWithNetRoleSwapping> Source(NewObject<UClassWithNetRoleSwapping>());
		TStrongObjectPtr<UClassWithNetRoleSwapping> Target(NewObject<UClassWithNetRoleSwapping>());

		Source->Role = ENetRole::ROLE_Authority;
		Source->RemoteRole = ENetRole::ROLE_AutonomousProxy;

		SourceReplicationState.PollPropertyReplicationState(Source.Get());
		TargetReplicationState.PollPropertyReplicationState(Target.Get());

		FReplicationStateOperations::Quantize(Context, QuantizedBuffer[0], SourceReplicationState.GetStateBuffer(), Descriptor);

		Writer.InitBytes(BitStreamBuffer, sizeof(BitStreamBuffer));
		FReplicationStateOperations::Serialize(Context, QuantizedBuffer[0], Descriptor);
		Writer.CommitWrites();
		UE_NET_ASSERT_FALSE(Writer.IsOverflown());
		UE_NET_ASSERT_FALSE(Context.HasError());

		Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());
		FReplicationStateOperations::Deserialize(Context, SourceBuffer[0], Descriptor);
		UE_NET_ASSERT_FALSE(Reader.IsOverflown());
		UE_NET_ASSERT_FALSE(Context.HasError());

		FReplicationStateOperations::Dequantize(Context, TargetReplicationState.GetStateBuffer(), SourceBuffer[0], Descriptor);

		constexpr bool bPushAll = true;
		TargetReplicationState.PushPropertyReplicationState(Target.Get(), Target.Get(), bPushAll);
		
		UE_NET_ASSERT_EQ_MSG(Target->RemoteRole, Source->Role, "Roles were not swapped");
		UE_NET_ASSERT_EQ_MSG(Target->Role, TEnumAsByte<ENetRole>(ENetRole::ROLE_SimulatedProxy), "Role was not downgraded");
	}
}

}
