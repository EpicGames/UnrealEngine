// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/Serialization/NetSerializer.h"

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FMinimalNetSerializer, REPLICATIONSYSTEMTESTPLUGIN_API);

// Keep state out of the following structs! We support tests being run in parallel.
struct FMinimalNetSerializer
{
	static constexpr uint32 Version = 0x534C494D;

	typedef int SourceType;
	typedef FNetSerializerConfig ConfigType;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);
};
UE_NET_IMPLEMENT_SERIALIZER(FMinimalNetSerializer);

UE_NET_DECLARE_SERIALIZER(FFullNetSerializer, REPLICATIONSYSTEMTESTPLUGIN_API);

struct FFullNetSerializer
{
	static constexpr uint32 Version = 0x00507353U;
	static constexpr bool bIsForwardingSerializer = true; // Triggers asserts if a function is missing

	typedef int SourceType;
	typedef FNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);
	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&);
	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);
	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&);
	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);
	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);
};
UE_NET_IMPLEMENT_SERIALIZER(FFullNetSerializer);

UE_NET_DECLARE_SERIALIZER(FNetSerializerWithHasConnectionSpecificSerializationTrait, REPLICATIONSYSTEMTESTPLUGIN_API);
struct FNetSerializerWithHasConnectionSpecificSerializationTrait : public FMinimalNetSerializer
{
	static constexpr bool bHasConnectionSpecificSerialization = true;
};
UE_NET_IMPLEMENT_SERIALIZER(FNetSerializerWithHasConnectionSpecificSerializationTrait);

UE_NET_DECLARE_SERIALIZER(FNetSerializerWithApply, REPLICATIONSYSTEMTESTPLUGIN_API);
struct FNetSerializerWithApply : public FMinimalNetSerializer
{
	typedef int SourceType;

	static void Apply(FNetSerializationContext&, const FNetApplyArgs&);
};
UE_NET_IMPLEMENT_SERIALIZER(FNetSerializerWithApply);


}

namespace UE::Net::Private
{

// FMinimalNetSerializer
UE_NET_TEST(FMinimalNetSerializer, CanGetNetSerializer)
{
	const FNetSerializer* Serializer = &UE_NET_GET_SERIALIZER(FMinimalNetSerializer);
	UE_NET_ASSERT_NE(Serializer, static_cast<const FNetSerializer*>(nullptr));
}

UE_NET_TEST(FMinimalNetSerializer, HasNoTraits)
{
	const FNetSerializer* Serializer = &UE_NET_GET_SERIALIZER(FMinimalNetSerializer);
	UE_NET_ASSERT_FALSE(EnumHasAnyFlags(Serializer->Traits, ~ENetSerializerTraits::None));
};

UE_NET_TEST(FMinimalNetSerializer, HasProperVersion)
{
	const FNetSerializer* Serializer = &UE_NET_GET_SERIALIZER(FMinimalNetSerializer);
	constexpr uint32 Version = FMinimalNetSerializer::Version;
	UE_NET_ASSERT_EQ(Serializer->Version, Version);
};

UE_NET_TEST(FMinimalNetSerializer, SerializeIsCalled)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FMinimalNetSerializer);

	FMinimalNetSerializer::SourceType SerializeCallCount = 0;

	FNetSerializationContext Context;
	FNetSerializeArgs Args;
	Args.Version = Serializer.Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(nullptr);
	Args.Source = NetSerializerValuePointer(&SerializeCallCount);
	Serializer.Serialize(Context, Args);
	UE_NET_ASSERT_EQ(SerializeCallCount, FMinimalNetSerializer::SourceType(1));
}

UE_NET_TEST(FMinimalNetSerializer, DeserializeIsCalled)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FMinimalNetSerializer);

	FMinimalNetSerializer::SourceType DeserializeCallCount = 0;

	FNetSerializationContext Context;
	FNetDeserializeArgs Args;
	Args.Version = Serializer.Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(nullptr);
	Args.Target = NetSerializerValuePointer(&DeserializeCallCount);
	Serializer.Deserialize(Context, Args);
	UE_NET_ASSERT_EQ(DeserializeCallCount, FMinimalNetSerializer::SourceType(1));
}

UE_NET_TEST(FMinimalNetSerializer, DoesNotHaveDefaultConfig)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FMinimalNetSerializer);
	UE_NET_ASSERT_EQ(Serializer.DefaultConfig, nullptr);
}

// FFullNetSerializer
UE_NET_TEST(FFullNetSerializer, CanGetNetSerializer)
{
	const FNetSerializer* Serializer = &UE_NET_GET_SERIALIZER(FFullNetSerializer);
	UE_NET_ASSERT_NE(Serializer, static_cast<const FNetSerializer*>(nullptr));
}

UE_NET_TEST(FFullNetSerializer, HasIsForwardingSerializerTrait)
{
	const FNetSerializer* Serializer = &UE_NET_GET_SERIALIZER(FFullNetSerializer);
	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(Serializer->Traits, ENetSerializerTraits::IsForwardingSerializer));
}

UE_NET_TEST(FFullNetSerializer, SerializeIsCalled)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FFullNetSerializer);

	FFullNetSerializer::SourceType CallCount = 0;

	FNetSerializationContext Context;
	FNetSerializeArgs Args;
	Args.Version = Serializer.Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(nullptr);
	Args.Source = NetSerializerValuePointer(&CallCount);
	Serializer.Serialize(Context, Args);
	UE_NET_ASSERT_EQ(CallCount, FFullNetSerializer::SourceType(1));
}

UE_NET_TEST(FFullNetSerializer, DeserializeIsCalled)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FFullNetSerializer);

	FFullNetSerializer::SourceType CallCount = 0;

	FNetSerializationContext Context;
	FNetDeserializeArgs Args;
	Args.Version = Serializer.Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(nullptr);
	Args.Target = NetSerializerValuePointer(&CallCount);
	Serializer.Deserialize(Context, Args);
	UE_NET_ASSERT_EQ(CallCount, FFullNetSerializer::SourceType(1));
}

UE_NET_TEST(FFullNetSerializer, SerializeDeltaIsCalled)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FFullNetSerializer);

	FFullNetSerializer::SourceType CallCount = 0;

	FNetSerializationContext Context;
	FNetSerializeDeltaArgs Args;
	Args.Version = Serializer.Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(nullptr);
	Args.Source = NetSerializerValuePointer(&CallCount);
	Serializer.SerializeDelta(Context, Args);
	UE_NET_ASSERT_EQ(CallCount, FFullNetSerializer::SourceType(1));
}

UE_NET_TEST(FFullNetSerializer, DeserializeDeltaIsCalled)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FFullNetSerializer);

	FFullNetSerializer::SourceType CallCount = 0;

	FNetSerializationContext Context;
	FNetDeserializeDeltaArgs Args;
	Args.Version = Serializer.Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(nullptr);
	Args.Target = NetSerializerValuePointer(&CallCount);
	Serializer.DeserializeDelta(Context, Args);
	UE_NET_ASSERT_EQ(CallCount, FFullNetSerializer::SourceType(1));
}

UE_NET_TEST(FFullNetSerializer, QuantizeIsCalled)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FFullNetSerializer);

	FFullNetSerializer::SourceType CallCount = 0;

	FNetSerializationContext Context;
	FNetQuantizeArgs Args;
	Args.Version = Serializer.Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(nullptr);
	Args.Target = NetSerializerValuePointer(&CallCount);
	Serializer.Quantize(Context, Args);
	UE_NET_ASSERT_EQ(CallCount, FFullNetSerializer::SourceType(1));
}

UE_NET_TEST(FFullNetSerializer, DequantizeIsCalled)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FFullNetSerializer);

	FFullNetSerializer::SourceType CallCount = 0;

	FNetSerializationContext Context;
	FNetDequantizeArgs Args;
	Args.Version = Serializer.Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(nullptr);
	Args.Target = NetSerializerValuePointer(&CallCount);
	Serializer.Dequantize(Context, Args);
	UE_NET_ASSERT_EQ(CallCount, FFullNetSerializer::SourceType(1));
}

UE_NET_TEST(FFullNetSerializer, IsEqualIsCalled)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FFullNetSerializer);

	FFullNetSerializer::SourceType CallCount = 0;

	FNetSerializationContext Context;
	FNetIsEqualArgs Args;
	Args.Version = Serializer.Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(nullptr);
	Args.Source0 = NetSerializerValuePointer(&CallCount);
	Args.bStateIsQuantized = false;
	Serializer.IsEqual(Context, Args);
	UE_NET_ASSERT_EQ(CallCount, FFullNetSerializer::SourceType(1));
}

UE_NET_TEST(FFullNetSerializer, ValidatelIsCalled)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FFullNetSerializer);

	FFullNetSerializer::SourceType CallCount = 0;

	FNetSerializationContext Context;
	FNetValidateArgs Args;
	Args.Version = Serializer.Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(nullptr);
	Args.Source = NetSerializerValuePointer(&CallCount);
	Serializer.Validate(Context, Args);
	UE_NET_ASSERT_EQ(CallCount, FFullNetSerializer::SourceType(1));
}

UE_NET_TEST(FFullNetSerializer, CloneDynamicStatelIsCalled)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FFullNetSerializer);

	FFullNetSerializer::SourceType CallCount = 0;

	FNetSerializationContext Context;
	FNetCloneDynamicStateArgs Args;
	Args.Version = Serializer.Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(nullptr);
	Args.Target = NetSerializerValuePointer(&CallCount);
	Serializer.CloneDynamicState(Context, Args);
	UE_NET_ASSERT_EQ(CallCount, FFullNetSerializer::SourceType(1));
}

UE_NET_TEST(FFullNetSerializer, FreeDynamicStatelIsCalled)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FFullNetSerializer);

	FFullNetSerializer::SourceType CallCount = 0;

	FNetSerializationContext Context;
	FNetFreeDynamicStateArgs Args;
	Args.Version = Serializer.Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(nullptr);
	Args.Source = NetSerializerValuePointer(&CallCount);
	Serializer.FreeDynamicState(Context, Args);
	UE_NET_ASSERT_EQ(CallCount, FFullNetSerializer::SourceType(1));
}

UE_NET_TEST(FFullNetSerializer, HasDefaultConfig)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FFullNetSerializer);
	UE_NET_ASSERT_EQ(Serializer.DefaultConfig, &FFullNetSerializer::DefaultConfig);
}

// FNetSerializerWithHasConnectionSpecificSerializationTrait
UE_NET_TEST(FNetSerializerWithHasConnectionSpecificSerializationTrait, HasHasConnectionSpecificSerializationTrait)
{
	const FNetSerializer* Serializer = &UE_NET_GET_SERIALIZER(FNetSerializerWithHasConnectionSpecificSerializationTrait);
	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(Serializer->Traits, ENetSerializerTraits::HasConnectionSpecificSerialization));
}

//
UE_NET_TEST(FNetSerializerWithApply, HasApplyTrait)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FNetSerializerWithApply);
	UE_NET_ASSERT_TRUE(EnumHasAnyFlags(Serializer.Traits, ENetSerializerTraits::HasApply));
}

UE_NET_TEST(FNetSerializerWithApply, HasApplyFunction)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FNetSerializerWithApply);
	UE_NET_ASSERT_NE(Serializer.Apply, NetApplyFunction(0));
}

UE_NET_TEST(FNetSerializerWithApply, ApplyGetsCalled)
{
	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FNetSerializerWithApply);

	FNetSerializerWithApply::SourceType CallCount = 0;

	FNetSerializationContext Context;
	FNetApplyArgs Args;
	Args.Version = Serializer.Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(nullptr);
	Args.Source = NetSerializerValuePointer(&CallCount);
	Serializer.Apply(Context, Args);
	UE_NET_ASSERT_EQ(CallCount, FNetSerializerWithApply::SourceType(1));
}

}

namespace UE::Net
{

// FMinimalNetSerializer implementation
void FMinimalNetSerializer::Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args)
{
	*reinterpret_cast<SourceType*>(Args.Source) += 1;
}

void FMinimalNetSerializer::Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args)
{
	*reinterpret_cast<SourceType*>(Args.Target) += 1;
}

// FFullNetSerializer implementation
void FFullNetSerializer::Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args)
{
	*reinterpret_cast<SourceType*>(Args.Source) += 1;
}

void FFullNetSerializer::Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args)
{
	*reinterpret_cast<SourceType*>(Args.Target) += 1;
}

void FFullNetSerializer::SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args)
{
	*reinterpret_cast<SourceType*>(Args.Source) += 1;
}

void FFullNetSerializer::DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args)
{
	*reinterpret_cast<SourceType*>(Args.Target) += 1;
}

void FFullNetSerializer::Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args)
{
	*reinterpret_cast<SourceType*>(Args.Target) += 1;
}

void FFullNetSerializer::Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args)
{
	*reinterpret_cast<SourceType*>(Args.Target) += 1;
}

bool FFullNetSerializer::IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args)
{
	*reinterpret_cast<SourceType*>(Args.Source0) += 1;
	return true;
}

bool FFullNetSerializer::Validate(FNetSerializationContext&, const FNetValidateArgs& Args)
{
	*reinterpret_cast<SourceType*>(Args.Source) += 1;
	return true;
}

void FFullNetSerializer::CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs& Args)
{
	*reinterpret_cast<SourceType*>(Args.Target) += 1;
}

void FFullNetSerializer::FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs& Args)
{
	*reinterpret_cast<SourceType*>(Args.Source) += 1;
}

void FFullNetSerializer::CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs& Args)
{
	*reinterpret_cast<SourceType*>(Args.Source) += 1;
}

// FNetSerializerWithApply
void FNetSerializerWithApply::Apply(FNetSerializationContext&, const FNetApplyArgs& Args)
{
	*reinterpret_cast<SourceType*>(Args.Source) += 1;
}

}
