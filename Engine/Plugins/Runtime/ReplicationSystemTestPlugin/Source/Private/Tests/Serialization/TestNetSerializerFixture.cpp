// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/NetSerializer.h"

namespace UE::Net
{

class FTestNetSerializerFixture::FFreeBufferScope
{
public:
	FFreeBufferScope(FTestNetSerializerFixture& InTestFixture, const FNetSerializerConfig* InNetSerializerConfig)
	: TestFixture(InTestFixture)
	, BuffersToFree{}
	, BuffersToFreeCount(0)
	, NetSerializerConfig(InNetSerializerConfig)
	{
	}

	~FFreeBufferScope()
	{
		FreeBuffers();
	}

	void AddBuffer(NetSerializerValuePointer Buffer)
	{
		check(BuffersToFreeCount < UE_ARRAY_COUNT(BuffersToFree));
		BuffersToFree[BuffersToFreeCount++] = Buffer;
	}

	void FreeBuffers()
	{
		if (EnumHasAnyFlags(TestFixture.Serializer.Traits, ENetSerializerTraits::IsForwardingSerializer | ENetSerializerTraits::HasDynamicState))
		{
			FNetFreeDynamicStateArgs Args;
			Args.Version = TestFixture.SerializerVersionOverride;
			Args.NetSerializerConfig = NetSerializerConfig;
			for (uint32 It = 0, EndIt = BuffersToFreeCount; It != EndIt; ++It)
			{
				Args.Source = BuffersToFree[It];
				TestFixture.Serializer.FreeDynamicState(TestFixture.Context, Args);
			}
		}
		BuffersToFreeCount = 0;
	}

private:
	FTestNetSerializerFixture& TestFixture;
	NetSerializerValuePointer BuffersToFree[3];
	uint32 BuffersToFreeCount;
	const FNetSerializerConfig* NetSerializerConfig = nullptr;
};

FTestNetSerializerFixture::FTestNetSerializerFixture(const FNetSerializer& InSerializer)
: FNetworkAutomationTestSuiteFixture()
, Serializer(InSerializer)
, SerializerVersionOverride(InSerializer.Version)
, Context(&Reader, &Writer)
{
	check(Serializer.QuantizedTypeSize <= sizeof(QuantizedBuffer[0]));

	// If this serializer could have allocated state then Taint quantized state buffers before testing.
	// Assume buffers are properly freed if necessary after every use of these buffers.
	if (EnumHasAnyFlags(InSerializer.Traits, ENetSerializerTraits::IsForwardingSerializer | ENetSerializerTraits::HasDynamicState))
	{
		FPlatformMemory::Memset(QuantizedBuffer[0], 0, sizeof(QuantizedBuffer[0]));
		FPlatformMemory::Memset(QuantizedBuffer[1], 0, sizeof(QuantizedBuffer[1]));
	}
}

void FTestNetSerializerFixture::PrepareTest(uint32 Flags)
{
	constexpr uint8 FillPattern = uint8(0xCDU);

	if (Flags & TaintBitStreamBuffer)
		FPlatformMemory::Memset(BitStreamBuffer, FillPattern, sizeof(BitStreamBuffer));
	if (Flags & TaintQuantizedBuffer0)
		FPlatformMemory::Memset(QuantizedBuffer[0], FillPattern, sizeof(QuantizedBuffer[0]));
	if (Flags & TaintQuantizedBuffer1)
		FPlatformMemory::Memset(QuantizedBuffer[1], FillPattern, sizeof(QuantizedBuffer[1]));
	if (Flags & TaintSourceBuffer0)
		FPlatformMemory::Memset(SourceBuffer[0], FillPattern, sizeof(SourceBuffer[0]));
	if (Flags & TaintSourceBuffer1)
		FPlatformMemory::Memset(SourceBuffer[1], FillPattern, sizeof(SourceBuffer[1]));
}

bool FTestNetSerializerFixture::TestQuantize(const FNetSerializerConfig& Config, NetSerializerValuePointer Value)
{
	const uint32 PrepareTestFlags = (bTaintBuffersBeforeTest ? (TaintQuantizedBuffer0 | TaintQuantizedBuffer1 | TaintSourceBuffer0) : 0U);
	PrepareTest(PrepareTestFlags);

	// 1. Verify that quantized value is equal to self
	// 2. Verify that dequantized value is equal to source
	// 3. Verify that quantized value is equal to second quantized value.

	FFreeBufferScope FreeScope(*this, &Config);

	FNetQuantizeArgs QuantizeArgs;
	QuantizeArgs.Version = SerializerVersionOverride;
	QuantizeArgs.NetSerializerConfig = &Config;
	QuantizeArgs.Source = Value;
	QuantizeArgs.Target = NetSerializerValuePointer(QuantizedBuffer[0]);
	Serializer.Quantize(Context, QuantizeArgs);
	FreeScope.AddBuffer(QuantizeArgs.Target);
	if (Context.HasError())
	{
		UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "Quantize() of value reported an error.");
		return false;
	}

	FNetIsEqualArgs QuantizedIsEqualArgs;
	QuantizedIsEqualArgs.Version = SerializerVersionOverride;
	QuantizedIsEqualArgs.NetSerializerConfig = &Config;
	QuantizedIsEqualArgs.Source0 = QuantizeArgs.Target;
	QuantizedIsEqualArgs.Source1 = QuantizeArgs.Target;
	QuantizedIsEqualArgs.bStateIsQuantized = true;
	const bool bIsEqual = Serializer.IsEqual(Context, QuantizedIsEqualArgs);
	if (!bIsEqual)
	{
		UE_NET_EXPECT_TRUE_MSG(bIsEqual, "IsEqual() on quantized value compared to itself returned false.");
		return false;
	}

	FNetDequantizeArgs DequantizeArgs;
	DequantizeArgs.Version = SerializerVersionOverride;
	DequantizeArgs.NetSerializerConfig = &Config;
	DequantizeArgs.Source = QuantizeArgs.Target;
	DequantizeArgs.Target = NetSerializerValuePointer(SourceBuffer[0]);
	Serializer.Dequantize(Context, DequantizeArgs);
	if (Context.HasError())
	{
		UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "Dequantize() of quantized value reported an error.");
		return false;
	}

	FNetIsEqualArgs DequantizedIsEqualArgs;
	DequantizedIsEqualArgs.Version = SerializerVersionOverride;
	DequantizedIsEqualArgs.NetSerializerConfig = &Config;
	DequantizedIsEqualArgs.Source0 = Value;
	DequantizedIsEqualArgs.Source1 = DequantizeArgs.Target;
	DequantizedIsEqualArgs.bStateIsQuantized = false;
	const bool bIsDequantizedEqual = Serializer.IsEqual(Context, DequantizedIsEqualArgs);
	if (!bIsDequantizedEqual)
	{
		UE_NET_EXPECT_TRUE_MSG(bIsDequantizedEqual, "IsEqual() on dequantized value compared to source value returned false.");
		return false;
	}

	// Check if tainted buffers causes memcmp fail
	if (!EnumHasAnyFlags(Serializer.Traits, ENetSerializerTraits::IsForwardingSerializer | ENetSerializerTraits::HasDynamicState))
	{
		FPlatformMemory::Memset(QuantizedBuffer[0], 0x00, sizeof(QuantizedBuffer[0]));
		FPlatformMemory::Memset(QuantizedBuffer[1], 0xFF, sizeof(QuantizedBuffer[1]));

		Serializer.Quantize(Context, QuantizeArgs);

		FNetQuantizeArgs QuantizeArgs2 = QuantizeArgs;
		QuantizeArgs2.Target = NetSerializerValuePointer(QuantizedBuffer[1]);
		Serializer.Quantize(Context, QuantizeArgs2);

		const bool bIsQuantizedEqual = !FPlatformMemory::Memcmp(QuantizedBuffer[0], QuantizedBuffer[1], Serializer.QuantizedTypeSize);
		if (!bIsQuantizedEqual)
		{
			UE_NET_EXPECT_TRUE_MSG(bIsQuantizedEqual, "Memcmp() on quantized values with tainted buffers failed.");
			return false;
		}
	}

	return true;
}

bool FTestNetSerializerFixture::TestSerialize(const FNetSerializerConfig& Config, NetSerializerValuePointer Value, NetSerializerValuePointer ExpectedValue, bool bQuantizedCompare, TOptional<TFunctionRef<bool(NetSerializerValuePointer, NetSerializerValuePointer)>> CompareFunc)
{
	const uint32 PrepareTestFlags = (bTaintBuffersBeforeTest ? (TaintBitStreamBuffer | TaintQuantizedBuffer0 | TaintQuantizedBuffer1 | TaintSourceBuffer0) : 0U);
	PrepareTest(PrepareTestFlags);

	// Serialization of source data is a two step process.
	// 1. Quantize, converting to some POD internal format.
	// 2. Serialize

	// Verification is done using deserialize followed by a quantized or dequantized compare.

	FFreeBufferScope FreeScope(*this, &Config);

	FNetQuantizeArgs QuantizeArgs;
	QuantizeArgs.Version = SerializerVersionOverride;
	QuantizeArgs.NetSerializerConfig = &Config;
	QuantizeArgs.Source = Value;
	QuantizeArgs.Target = NetSerializerValuePointer(QuantizedBuffer[0]);
	Serializer.Quantize(Context, QuantizeArgs);
	FreeScope.AddBuffer(QuantizeArgs.Target);
	if (Context.HasError())
	{
		UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "Quantize() of value reported an error.");
		return false;
	}

	Writer.InitBytes(BitStreamBuffer, sizeof(BitStreamBuffer));
	FNetSerializeArgs SerializeArgs;
	SerializeArgs.Version = SerializerVersionOverride;
	SerializeArgs.NetSerializerConfig = &Config;
	SerializeArgs.Source = QuantizeArgs.Target;
	Serializer.Serialize(Context, SerializeArgs);
	Writer.CommitWrites();
	if (Writer.IsOverflown() || Context.HasError())
	{
		UE_NET_EXPECT_FALSE_MSG(Writer.IsOverflown(), "FNetBitStreamWriter overflowed.");
		UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "Serialize() reported an error.");
		return false;
	}

	Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());
	FNetDeserializeArgs DeserializeArgs;
	DeserializeArgs.Version = SerializerVersionOverride;
	DeserializeArgs.NetSerializerConfig = &Config;
	DeserializeArgs.Target = NetSerializerValuePointer(QuantizedBuffer[1]);
	Serializer.Deserialize(Context, DeserializeArgs);
	FreeScope.AddBuffer(DeserializeArgs.Target);
	if (Reader.IsOverflown() || Context.HasError())
	{
		UE_NET_EXPECT_FALSE_MSG(Reader.IsOverflown(), "FNetBitStreamReader overflowed.");
		UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "Deserialize() reported an error.");
		return false;
	}

	if (!bQuantizedCompare || CompareFunc.IsSet())
	{
		// Need to dequantize the deserialized value
		FNetDequantizeArgs DequantizeArgs;
		DequantizeArgs.Version = SerializerVersionOverride;
		DequantizeArgs.NetSerializerConfig = &Config;
		DequantizeArgs.Source = DeserializeArgs.Target;
		DequantizeArgs.Target = NetSerializerValuePointer(SourceBuffer[0]);
		Serializer.Dequantize(Context, DequantizeArgs);
		if (Context.HasError())
		{
			UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "Dequantize() of deserialized value reported an error.");
			return false;
		}

		FNetIsEqualArgs IsEqualArgs;
		IsEqualArgs.Version = SerializerVersionOverride;
		IsEqualArgs.NetSerializerConfig = &Config;
		IsEqualArgs.Source0 = ExpectedValue;
		IsEqualArgs.Source1 = DequantizeArgs.Target;
		IsEqualArgs.bStateIsQuantized = false;

		if (CompareFunc.IsSet())
		{
			const bool bIsEqual = (*CompareFunc)(IsEqualArgs.Source0, IsEqualArgs.Source1);
			if (!bIsEqual)
			{
				UE_NET_EXPECT_TRUE_MSG(bIsEqual, "Custom compare on dequantized values returned false");
				return false;
			}
		}
		else
		{
			const bool bIsEqual = Serializer.IsEqual(Context, IsEqualArgs);
			if (!bIsEqual)
			{
				UE_NET_EXPECT_TRUE_MSG(bIsEqual, "IsEqual() on dequantized values returned false");
				return false;
			}
		}

		return true;
	}
	else
	{
		if (Value != ExpectedValue)
		{
			// It's fine to reuse the QuantizedBuffer already used
			QuantizeArgs.Source = ExpectedValue;
			Serializer.Quantize(Context, QuantizeArgs);
			if (Context.HasError())
			{
				UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "Quantize() of expected value reported an error.");
				return false;
			}
		}

		FNetIsEqualArgs IsEqualArgs;
		IsEqualArgs.Version = SerializerVersionOverride;
		IsEqualArgs.NetSerializerConfig = &Config;
		IsEqualArgs.Source0 = QuantizeArgs.Target;
		IsEqualArgs.Source1 = DeserializeArgs.Target;
		IsEqualArgs.bStateIsQuantized = true;
		const bool bResult = Serializer.IsEqual(Context, IsEqualArgs);
		return bResult;
	}
}

bool FTestNetSerializerFixture::TestSerializeDelta(const FNetSerializerConfig& Config, NetSerializerValuePointer Value, NetSerializerValuePointer PrevValue)
{
	const uint32 PrepareTestFlags = (bTaintBuffersBeforeTest ? (TaintBitStreamBuffer | TaintQuantizedBuffer0 | TaintQuantizedBuffer1 | TaintSourceBuffer0) : 0U);
	PrepareTest(PrepareTestFlags);

	// Delta serialization testing requires quantizing the value and previous value.
	// After that SerializeDelta can be called.
	// Verification is done by verifying that the deserialized delta value matches the quantized source value.

	FFreeBufferScope FreeScope(*this, &Config);

	FNetQuantizeArgs QuantizeValueArgs;
	FNetQuantizeArgs QuantizePrevValueArgs;
	{
		{
			QuantizeValueArgs.Version = SerializerVersionOverride;
			QuantizeValueArgs.NetSerializerConfig = &Config;
			QuantizeValueArgs.Source = Value;
			QuantizeValueArgs.Target = NetSerializerValuePointer(QuantizedBuffer[0]);
			Serializer.Quantize(Context, QuantizeValueArgs);
			FreeScope.AddBuffer(QuantizeValueArgs.Target);
			if (Context.HasError())
			{
				UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "Quantize() of value reported an error.");
				return false;
			}
		}

		{
			QuantizePrevValueArgs = QuantizeValueArgs;
			QuantizePrevValueArgs.Source = PrevValue;
			QuantizePrevValueArgs.Target = NetSerializerValuePointer(QuantizedBuffer[1]);

			Serializer.Quantize(Context, QuantizePrevValueArgs);
			FreeScope.AddBuffer(QuantizePrevValueArgs.Target);
			if (Context.HasError())
			{
				UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "Quantize() of value reported an error.");
				return false;
			}
		}
	}

	// SerializeDelta
	FNetSerializeDeltaArgs SerializeDeltaArgs;
	{
		Writer.InitBytes(BitStreamBuffer, sizeof(BitStreamBuffer));
		SerializeDeltaArgs.Version = SerializerVersionOverride;
		SerializeDeltaArgs.NetSerializerConfig = &Config;
		SerializeDeltaArgs.Source = QuantizeValueArgs.Target;
		SerializeDeltaArgs.Prev = QuantizePrevValueArgs.Target;
		Serializer.SerializeDelta(Context, SerializeDeltaArgs);
		Writer.CommitWrites();
		if (Writer.IsOverflown() || Context.HasError())
		{
			UE_NET_EXPECT_FALSE_MSG(Writer.IsOverflown(), "FNetBitStreamWriter overflowed.");
			UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "SerializeDelta() reported an error.");
			return false;
		}
	}

	// DeserializeDelta
	FNetDeserializeDeltaArgs DeserializeDeltaArgs;
	{
		Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());
		DeserializeDeltaArgs.Version = SerializerVersionOverride;
		DeserializeDeltaArgs.NetSerializerConfig = &Config;
		DeserializeDeltaArgs.Target = NetSerializerValuePointer(SourceBuffer[0]);
		DeserializeDeltaArgs.Prev = QuantizePrevValueArgs.Target;
		Serializer.DeserializeDelta(Context, DeserializeDeltaArgs);
		FreeScope.AddBuffer(DeserializeDeltaArgs.Target);
		if (Reader.IsOverflown() || Context.HasError())
		{
			UE_NET_EXPECT_FALSE_MSG(Reader.IsOverflown(), "FNetBitStreamReader overflowed.");
			UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "DeserializeDelta() reported an error.");
			return false;
		}
	}

	// Equality check. The quantized representation of the deserialized value should match that of the source.
	{
		FNetIsEqualArgs IsEqualArgs;
		IsEqualArgs.Version = SerializerVersionOverride;
		IsEqualArgs.NetSerializerConfig = &Config;
		IsEqualArgs.Source0 = SerializeDeltaArgs.Source;
		IsEqualArgs.Source1 = DeserializeDeltaArgs.Target;
		IsEqualArgs.bStateIsQuantized = true;
		const bool bResult = Serializer.IsEqual(Context, IsEqualArgs);
		return bResult;
	}
}

bool FTestNetSerializerFixture::TestIsEqual(const FNetSerializerConfig& Config, NetSerializerValuePointer Source0, NetSerializerValuePointer Source1, bool bExpectedResult, bool bQuantizeValues)
{
	const uint32 PrepareTestFlags = (bTaintBuffersBeforeTest && bQuantizeValues ? (TaintQuantizedBuffer0 | TaintQuantizedBuffer1) : 0U);
	PrepareTest(PrepareTestFlags);

	FFreeBufferScope FreeScope(*this, &Config);

	FNetIsEqualArgs IsEqualArgs;
	IsEqualArgs.Version = SerializerVersionOverride;
	IsEqualArgs.NetSerializerConfig = &Config;
	IsEqualArgs.bStateIsQuantized = bQuantizeValues;
	if (bQuantizeValues)
	{
		FNetQuantizeArgs QuantizeArgs;
		QuantizeArgs.Version = IsEqualArgs.Version;
		QuantizeArgs.NetSerializerConfig = IsEqualArgs.NetSerializerConfig;

		QuantizeArgs.Source = Source0;
		QuantizeArgs.Target = NetSerializerValuePointer(QuantizedBuffer[0]);
		Serializer.Quantize(Context, QuantizeArgs);
		FreeScope.AddBuffer(QuantizeArgs.Target);
		IsEqualArgs.Source0 = QuantizeArgs.Target;

		QuantizeArgs.Source = Source1;
		QuantizeArgs.Target = NetSerializerValuePointer(QuantizedBuffer[1]);
		Serializer.Quantize(Context, QuantizeArgs);
		FreeScope.AddBuffer(QuantizeArgs.Target);
		IsEqualArgs.Source1 = QuantizeArgs.Target;
	}
	else
	{
		IsEqualArgs.Source0 = Source0;
		IsEqualArgs.Source1 = Source1;
	}

	const bool bResult1 = Serializer.IsEqual(Context, IsEqualArgs);
	if (bResult1 != bExpectedResult)
		return false;

	Swap(IsEqualArgs.Source0, IsEqualArgs.Source1);
	const bool bResult2 = Serializer.IsEqual(Context, IsEqualArgs);
	if (bResult2 != bExpectedResult)
	{
		UE_NET_EXPECT_EQ_MSG(bResult1, bResult2, "IsEqual did not return the same result when source values were swapped.");
		return false;
	}

	return true;
}

bool FTestNetSerializerFixture::TestValidate(const FNetSerializerConfig& Config, NetSerializerValuePointer Source, bool bExpectedResult)
{
	constexpr uint32 PrepareTestFlags = 0U;
	PrepareTest(PrepareTestFlags);

	FNetValidateArgs ValidateArgs;
	ValidateArgs.Version = SerializerVersionOverride;
	ValidateArgs.NetSerializerConfig = &Config;
	ValidateArgs.Source = Source;

	const bool bResult = Serializer.Validate(Context, ValidateArgs);
	return bResult == bExpectedResult;
}

bool FTestNetSerializerFixture::TestCloneDynamicState(const FNetSerializerConfig& Config, NetSerializerValuePointer Value)
{
	// Taint the clone target buffer!
	constexpr uint32 PrepareTestFlags = TaintQuantizedBuffer1;
	PrepareTest(PrepareTestFlags);

	FFreeBufferScope FreeScope(*this, &Config);

	FNetQuantizeArgs QuantizeArgs;
	QuantizeArgs.Version = SerializerVersionOverride;
	QuantizeArgs.NetSerializerConfig = &Config;
	QuantizeArgs.Source = Value;
	QuantizeArgs.Target = NetSerializerValuePointer(QuantizedBuffer[0]);
	Serializer.Quantize(Context, QuantizeArgs);
	FreeScope.AddBuffer(QuantizeArgs.Target);
	if (Context.HasError())
	{
		UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "Quantize() of value reported an error.");
		return false;
	}

	FNetCloneDynamicStateArgs CloneArgs;
	CloneArgs.Version = SerializerVersionOverride;
	CloneArgs.NetSerializerConfig = NetSerializerConfigParam(&Config);
	CloneArgs.Source = NetSerializerValuePointer(QuantizedBuffer[0]);
	CloneArgs.Target = NetSerializerValuePointer(QuantizedBuffer[1]);
	FreeScope.AddBuffer(CloneArgs.Target);
	Serializer.CloneDynamicState(Context, CloneArgs);
	if (Context.HasError())
	{
		UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "CloneDynamicState() reported an error.");
		return false;
	}

	// Free the original quantized state. If the cloning is misbehaving, such as only copying, it will probably crash after this.
	{
		FNetFreeDynamicStateArgs FreeArgs;
		FreeArgs.Version = SerializerVersionOverride;
		FreeArgs.NetSerializerConfig = NetSerializerConfigParam(&Config);
		FreeArgs.Source = QuantizeArgs.Target;
		Serializer.FreeDynamicState(Context, FreeArgs);
		if (Context.HasError())
		{
			UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "FreeDynamicState() reported an error.");
			return false;
		}
	}

	// Dequantize the cloned state and compare against the original.
	{
		FNetDequantizeArgs DequantizeArgs;
		DequantizeArgs.Version = SerializerVersionOverride;
		DequantizeArgs.NetSerializerConfig = &Config;
		DequantizeArgs.Source = CloneArgs.Target;
		DequantizeArgs.Target = NetSerializerValuePointer(SourceBuffer[0]);
		Serializer.Dequantize(Context, DequantizeArgs);
		if (Context.HasError())
		{
			UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "Dequantize() of quantized value reported an error.");
			return false;
		}

		FNetIsEqualArgs DequantizedIsEqualArgs;
		DequantizedIsEqualArgs.Version = SerializerVersionOverride;
		DequantizedIsEqualArgs.NetSerializerConfig = &Config;
		DequantizedIsEqualArgs.Source0 = Value;
		DequantizedIsEqualArgs.Source1 = DequantizeArgs.Target;
		DequantizedIsEqualArgs.bStateIsQuantized = false;
		const bool bIsDequantizedEqual = Serializer.IsEqual(Context, DequantizedIsEqualArgs);
		if (!bIsDequantizedEqual)
		{
			UE_NET_EXPECT_TRUE_MSG(bIsDequantizedEqual, "IsEqual() on dequantized value compared to source value returned false.");
			return false;
		}
	}

	return true;
}

bool FTestNetSerializerFixture::Serialize(const FNetSerializerConfig& Config, NetSerializerValuePointer Value)
{
	const uint32 PrepareTestFlags = (bTaintBuffersBeforeTest ? (TaintBitStreamBuffer | TaintQuantizedBuffer0) : 0U);
	PrepareTest(PrepareTestFlags);

	FFreeBufferScope FreeScope(*this, &Config);

	FNetQuantizeArgs QuantizeArgs;
	QuantizeArgs.Version = SerializerVersionOverride;
	QuantizeArgs.NetSerializerConfig = &Config;
	QuantizeArgs.Source = Value;
	QuantizeArgs.Target = NetSerializerValuePointer(QuantizedBuffer[0]);
	Serializer.Quantize(Context, QuantizeArgs);
	FreeScope.AddBuffer(QuantizeArgs.Target);
	if (Context.HasError())
	{
		UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "Quantize() of value reported an error.");
		return false;
	}

	Writer.InitBytes(BitStreamBuffer, sizeof(BitStreamBuffer));
	FNetSerializeArgs SerializeArgs;
	SerializeArgs.Version = SerializerVersionOverride;
	SerializeArgs.NetSerializerConfig = &Config;
	SerializeArgs.Source = QuantizeArgs.Target;
	Serializer.Serialize(Context, SerializeArgs);
	Writer.CommitWrites();
	if (Writer.IsOverflown() || Context.HasError())
	{
		UE_NET_EXPECT_FALSE_MSG(Writer.IsOverflown(), "FNetBitStreamWriter overflowed.");
		UE_NET_EXPECT_FALSE_MSG(Context.HasError(), "Serialize() reported an error.");
		return false;
	}

	return true;
}

}
