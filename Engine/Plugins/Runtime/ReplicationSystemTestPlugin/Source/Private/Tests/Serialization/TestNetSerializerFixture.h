// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Templates/IsPODType.h"
#include <type_traits>

namespace UE::Net
{

class FTestNetSerializerFixture : public FNetworkAutomationTestSuiteFixture
{
public:
	FTestNetSerializerFixture(const FNetSerializer& Serializer);

	void OverrideNetSerializerVersion(uint32 Version) { SerializerVersionOverride = Version; }
	void SetDoTaintBuffersBeforeTest(bool bTaintBuffers) { bTaintBuffersBeforeTest = bTaintBuffers; }
	
	bool TestQuantize(const FNetSerializerConfig& Config, NetSerializerValuePointer Value);
	bool TestSerialize(const FNetSerializerConfig& Config, NetSerializerValuePointer Value, NetSerializerValuePointer ExpectedValue, bool bQuantizedCompare, TOptional<TFunctionRef<bool(NetSerializerValuePointer, NetSerializerValuePointer)>> CompareFunc = {});
	bool TestSerializeDelta(const FNetSerializerConfig& Config, NetSerializerValuePointer Value, NetSerializerValuePointer PrevValue);
	bool TestIsEqual(const FNetSerializerConfig& Config, NetSerializerValuePointer Source0, NetSerializerValuePointer Source1, bool bExpectedResult, bool bQuantizeValues);
	bool TestValidate(const FNetSerializerConfig& Config, NetSerializerValuePointer Source, bool bExpectedResult);
	bool TestCloneDynamicState(const FNetSerializerConfig& Config, NetSerializerValuePointer Value);

	// Quantizes value and then serializes it to the Writer. Returns true if the serialization succeeded, false otherwise.
	bool Serialize(const FNetSerializerConfig& Config, NetSerializerValuePointer Source);

private:
	enum PrepareTestFlags : uint32
	{
		TaintBitStreamBuffer = 1U << 0U,
		TaintQuantizedBuffer0 = 1U << 1U,
		TaintQuantizedBuffer1 = 1U << 2U,
		TaintSourceBuffer0 = 1U << 3U,
		TaintSourceBuffer1 = 1U << 4U,
	};

	void PrepareTest(uint32 Flags);

protected:
	class FFreeBufferScope;
	friend class FFreeBufferScope;

	const FNetSerializer& Serializer;
	uint32 SerializerVersionOverride;

	FNetBitStreamReader Reader;
	FNetBitStreamWriter Writer;

	FNetSerializationContext Context;

	enum : uint32
	{
		BitStreamBufferSize = 8192,
		BufferSize = 1024,
	};

	alignas(16) uint8 BitStreamBuffer[BitStreamBufferSize] = {};
	alignas(16) uint8 QuantizedBuffer[2][BufferSize] = {};
	alignas(16) uint8 SourceBuffer[2][BufferSize] = {};

	bool bTaintBuffersBeforeTest = false;
};

typedef FTestMessage&(*NetSerializerConfigPrinter)(FTestMessage& Message, const FNetSerializerConfig& Config);

template<NetSerializerConfigPrinter ConfigPrinter, typename SourceType>
class TTestNetSerializerFixture : public FTestNetSerializerFixture
{
public:
	TTestNetSerializerFixture(const FNetSerializer& Serializer)
	: FTestNetSerializerFixture(Serializer)
	{
		if (!TIsPODType<SourceType>::Value)
		{
			new (SourceBuffer[0]) SourceType;
			new (SourceBuffer[1]) SourceType;
		}
	}

	~TTestNetSerializerFixture()
	{
		if constexpr (!std::is_trivially_destructible_v<SourceType>)
		{
			reinterpret_cast<SourceType*>(SourceBuffer[0])->~SourceType();
			reinterpret_cast<SourceType*>(SourceBuffer[1])->~SourceType();
		}
	}

	bool TestQuantize(const SourceType* Values, SIZE_T ValueCount, const FNetSerializerConfig& Config);
	bool TestSerialize(const SourceType* Values, const SourceType* ExpectedValues, SIZE_T ValueCount, const FNetSerializerConfig& Config, bool bQuantizedCompare, TOptional<TFunctionRef<bool(NetSerializerValuePointer, NetSerializerValuePointer)>> CompareFunc = {});
	bool TestSerializeDelta(const SourceType* Values, SIZE_T ValueCount, const FNetSerializerConfig& Config);
	bool TestIsEqual(const SourceType* Values0, const SourceType* Values1, const bool* ExpectedResults, SIZE_T ValueCount, const FNetSerializerConfig& Config, bool bQuantizedCompare);
	bool TestValidate(const SourceType* Values, const bool* ExpectedResults, SIZE_T ValueCount, const FNetSerializerConfig& Config);
	bool TestCloneDynamicState(const SourceType* Values, SIZE_T ValueCount, const FNetSerializerConfig& Config);
};

template<NetSerializerConfigPrinter ConfigPrinter, typename SourceType>
bool TTestNetSerializerFixture<ConfigPrinter, SourceType>::TestQuantize(const SourceType* Values, SIZE_T ValueCount, const FNetSerializerConfig& Config)
{
	for (SIZE_T ValueIt = 0, ValueEndIt = ValueCount; ValueIt != ValueEndIt; ++ValueIt)
	{
		const SourceType* Value = Values + ValueIt;

		// Must wrap the call in a function returning void due to how the assert macros are implemented.
		const auto& TestQuantizationWrapper = [this](const FNetSerializerConfig& Config, const SourceType* Value, bool& bOutTestCaseSuccess) -> void
		{
			FTestMessage ConfigMessage;
			bOutTestCaseSuccess = FTestNetSerializerFixture::TestQuantize(Config, NetSerializerValuePointer(Value));
			UE_NET_ASSERT_TRUE_MSG(bOutTestCaseSuccess, "Quantization failed with Value: '" << *Value << "' " << ConfigPrinter(ConfigMessage, Config));
		};

		bool bOutTestCaseSuccess = false;
		TestQuantizationWrapper(Config, Value, bOutTestCaseSuccess);
		if (!bOutTestCaseSuccess)
		{
			return false;
		}
	}

	return true;
}

template<NetSerializerConfigPrinter ConfigPrinter, typename SourceType>
bool TTestNetSerializerFixture<ConfigPrinter, SourceType>::TestSerialize(const SourceType* Values, const SourceType* ExpectedValues, SIZE_T ValueCount, const FNetSerializerConfig& Config, bool bQuantizedCompare, TOptional<TFunctionRef<bool(NetSerializerValuePointer, NetSerializerValuePointer)>> CompareFunc)
{
	for (SIZE_T ValueIt = 0, ValueEndIt = ValueCount; ValueIt != ValueEndIt; ++ValueIt)
	{
		const SourceType* Value = Values + ValueIt;
		const SourceType* ExpectedValue = ExpectedValues + ValueIt;

		// Must wrap the call in a function returning void due to how the assert macros are implemented.
		const auto& TestSerializationWrapper = [this](const FNetSerializerConfig& Config, const SourceType* Value, const SourceType* ExpectedValue, bool bQuantizedCompare, TOptional<TFunctionRef<bool (NetSerializerValuePointer, NetSerializerValuePointer)>> CompareFunc, bool& bOutTestCaseSuccess) -> void
		{
			FTestMessage ConfigMessage;
			bOutTestCaseSuccess = FTestNetSerializerFixture::TestSerialize(Config, NetSerializerValuePointer(Value), NetSerializerValuePointer(ExpectedValue), bQuantizedCompare, CompareFunc);
			UE_NET_ASSERT_TRUE_MSG(bOutTestCaseSuccess, "Serialization failed with Value: '" << *Value << "' and ExpectedValue: '" << *ExpectedValue << "' " << ConfigPrinter(ConfigMessage, Config));
		};

		bool bOutTestCaseSuccess = false;
		TestSerializationWrapper(Config, Value, ExpectedValue, bQuantizedCompare, CompareFunc, bOutTestCaseSuccess);
		if (!bOutTestCaseSuccess)
		{
			return false;
		}
	}

	return true;
}

// TestSerializeDelta will test to delta compress each value against every value, a total of ValueCount*ValueCount number of tests.
template<NetSerializerConfigPrinter ConfigPrinter, typename SourceType>
bool TTestNetSerializerFixture<ConfigPrinter, SourceType>::TestSerializeDelta(const SourceType* Values, SIZE_T ValueCount, const FNetSerializerConfig& Config)
{
	for (SIZE_T ValueIt = 0, ValueEndIt = ValueCount; ValueIt != ValueEndIt; ++ValueIt)
	{
		const SourceType* Value = Values + ValueIt;
		for (SIZE_T PrevValueIt = 0, PrevValueEndIt = ValueCount; PrevValueIt != PrevValueEndIt; ++PrevValueIt)
		{
			const SourceType* PrevValue = Values + PrevValueIt;

			// Must wrap the call in a function returning void due to how the assert macros are implemented.
			const auto& TestSerializeDeltaWrapper = [this](const FNetSerializerConfig& Config, const SourceType* Value, const SourceType* PrevValue, bool& bOutTestCaseSuccess) -> void
			{
				FTestMessage ConfigMessage;
				bOutTestCaseSuccess = FTestNetSerializerFixture::TestSerializeDelta(Config, NetSerializerValuePointer(Value), NetSerializerValuePointer(PrevValue));
				UE_NET_ASSERT_TRUE_MSG(bOutTestCaseSuccess, "Delta serialization failed with Value: '" << *Value << "' and PrevValue: '" << *PrevValue << "' " << ConfigPrinter(ConfigMessage, Config));
			};

			bool bOutTestCaseSuccess = false;
			TestSerializeDeltaWrapper(Config, Value, PrevValue, bOutTestCaseSuccess);
			if (!bOutTestCaseSuccess)
			{
				return false;
			}
		}
	}

	return true;
}

template<NetSerializerConfigPrinter ConfigPrinter, typename SourceType>
bool TTestNetSerializerFixture<ConfigPrinter, SourceType>::TestIsEqual(const SourceType* Values0, const SourceType* Values1, const bool* ExpectedResults, SIZE_T ValueCount, const FNetSerializerConfig& Config, bool bQuantizedCompare)
{
	for (SIZE_T ValueIt = 0, ValueEndIt = ValueCount; ValueIt != ValueEndIt; ++ValueIt)
	{
		// Must wrap the call in a function returning void due to how the assert macros are implemented.
		const auto& TestIsEqualWrapper = [this](const FNetSerializerConfig& Config, const SourceType* Value0, const SourceType* Value1, bool bExpectedResult, bool bQuantizedCompare, bool& bOutTestCaseSuccess) -> void
		{
			FTestMessage ConfigMessage;
			bOutTestCaseSuccess = FTestNetSerializerFixture::TestIsEqual(Config, NetSerializerValuePointer(Value0), NetSerializerValuePointer(Value1), bExpectedResult, bQuantizedCompare);
			UE_NET_ASSERT_TRUE_MSG(bOutTestCaseSuccess, "IsEqual failed. Value0: '" << *Value0 << "' Value1: '" << *Value1 << "' " << ConfigPrinter(ConfigMessage, Config) << ". Expected IsEqual to return " << bExpectedResult);
		};

		const SourceType* Value0 = Values0 + ValueIt;
		const SourceType* Value1 = Values1 + ValueIt;
		const bool bExpectedResult = ExpectedResults[ValueIt];
		bool bOutTestCaseSuccess = false;
		TestIsEqualWrapper(Config, Value0, Value1, bExpectedResult, bQuantizedCompare, bOutTestCaseSuccess);
		if (!bOutTestCaseSuccess)
		{
			return false;
		}
	}

	return true;
}

template<NetSerializerConfigPrinter ConfigPrinter, typename SourceType>
bool TTestNetSerializerFixture<ConfigPrinter, SourceType>::TestValidate(const SourceType* Values, const bool* ExpectedResults, SIZE_T ValueCount, const FNetSerializerConfig& Config)
{
	for (SIZE_T ValueIt = 0, ValueEndIt = ValueCount; ValueIt != ValueEndIt; ++ValueIt)
	{
		// Must wrap the call in a function returning void due to how the assert macros are implemented.
		const auto& TestValidateWrapper = [this](const FNetSerializerConfig& Config, const SourceType* Value, bool bExpectedResult, bool& bOutTestCaseSuccess) -> void
		{
			FTestMessage ConfigMessage;
			bOutTestCaseSuccess = FTestNetSerializerFixture::TestValidate(Config, NetSerializerValuePointer(Value), bExpectedResult);
			UE_NET_ASSERT_TRUE_MSG(bOutTestCaseSuccess, "Validate failed. Value: '" << *Value << "' " << ConfigPrinter(ConfigMessage, Config) << ". Expected Validate to return " << bExpectedResult);
		};

		const SourceType* Value = Values + ValueIt;
		const bool bExpectedResult = ExpectedResults[ValueIt];
		bool bOutTestCaseSuccess = false;
		TestValidateWrapper(Config, Value, bExpectedResult, bOutTestCaseSuccess);
		if (!bOutTestCaseSuccess)
		{
			return false;
		}
	}

	return true;
}

template<NetSerializerConfigPrinter ConfigPrinter, typename SourceType>
bool TTestNetSerializerFixture<ConfigPrinter, SourceType>::TestCloneDynamicState(const SourceType* Values, SIZE_T ValueCount, const FNetSerializerConfig& Config)
{
	for (SIZE_T ValueIt = 0, ValueEndIt = ValueCount; ValueIt != ValueEndIt; ++ValueIt)
	{
		const SourceType* Value = Values + ValueIt;

		// Must wrap the call in a function returning void due to how the assert macros are implemented.
		const auto& TestCloneDynamicStateWrapper = [this](const FNetSerializerConfig& Config, const SourceType* Value, bool& bOutTestCaseSuccess) -> void
		{
			FTestMessage ConfigMessage;
			bOutTestCaseSuccess = FTestNetSerializerFixture::TestCloneDynamicState(Config, NetSerializerValuePointer(Value));
			UE_NET_ASSERT_TRUE_MSG(bOutTestCaseSuccess, "CloneDynamicState failed with Value: '" << *Value << "' " << ConfigPrinter(ConfigMessage, Config));
		};

		bool bOutTestCaseSuccess = false;
		TestCloneDynamicStateWrapper(Config, Value, bOutTestCaseSuccess);
		if (!bOutTestCaseSuccess)
		{
			return false;
		}
	}

	return true;
}

}
