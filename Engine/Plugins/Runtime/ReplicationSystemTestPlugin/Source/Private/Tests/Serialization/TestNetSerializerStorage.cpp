// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/Serialization/NetSerializerArrayStorage.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"

namespace UE::Net::Private
{

class FNetSerializerStorageTest : public FNetworkAutomationTestSuiteFixture
{
public:
	FNetSerializerStorageTest() : FNetworkAutomationTestSuiteFixture()
	{
		Context.SetInternalContext(&InternalContext);
	}

	static constexpr uint32 NumInlinedElements = 4;

	using TestDataType = uint32;
	
	using FArrayStorage = UE::Net::FNetSerializerArrayStorage<TestDataType>;
	using FInlinedArrayStorage = UE::Net::FNetSerializerArrayStorage<TestDataType, UE::Net::AllocationPolicies::TInlinedElementAllocationPolicy<NumInlinedElements>>;

	struct TestState
	{
		FArrayStorage Storage;
		FInlinedArrayStorage InlinedStorage;
	};

	template <typename T>
	void InitializeWithTestData(T& ArrayStorage, const TestDataType* Data, uint32 Num)
	{
		ArrayStorage.AdjustSize(Context, Num);
		FMemory::Memcpy(ArrayStorage.GetData(), Data, sizeof(TestDataType)*Num);
	}

	template <typename T>
	bool Validate(const T& ArrayStorage, const TestDataType* Data, uint32 Num)
	{
		if (ArrayStorage.Num() < Num)
		{
			return false;
		}

		return FMemory::Memcmp(ArrayStorage.GetData(), Data, sizeof(TestDataType)*Num) == 0U;
	}

	
protected:
	FNetSerializationContext Context;
	FInternalNetSerializationContext InternalContext;	
};

UE_NET_TEST_FIXTURE(FNetSerializerStorageTest, DefaultConstructed)
{
	using namespace UE::Net;

	TestState State;

	UE_NET_ASSERT_EQ(0U, State.Storage.Num());
	UE_NET_ASSERT_EQ(0U, State.InlinedStorage.Num());
}

UE_NET_TEST_FIXTURE(FNetSerializerStorageTest, DefaultConstructedInGarbageMemory)
{
	TTypeCompatibleBytes<TestState> StateData;

	// Set to garbage
	FMemory::Memset(&StateData, uint8(-1), sizeof(TestState));

	// In place new in garbage memory
	TestState* State = (TestState*)(new (&StateData) TestState);

	UE_NET_ASSERT_EQ(0U, State->Storage.Num());
	UE_NET_ASSERT_EQ(0U, State->InlinedStorage.Num());
}

UE_NET_TEST_FIXTURE(FNetSerializerStorageTest, ZeroedStateIsValid)
{
	TTypeCompatibleBytes<TestState> StateData;

	// Init in zeroed memory
	FMemory::Memzero(&StateData, sizeof(TestState));
	TestState* State = StateData.GetTypedPtr();

	UE_NET_ASSERT_EQ(0U, State->Storage.Num());
	UE_NET_ASSERT_EQ(0U, State->InlinedStorage.Num());
}

UE_NET_TEST_FIXTURE(FNetSerializerStorageTest, FreeOnZeroedState)
{
	TTypeCompatibleBytes<TestState> StateData;

	// Init in zeroed memory
	FMemory::Memzero(&StateData, sizeof(TestState));
	TestState* State = StateData.GetTypedPtr();
	
	State->Storage.Free(Context);
	UE_NET_ASSERT_EQ(0U, State->Storage.Num());

	State->InlinedStorage.Free(Context);
	UE_NET_ASSERT_EQ(0U, State->InlinedStorage.Num());
}

UE_NET_TEST_FIXTURE(FNetSerializerStorageTest, FreeOnDefaultConstructed)
{
	using namespace UE::Net;

	TestState State;
	State.Storage.Free(Context);
	UE_NET_ASSERT_EQ(0U, State.Storage.Num());

	State.InlinedStorage.Free(Context);
	UE_NET_ASSERT_EQ(0U, State.InlinedStorage.Num());
}

UE_NET_TEST_FIXTURE(FNetSerializerStorageTest, AdjustSizeGrowPreservesExistingData)
{
	static const TestDataType TestDataArray[] = {1U, 2U, 3U, 4U, 5U ,6U ,7U ,8U};
	
	TestState State;

	FArrayStorage& Storage = State.Storage;

	Storage.AdjustSize(Context, 3U);
	UE_NET_ASSERT_EQ(3U, Storage.Num());

	// Fill with some data
	InitializeWithTestData(Storage, TestDataArray, 3U);

	// Validate that we have the expected data
	UE_NET_ASSERT_TRUE(Validate(Storage, TestDataArray, 3U));
	Storage.AdjustSize(Context, 8U);
	UE_NET_ASSERT_EQ(8U, Storage.Num());

	// Validate that the expected data still is in storage
	UE_NET_ASSERT_TRUE(Validate(Storage, TestDataArray, 3U));

	// Free allocated data
	Storage.Free(Context);
}

UE_NET_TEST_FIXTURE(FNetSerializerStorageTest, AdjustSizeGrowPreservesExistingFromInlinedElements)
{
	static const TestDataType TestDataArray[] = {1U, 2U, 3U, 4U, 5U ,6U ,7U ,8U};
	
	TestState State;

	FInlinedArrayStorage& Storage = State.InlinedStorage;

	Storage.AdjustSize(Context, 3U);
	UE_NET_ASSERT_EQ(3U, Storage.Num());

	// Fill with some data
	InitializeWithTestData(Storage, TestDataArray, 3U);

	// Validate that we have the expected data
	UE_NET_ASSERT_TRUE(Validate(Storage, TestDataArray, 3U));

	// Adjust size to go from inlined storage to allocated storage
	Storage.AdjustSize(Context, 8U);
	UE_NET_ASSERT_EQ(8U, Storage.Num());

	// Validate that the expected data still is in storage
	UE_NET_ASSERT_TRUE(Validate(Storage, TestDataArray, 3U));

	// Free allocated data
	Storage.Free(Context);
}

UE_NET_TEST_FIXTURE(FNetSerializerStorageTest, AdjustSizeShrinkPreservesExistingExternalElementsWhenMovingToInlinedStorage)
{
	static const TestDataType TestDataArray[] = {1U, 2U, 3U, 4U, 5U ,6U ,7U ,8U};
	
	TestState State;

	FInlinedArrayStorage& Storage = State.InlinedStorage;

	Storage.AdjustSize(Context, 8U);
	UE_NET_ASSERT_EQ(8U, Storage.Num());

	// Fill with some data
	InitializeWithTestData(Storage, TestDataArray, 8U);

	// Validate that we have the expected data
	UE_NET_ASSERT_TRUE(Validate(Storage, TestDataArray, 8U));

	// Adjust size to shrink back into what fits in inlined storage
	Storage.AdjustSize(Context, NumInlinedElements);
	UE_NET_ASSERT_EQ(NumInlinedElements, Storage.Num());

	// Validate that the expected data still is in storage
	UE_NET_ASSERT_TRUE(Validate(Storage, TestDataArray, NumInlinedElements));

	// Free allocated data
	Storage.Free(Context);
}

UE_NET_TEST_FIXTURE(FNetSerializerStorageTest, AdjustSizeShrinkPreservesExistingExternalElements)
{
	static const TestDataType TestDataArray[] = {1U, 2U, 3U, 4U, 5U ,6U ,7U ,8U};
	
	TestState State;

	FArrayStorage& Storage = State.Storage;

	Storage.AdjustSize(Context, 8U);
	UE_NET_ASSERT_EQ(8U, Storage.Num());

	// Fill with some data
	InitializeWithTestData(Storage, TestDataArray, 8U);

	// Validate that we have the expected data
	UE_NET_ASSERT_TRUE(Validate(Storage, TestDataArray, 8U));

	// Adjust size to shrink
	Storage.AdjustSize(Context, 3U);
	UE_NET_ASSERT_EQ(3U, Storage.Num());

	// Validate that the expected data still is in storage
	UE_NET_ASSERT_TRUE(Validate(Storage, TestDataArray, 3U));

	// Free allocated data
	Storage.Free(Context);
}

UE_NET_TEST_FIXTURE(FNetSerializerStorageTest, CloneFromZeroedState)
{
	using namespace UE::Net;

	TestState ZeroedState;
	FMemory::Memzero(ZeroedState);
	TestState TargetState;

	// This is how clone normally is used from a NetSerialzier, first a memcpy followed by call to clone for dynamic allocations
	FMemory::Memcpy(TargetState, ZeroedState);

	UE_NET_ASSERT_EQ(0U, ZeroedState.Storage.Num());
	UE_NET_ASSERT_EQ(0U, ZeroedState.InlinedStorage.Num());

	TargetState.Storage.Clone(Context, ZeroedState.Storage);
	TargetState.InlinedStorage.Clone(Context, ZeroedState.InlinedStorage);

	UE_NET_ASSERT_EQ(TargetState.Storage.Num(), ZeroedState.Storage.Num());
	UE_NET_ASSERT_EQ(TargetState.InlinedStorage.Num(), ZeroedState.InlinedStorage.Num());
}

UE_NET_TEST_FIXTURE(FNetSerializerStorageTest, CloneFromZeroedStateToGarbageState)
{
	using namespace UE::Net;

	TestState ZeroedState;
	TestState TargetState;

	FMemory::Memzero(ZeroedState);
	FMemory::Memset(&TargetState, uint8(-1), sizeof(TestState));
	
	UE_NET_ASSERT_EQ(0U, ZeroedState.Storage.Num());
	UE_NET_ASSERT_EQ(0U, ZeroedState.InlinedStorage.Num());

	TargetState.Storage.Clone(Context, ZeroedState.Storage);
	TargetState.InlinedStorage.Clone(Context, ZeroedState.InlinedStorage);

	UE_NET_ASSERT_EQ(TargetState.Storage.Num(), ZeroedState.Storage.Num());
	UE_NET_ASSERT_EQ(TargetState.InlinedStorage.Num(), ZeroedState.InlinedStorage.Num());
}

UE_NET_TEST_FIXTURE(FNetSerializerStorageTest, CloneFromDynamicStorage)
{
	static const TestDataType TestDataArray[] = {1U, 2U, 3U, 4U, 5U ,6U ,7U ,8U};

	TestState SourceState;
	TestState TargetState;

	// This is how clone normally is used from a NetSerialzier, first a memcpy followed by call to clone for dynamic allocations
	FMemory::Memcpy(TargetState, SourceState);

	FArrayStorage& SourceStorage = SourceState.Storage;
	FArrayStorage& TargetStorage = TargetState.Storage;

	SourceStorage.AdjustSize(Context, 8U);
	InitializeWithTestData(SourceStorage, TestDataArray, 8U);

	// Clone
	TargetStorage.Clone(Context, SourceStorage);

	// Free source
	// Free source, after clearing it to make sure that we are not using the memory by accident in the cloned storage
	FMemory::Memset(SourceStorage.GetData(), 0, SourceStorage.Num() * sizeof(TestDataType));
	SourceStorage.Free(Context);

	// Validate clone after freeing SourceStorage
	UE_NET_ASSERT_TRUE(Validate(TargetStorage, TestDataArray, 8U));

	// Free Target
	TargetStorage.Free(Context);
}

UE_NET_TEST_FIXTURE(FNetSerializerStorageTest, CloneFromInlinedStorage)
{
	static const TestDataType TestDataArray[] = {1U, 2U, 3U, 4U, 5U ,6U ,7U ,8U};

	TestState SourceState;
	TestState TargetState;

	// This is how clone normally is used from a NetSerialzier, first a memcpy followed by call to clone for dynamic allocations
	FMemory::Memcpy(TargetState, SourceState);

	FArrayStorage& SourceStorage = SourceState.Storage;
	FArrayStorage& TargetStorage = TargetState.Storage;

	SourceStorage.AdjustSize(Context, NumInlinedElements);
	InitializeWithTestData(SourceStorage, TestDataArray, NumInlinedElements);

	// Clone
	TargetStorage.Clone(Context, SourceStorage);

	// Validate clone
	UE_NET_ASSERT_TRUE(Validate(TargetStorage, SourceStorage.GetData(), SourceStorage.Num()));

	// Free source, after clearing it to make sure that we are not using the memory by accident in the cloned storage
	FMemory::Memset(SourceStorage.GetData(), 0, SourceStorage.Num() * sizeof(TestDataType));
	SourceStorage.Free(Context);

	// Validate clone after freeing SourceStorage
	UE_NET_ASSERT_TRUE(Validate(TargetStorage, TestDataArray, NumInlinedElements));

	// Free Target
	TargetStorage.Free(Context);
}

}
