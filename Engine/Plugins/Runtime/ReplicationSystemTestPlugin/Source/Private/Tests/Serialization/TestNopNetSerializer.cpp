// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/Serialization/NetSerializers.h"

namespace UE::Net::Private
{

class FTestNopNetSerializerFixture : public FNetworkAutomationTestSuiteFixture
{
public:
	FTestNopNetSerializerFixture() : Serializer(nullptr) {}

protected:
	virtual void SetUp() override;

protected:
	const FNetSerializer* Serializer;
};

void FTestNopNetSerializerFixture::SetUp()
{
	Serializer = &UE_NET_GET_SERIALIZER(FNopNetSerializer);
}

/**
  * These tests are basically just checking that all operations are as no-op they can be.
  * For functions returning things we check they return something "good".
  */
UE_NET_TEST_FIXTURE(FTestNopNetSerializerFixture, SerializerIsSetupCorrectly)
{
	UE_NET_ASSERT_EQ(Serializer->QuantizedTypeSize, uint16(0));
	UE_NET_ASSERT_EQ(Serializer->QuantizedTypeAlignment, uint16(1));

	UE_NET_ASSERT_FALSE(EnumHasAnyFlags(Serializer->Traits, ~ENetSerializerTraits::None));
}

UE_NET_TEST_FIXTURE(FTestNopNetSerializerFixture, Serialize)
{
	FNetSerializationContext Context;

	FNetSerializeArgs Args = {};
	Args.NetSerializerConfig = Serializer->DefaultConfig;
	Serializer->Serialize(Context, Args);
	UE_NET_ASSERT_FALSE(Context.HasError());
}

UE_NET_TEST_FIXTURE(FTestNopNetSerializerFixture, Deserialize)
{
	FNetSerializationContext Context;

	FNetDeserializeArgs Args = {};
	Args.NetSerializerConfig = Serializer->DefaultConfig;
	Serializer->Deserialize(Context, Args);
	UE_NET_ASSERT_FALSE(Context.HasError());
}

UE_NET_TEST_FIXTURE(FTestNopNetSerializerFixture, SerializeDelta)
{
	FNetSerializationContext Context;

	FNetSerializeDeltaArgs Args = {};
	Args.NetSerializerConfig = Serializer->DefaultConfig;
	Serializer->SerializeDelta(Context, Args);
	UE_NET_ASSERT_FALSE(Context.HasError());
}

UE_NET_TEST_FIXTURE(FTestNopNetSerializerFixture, DeserializeDelta)
{
	FNetSerializationContext Context;

	FNetDeserializeDeltaArgs Args = {};
	Args.NetSerializerConfig = Serializer->DefaultConfig;
	Serializer->DeserializeDelta(Context, Args);
	UE_NET_ASSERT_FALSE(Context.HasError());
}

UE_NET_TEST_FIXTURE(FTestNopNetSerializerFixture, Quantize)
{
	FNetSerializationContext Context;

	FNetQuantizeArgs Args = {};
	Args.NetSerializerConfig = Serializer->DefaultConfig;
	Serializer->Quantize(Context, Args);
	UE_NET_ASSERT_FALSE(Context.HasError());
}

UE_NET_TEST_FIXTURE(FTestNopNetSerializerFixture, Dequantize)
{
	FNetSerializationContext Context;

	FNetDequantizeArgs Args = {};
	Args.NetSerializerConfig = Serializer->DefaultConfig;
	Serializer->Dequantize(Context, Args);
	UE_NET_ASSERT_FALSE(Context.HasError());
}

UE_NET_TEST_FIXTURE(FTestNopNetSerializerFixture, IsEqual)
{
	FNetSerializationContext Context;

	FNetIsEqualArgs Args = {};
	Args.NetSerializerConfig = Serializer->DefaultConfig;
	const bool bIsEqual = Serializer->IsEqual(Context, Args);
	UE_NET_ASSERT_TRUE(bIsEqual);
	UE_NET_ASSERT_FALSE(Context.HasError());
}

UE_NET_TEST_FIXTURE(FTestNopNetSerializerFixture, Validate)
{
	FNetSerializationContext Context;

	FNetValidateArgs Args = {};
	Args.NetSerializerConfig = Serializer->DefaultConfig;
	const bool bIsValid = Serializer->Validate(Context, Args);
	UE_NET_ASSERT_TRUE(bIsValid);
	UE_NET_ASSERT_FALSE(Context.HasError());
}

UE_NET_TEST_FIXTURE(FTestNopNetSerializerFixture, CloneDynamicState)
{
	UE_NET_ASSERT_EQ(Serializer->CloneDynamicState, NetCloneDynamicStateFunction(nullptr));
}

UE_NET_TEST_FIXTURE(FTestNopNetSerializerFixture, FreeDynamicState)
{
	UE_NET_ASSERT_EQ(Serializer->FreeDynamicState, NetFreeDynamicStateFunction(nullptr));
}

}
