// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/ObjectNetSerializer.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Core/NetObjectReference.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Templates/UnrealTemplate.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"

namespace UE::Net::Private
{

static FTestMessage& PrintObjectNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestObjectReference)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestReplicatedIrisObjectWithObjectReference* ObjectD = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();
	UTestReplicatedIrisObjectWithObjectReference* ObjectC = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();
	UTestReplicatedIrisObjectWithObjectReference* ObjectB = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();
	UTestReplicatedIrisObjectWithObjectReference* ObjectA = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();

	ObjectA->RawObjectPtrRef = ObjectB;
	ObjectB->RawObjectPtrRef = ObjectC;

	// This will not work since we do not have any dependencies
	ObjectD->RawObjectPtrRef = ObjectA;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Check state, we cheat a bit and use the server NetRefHandle for lookup
	// THIS will fail if we start using full handle compares
	auto ClientObjectA = Client->GetObjectAs<UTestReplicatedIrisObjectWithObjectReference>(ObjectA->NetRefHandle);
	auto ClientObjectB = Client->GetObjectAs<UTestReplicatedIrisObjectWithObjectReference>(ObjectB->NetRefHandle);
	auto ClientObjectC = Client->GetObjectAs<UTestReplicatedIrisObjectWithObjectReference>(ObjectC->NetRefHandle);
	auto ClientObjectD = Client->GetObjectAs<UTestReplicatedIrisObjectWithObjectReference>(ObjectD->NetRefHandle);
	
	UE_NET_ASSERT_NE(ClientObjectA, nullptr);
	UE_NET_ASSERT_NE(ClientObjectB, nullptr);
	UE_NET_ASSERT_NE(ClientObjectC, nullptr);
	UE_NET_ASSERT_NE(ClientObjectD, nullptr);

	// Verify that we managed to resolved the references
	UE_NET_ASSERT_EQ((UObject*)ClientObjectA->RawObjectPtrRef, (UObject*)ClientObjectB);
	UE_NET_ASSERT_EQ((UObject*)ClientObjectB->RawObjectPtrRef, (UObject*)ClientObjectC);
	UE_NET_ASSERT_EQ((UObject*)ClientObjectC->RawObjectPtrRef, (UObject*)nullptr);
	UE_NET_ASSERT_EQ((UObject*)ClientObjectD->RawObjectPtrRef, (UObject*)ClientObjectA);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestCircularReference)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestReplicatedIrisObjectWithObjectReference* ObjectD = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();
	UTestReplicatedIrisObjectWithObjectReference* ObjectC = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();
	UTestReplicatedIrisObjectWithObjectReference* ObjectB = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();
	UTestReplicatedIrisObjectWithObjectReference* ObjectA = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();

	ObjectA->RawObjectPtrRef = ObjectB;
	ObjectB->RawObjectPtrRef = ObjectC;
	ObjectC->RawObjectPtrRef = ObjectD;
	ObjectD->RawObjectPtrRef = ObjectA;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Check state, we cheat a bit and use the server NetRefHandle for lookup
	// THIS will fail if we start using full handle compares
	auto ClientObjectA = Client->GetObjectAs<UTestReplicatedIrisObjectWithObjectReference>(ObjectA->NetRefHandle);
	auto ClientObjectB = Client->GetObjectAs<UTestReplicatedIrisObjectWithObjectReference>(ObjectB->NetRefHandle);
	auto ClientObjectC = Client->GetObjectAs<UTestReplicatedIrisObjectWithObjectReference>(ObjectC->NetRefHandle);
	auto ClientObjectD = Client->GetObjectAs<UTestReplicatedIrisObjectWithObjectReference>(ObjectD->NetRefHandle);
	
	UE_NET_ASSERT_NE(ClientObjectA, nullptr);
	UE_NET_ASSERT_NE(ClientObjectB, nullptr);
	UE_NET_ASSERT_NE(ClientObjectC, nullptr);
	UE_NET_ASSERT_NE(ClientObjectD, nullptr);

	// Verify that we managed to resolved the references
	UE_NET_ASSERT_EQ((UObject*)ClientObjectA->RawObjectPtrRef, (UObject*)ClientObjectB);
	UE_NET_ASSERT_EQ((UObject*)ClientObjectB->RawObjectPtrRef, (UObject*)ClientObjectC);
	UE_NET_ASSERT_EQ((UObject*)ClientObjectC->RawObjectPtrRef, (UObject*)ClientObjectD);
	UE_NET_ASSERT_EQ((UObject*)ClientObjectD->RawObjectPtrRef, (UObject*)ClientObjectA);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestCircularReferenceWithLimitedThroughput)
{
	// Limit packet size
	Server->SetMaxSendPacketSize(136U);

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestReplicatedIrisObjectWithObjectReference* ObjectD = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();
	UTestReplicatedIrisObjectWithObjectReference* ObjectC = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();
	UTestReplicatedIrisObjectWithObjectReference* ObjectB = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();
	UTestReplicatedIrisObjectWithObjectReference* ObjectA = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();

	ObjectA->RawObjectPtrRef = ObjectB;
	ObjectB->RawObjectPtrRef = ObjectC;
	ObjectC->RawObjectPtrRef = ObjectD;
	ObjectD->RawObjectPtrRef = ObjectA;

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Send and deliver packets
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Check state, we cheat a bit and use the server NetRefHandle for lookup
	// THIS will fail if we start using full handle compares
	auto ClientObjectA = Client->GetObjectAs<UTestReplicatedIrisObjectWithObjectReference>(ObjectA->NetRefHandle);
	auto ClientObjectB = Client->GetObjectAs<UTestReplicatedIrisObjectWithObjectReference>(ObjectB->NetRefHandle);
	auto ClientObjectC = Client->GetObjectAs<UTestReplicatedIrisObjectWithObjectReference>(ObjectC->NetRefHandle);
	auto ClientObjectD = Client->GetObjectAs<UTestReplicatedIrisObjectWithObjectReference>(ObjectD->NetRefHandle);
	
	UE_NET_ASSERT_NE(ClientObjectA, nullptr);
	UE_NET_ASSERT_NE(ClientObjectB, nullptr);
	UE_NET_ASSERT_NE(ClientObjectC, nullptr);
	UE_NET_ASSERT_NE(ClientObjectD, nullptr);

	// Verify that we managed to resolved the references
	UE_NET_ASSERT_EQ((UObject*)ClientObjectA->RawObjectPtrRef, (UObject*)ClientObjectB);
	UE_NET_ASSERT_EQ((UObject*)ClientObjectB->RawObjectPtrRef, (UObject*)ClientObjectC);
	UE_NET_ASSERT_EQ((UObject*)ClientObjectC->RawObjectPtrRef, (UObject*)ClientObjectD);
	UE_NET_ASSERT_EQ((UObject*)ClientObjectD->RawObjectPtrRef, (UObject*)ClientObjectA);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestSoftObjectReferenceToDynamicObject)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestReplicatedIrisObjectWithObjectReference* ObjectB = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();
	UTestReplicatedIrisObjectWithObjectReference* ObjectA = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();

	ObjectA->SoftObjectPtrRef = ObjectB;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Check state, we cheat a bit and use the server NetRefHandle for lookup
	// THIS will fail if we start using full handle compares
	auto ClientObjectA = Client->GetObjectAs<UTestReplicatedIrisObjectWithObjectReference>(ObjectA->NetRefHandle);
	auto ClientObjectB = Client->GetObjectAs<UTestReplicatedIrisObjectWithObjectReference>(ObjectB->NetRefHandle);

	UE_NET_ASSERT_NE(ClientObjectA, nullptr);
	UE_NET_ASSERT_NE(ClientObjectB, nullptr);

	// Verify that we managed to resolved the reference
	UE_NET_ASSERT_EQ((UObject*)ClientObjectA->SoftObjectPtrRef.Get(), (UObject*)ClientObjectB);
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestGarbageObjectPtrReference)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestReplicatedIrisObjectWithObjectReference* ObjectA = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();
	UTestReplicatedIrisObject* ObjectB = Server->CreateObject<UTestReplicatedIrisObject>();

	void* FakeObject = FMemory::Malloc(sizeof(UTestReplicatedIrisObject), alignof(UTestReplicatedIrisObject));
	FMemory::Memcpy(FakeObject, ObjectB, sizeof(UTestReplicatedIrisObject));
	FObjectPtr TestInvalidObjectPtr_Untyped((UObject*)FakeObject);
	TObjectPtr<UObject> TestInvalidObjectPtr(TestInvalidObjectPtr_Untyped);

	ObjectA->RawObjectPtrRef = TestInvalidObjectPtr;

	// Send and deliver packet
	Server->UpdateAndSend({Client});

	ObjectA->RawObjectPtrRef = ObjectB;

	// Invalidate fake object
	FMemory::Memzero(FakeObject, sizeof(UTestReplicatedIrisObject));
	FMemory::Free(FakeObject);

	// Force TObjectPtr creation to mark objects as reachable
	TGuardValue<bool> IsIncrementalReachabilityPending(UE::GC::GIsIncrementalReachabilityPending, true);

	// Send and deliver packet. If a TObjectPtr is created this may crash if for example running with -stompmalloc or assert if running with checks enabled.
	Server->UpdateAndSend({Client});
}

UE_NET_TEST_FIXTURE(FReplicationSystemServerClientTestFixture, TestTypedObjectReference)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Object on which we conduct the test
	UTestReplicatedIrisObjectWithObjectReference* ServerObject = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();

	// Spawn different types of objects on server
	UTestReplicatedIrisObjectWithObjectReference* ReplicatedIrisObjectWithObjectReference = Server->CreateObject<UTestReplicatedIrisObjectWithObjectReference>();
	UTestReplicatedIrisObject* ReplicatedIrisObject = Server->CreateObject<UTestReplicatedIrisObject>();

	// Untyped should always work
	ServerObject->RawObjectPtrRef = ReplicatedIrisObject;
	ServerObject->WeakObjectPtrObjectRef = ReplicatedIrisObject;
	ServerObject->SoftObjectPtrRef = ReplicatedIrisObject;

	// HACK: to set invalid types.
 	// We expect this to serialize as an invalid object reference
	*(TObjectPtr<UTestReplicatedIrisObject>*)(&ServerObject->TypedRawObjectPtrRef) = ReplicatedIrisObject;
	*(TWeakObjectPtr<UTestReplicatedIrisObject>*)(&ServerObject->TypedWeakObjectPtrObjectRef) = ReplicatedIrisObject;
	*(TSoftObjectPtr<UTestReplicatedIrisObject>*)(&ServerObject->TypedSoftObjectPtrRef) = ReplicatedIrisObject;

	// Send and deliver packet
	{
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Error);
		Server->UpdateAndSend({Client});
	}

	UTestReplicatedIrisObjectWithObjectReference* ClientObject = Client->GetObjectAs<UTestReplicatedIrisObjectWithObjectReference>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// We expect these to be resolvable
	UE_NET_ASSERT_NE(ClientObject->RawObjectPtrRef.Get(), nullptr);
	UE_NET_ASSERT_NE(ClientObject->WeakObjectPtrObjectRef.Get(), nullptr);
	UE_NET_ASSERT_NE(ClientObject->SoftObjectPtrRef.Get(), nullptr);

	// We expect these to be invalid
	UE_NET_ASSERT_EQ(ClientObject->TypedRawObjectPtrRef.Get(), nullptr);
	UE_NET_ASSERT_EQ(ClientObject->TypedWeakObjectPtrObjectRef.Get(), nullptr);
	UE_NET_ASSERT_EQ(ClientObject->TypedSoftObjectPtrRef.Get(), nullptr);
}


}
