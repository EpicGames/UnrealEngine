// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestObjectsAsRemoteReferencesSerialization.h"
#include "Tests/ReplicationSystem/ReplicationSystemConfigOverrideTestFixture.h"
#include "Tests/Serialization/TestRemoteObjectReferenceNetSerializer.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

void UTestObjectWithReferencesAsRemote::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
}

void UTestObjectWithReferencesAsRemote::ClientRPCWithRawPointer_Implementation(UObject* RawPointer)
{
	LastReceivedRawPointer = RawPointer;
}

#if UE_WITH_REMOTE_OBJECT_HANDLE

namespace UE::Net::Private
{

UE_NET_TEST_FIXTURE(FReplicationSystemConfigOverrideRPCTestFixture, TestObjectsAsRemoteReferences)
{
	FReplicationSystemTestNode::FReplicationSystemParamsOverride Params;
	Params.bUseRemoteObjectReferences = true;

	// Create the server
	CreateServer(Params);

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient(Params);

	// Spawn objects on server
	UTestObjectWithReferencesAsRemote* TestObject = Server->CreateObject<UTestObjectWithReferencesAsRemote>();
	UReplicatedTestObject* ReferencedObject = Server->CreateObject<UReplicatedTestObject>();

	TestObject->bIsServerObject = true;
	TestObject->ReplicationSystem = Server->GetReplicationSystem();
	Server->ReplicationSystem->SetOwningNetConnection(TestObject->NetRefHandle, Client->ConnectionIdOnServer);

	TestObject->ReplicatedTObjectPtr = ReferencedObject;
	TestObject->ReplicatedWeakObjectPtr = ReferencedObject;
	TestObject->ClientRPCWithRawPointer(ReferencedObject);

	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	UTestObjectWithReferencesAsRemote* ClientTestObject = Client->GetObjectAs<UTestObjectWithReferencesAsRemote>(TestObject->NetRefHandle);
	UReplicatedTestObject* ClientReferencedObject = Client->GetObjectAs<UReplicatedTestObject>(ReferencedObject->NetRefHandle);

	UE_NET_ASSERT_NE(ClientTestObject, nullptr);

	// Verify that we received valid references
	UE_NET_ASSERT_TRUE(ClientTestObject->ReplicatedTObjectPtr);
	UE_NET_ASSERT_TRUE(ClientTestObject->ReplicatedWeakObjectPtr.IsValid());
	UE_NET_ASSERT_TRUE(ClientTestObject->LastReceivedRawPointer);

	// These are expected to resolve to to server object in this test case, since both the client and server
	// exist in the same engine instance.
	UE_NET_ASSERT_EQ(ClientTestObject->ReplicatedTObjectPtr, ReferencedObject);
	UE_NET_ASSERT_EQ(ClientTestObject->ReplicatedWeakObjectPtr.Get(), ReferencedObject);

	// The raw pointer RPC parameter doesn't go through the RemoteReference path and should point to the replicated
	// object on the client.
	UE_NET_ASSERT_EQ(ClientTestObject->LastReceivedRawPointer, ClientReferencedObject);
}

UE_NET_TEST_FIXTURE(FReplicationSystemConfigOverrideRPCTestFixture, TestObjectsAsRemoteReferencePaths)
{
	FReplicationSystemTestNode::FReplicationSystemParamsOverride Params;
	Params.bUseRemoteObjectReferences = true;

	// Create the server
	CreateServer(Params);

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient(Params);

	// Don't want these to trigger in this test
	FScopedRemoteDelegateOverride ScopedDelegateOverride;

	// Spawn replicated object on server to do the replication
	UTestObjectWithReferencesAsRemote* TestObject = Server->CreateObject<UTestObjectWithReferencesAsRemote>();

	// Spawn a non-replicated object with a deterministic path, it will be the one referenced by the FRemoteObjectReferece
	UTestNamedObject* ServerReferencedObject = NewObject<UTestNamedObject>(GetTransientPackage(), UTestNamedObject::StaticClass(), "TestNamedObject");

	TestObject->bIsServerObject = true;
	TestObject->ReplicationSystem = Server->GetReplicationSystem();
	Server->ReplicationSystem->SetOwningNetConnection(TestObject->NetRefHandle, Client->ConnectionIdOnServer);

	TestObject->ReplicatedTObjectPtr = ServerReferencedObject;
	TestObject->ReplicatedWeakObjectPtr = ServerReferencedObject;
	TestObject->ClientRPCWithRawPointer(ServerReferencedObject);

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Simulate having different instances of objects on the client & server, this forces the FRemoteObjectReference
	// to look up the object by path on the client. We accomplish this by destroying, GCing, and recreating the object.
	ServerReferencedObject->MarkAsGarbage();
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	ServerReferencedObject = nullptr;

	UTestNamedObject* ClientReferencedObject = NewObject<UTestNamedObject>(GetTransientPackage(), UTestNamedObject::StaticClass(), "TestNamedObject");

	// Deliver to client now that the referenced object has been recreated
	Server->DeliverTo(Client, true);

	UTestObjectWithReferencesAsRemote* ClientTestObject = Client->GetObjectAs<UTestObjectWithReferencesAsRemote>(TestObject->NetRefHandle);

	UE_NET_ASSERT_NE(ClientTestObject, nullptr);

	// Client should be able to resolve the remote references since they resolve by path

	// Check the TObjectPtr
	UE_NET_ASSERT_TRUE(ClientTestObject->ReplicatedTObjectPtr);

	UTestNamedObject* ClientResolvedTObjectProperty = Cast<UTestNamedObject>(ClientTestObject->ReplicatedTObjectPtr);
	UE_NET_ASSERT_EQ(ClientResolvedTObjectProperty, ClientReferencedObject);

	// Check the TWeakObjectPtr
	UE_NET_ASSERT_TRUE(ClientTestObject->ReplicatedWeakObjectPtr.IsValid());

	UTestNamedObject* ClientResolvedWeakObjectProperty = Cast<UTestNamedObject>(ClientTestObject->ReplicatedWeakObjectPtr);
	UE_NET_ASSERT_EQ(ClientResolvedWeakObjectProperty, ClientReferencedObject);

	// The raw pointer will be null because it intentionally doesn't go through the remote reference path
	UE_NET_ASSERT_EQ(ClientTestObject->LastReceivedRawPointer, nullptr);
}

}

#endif