// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Tests/ReplicationSystem/RPC/ReplicatedTestObjectWithRPC.h"
#include "TestObjectsAsRemoteReferencesSerialization.generated.h"

UCLASS()
class UTestObjectWithReferencesAsRemote : public UTestReplicatedObjectWithRPC
{
	GENERATED_BODY()

public:
	void GetLifetimeReplicatedProps(TArray<class FLifetimeProperty>& OutLifetimeProps) const;

	UPROPERTY(Replicated)
	TObjectPtr<UObject> ReplicatedTObjectPtr;

	UPROPERTY(Replicated)
	TWeakObjectPtr<UObject> ReplicatedWeakObjectPtr;

	// Raw object pointers are disallowed as member properties in some contexts, but allowed as RPC parameters
	UFUNCTION(Client, Reliable)
	void ClientRPCWithRawPointer(UObject* RawPointer);

	UObject* LastReceivedRawPointer = nullptr;
};
