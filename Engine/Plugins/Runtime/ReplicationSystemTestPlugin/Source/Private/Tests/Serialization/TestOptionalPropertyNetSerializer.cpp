// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestOptionalPropertyNetSerializer.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Tests/Serialization/TestNetSerializerFixture.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetReferenceCollector.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/InternalNetSerializers.h"

#include "Net/UnrealNetwork.h"
#include "UObject/PropertyOptional.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"

namespace UE::Net::Private
{
using OptionalPropertyType = TOptional<FTestOptionalPropertyNetSerializer_StructWithOptionals>;

class FTestOptionalPropertyNetSerializerFixture : public FReplicationSystemServerClientTestFixture
{
public:
	FTestOptionalPropertyNetSerializerFixture() : FReplicationSystemServerClientTestFixture() {}

protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	// Composable operations for testing the serializer
	void Serialize();
	void Deserialize();
	void SerializeDelta();
	void DeserializeDelta();
	void Quantize();
	void QuantizeTwoStates();
	void Dequantize();
	bool IsEqual(bool bQuantized);
	void Clone();
	void Validate();
	void FreeQuantizedState();

	// Modify some of the optional properties
	void SetNonDefaultInstanceState();

	// Adds multiple elements, at least one uninitialized and at least two initialized with at least one modified property
	void SetNonDefaultArrayState();

	bool AreEqual(const OptionalPropertyType* Value0, const OptionalPropertyType* Value1);

	const FNetSerializer* GetOptionalPropertyNetSerializer() const;
	const FOptionalPropertyNetSerializerConfig* GetOptionalPropertyNetSerializerConfig() const;

protected:
	FNetSerializationContext NetSerializationContext;
	FInternalNetSerializationContext InternalNetSerializationContext;

	OptionalPropertyType OptionalProperty0;
	OptionalPropertyType OptionalProperty1;

	FStructNetSerializerConfig StructWithOptionalsNetSerializerConfig;

	alignas(16) uint8 QuantizedBuffer[2][128];
	alignas(16) uint8 ClonedQuantizedBuffer[2][128];
	alignas(16) uint8 BitStreamBuffer[2048];

	bool bHasQuantizedState = false;
	bool bHasClonedQuantizedState = false;

	uint32 QuantizedStateCount = 0;
	uint32 ClonedQuantizedStateCount = 0;

	FNetBitStreamWriter Writer;
	FNetBitStreamReader Reader;
};

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestTOptionalCanSetInt)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestOptionalPropertyNetSerializer_TestObject* ServerObject = Server->CreateObject<UTestOptionalPropertyNetSerializer_TestObject>();

	ServerObject->StructWithOptionals.OptInt = 1;

	Server->UpdateAndSend({Client});

	// Verify that object has been spawned on client
	UTestOptionalPropertyNetSerializer_TestObject* ClientObject = Client->GetObjectAs<UTestOptionalPropertyNetSerializer_TestObject>(ServerObject->NetRefHandle);

	// Verify that created server handle now also exists on client
	UE_NET_ASSERT_NE(ClientObject, nullptr);	

	UE_NET_ASSERT_TRUE(ServerObject->StructWithOptionals.OptInt == ClientObject->StructWithOptionals.OptInt);

	Server->UpdateAndSend({Client});
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestTOptionalDoesNotOverwriteNotReplicatedData)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestOptionalPropertyNetSerializer_TestObject* ServerObject = Server->CreateObject<UTestOptionalPropertyNetSerializer_TestObject>();

	ServerObject->StructWithNotFullyReplicatedOptionalStruct.OptStruct.Emplace(FTestOptionalPropertyNetSerializer_NotFullyReplicatedStruct());

	Server->UpdateAndSend({Client});

	// Verify that object has been spawned on client
	UTestOptionalPropertyNetSerializer_TestObject* ClientObject = Client->GetObjectAs<UTestOptionalPropertyNetSerializer_TestObject>(ServerObject->NetRefHandle);

	// Verify that created server handle now also exists on client
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	
	UE_NET_ASSERT_EQ(ServerObject->StructWithNotFullyReplicatedOptionalStruct.OptStruct->NotReplicatedInt, ClientObject->StructWithNotFullyReplicatedOptionalStruct.OptStruct->NotReplicatedInt);

	// Modify replicated struct member 
	++ServerObject->StructWithNotFullyReplicatedOptionalStruct.OptStruct->IntA;
	// Modify not replicated struct member.
	ServerObject->StructWithNotFullyReplicatedOptionalStruct.OptStruct->NotReplicatedInt = 7;

	Server->UpdateAndSend({Client});

	UE_NET_ASSERT_EQ(ServerObject->StructWithNotFullyReplicatedOptionalStruct.OptStruct->IntA, ClientObject->StructWithNotFullyReplicatedOptionalStruct.OptStruct->IntA);
	// Verify that we leave NotReplicatedInt untouched.
	UE_NET_ASSERT_NE(ServerObject->StructWithNotFullyReplicatedOptionalStruct.OptStruct->NotReplicatedInt, ClientObject->StructWithNotFullyReplicatedOptionalStruct.OptStruct->NotReplicatedInt);
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestTOptionalCanSetStruct)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestOptionalPropertyNetSerializer_TestObject* ServerObject = Server->CreateObject<UTestOptionalPropertyNetSerializer_TestObject>();

	FTestOptionalPropertyNetSerializer_SimpleStruct OptStruct;
	OptStruct.IntA = 1;
	ServerObject->StructWithOptionals.OptStruct = OptStruct;

	Server->UpdateAndSend({Client});

	// Verify that object has been spawned on client
	UTestOptionalPropertyNetSerializer_TestObject* ClientObject = Client->GetObjectAs<UTestOptionalPropertyNetSerializer_TestObject>(ServerObject->NetRefHandle);

	// Verify that created server handle now also exists on client
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	UE_NET_ASSERT_EQ(ServerObject->StructWithOptionals.OptStruct.IsSet(), ClientObject->StructWithOptionals.OptStruct.IsSet());
	UE_NET_ASSERT_EQ(ServerObject->StructWithOptionals.OptStruct->IntA, ClientObject->StructWithOptionals.OptStruct->IntA);
}

// Instance tests
UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestQuantizeUninitialized)
{
	Quantize();
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestQuantizeInitialized)
{
	SetNonDefaultInstanceState();
	Quantize();
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestDequantizeUninitialized)
{
	Quantize();
	Dequantize();
	UE_NET_ASSERT_TRUE(AreEqual(&OptionalProperty0, &OptionalProperty1));
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestDequantizeInitialized)
{
	SetNonDefaultInstanceState();
	Quantize();
	Dequantize();
	UE_NET_ASSERT_TRUE(AreEqual(&OptionalProperty0, &OptionalProperty1));
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestSerializeUninitialized)
{
	Quantize();
	Serialize();
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestSerializeInitialized)
{
	SetNonDefaultInstanceState();
	Quantize();
	Serialize();
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestDeserializeUninitialized)
{
	Quantize();
	Serialize();
	FreeQuantizedState();
	Deserialize();
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestDeserializeInitialized)
{
	SetNonDefaultInstanceState();
	Quantize();
	Serialize();
	FreeQuantizedState();
	Deserialize();
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestDequantizeSerializedUninitializedState)
{
	Quantize();
	Serialize();
	FreeQuantizedState();
	Deserialize();
	Dequantize();
	UE_NET_ASSERT_TRUE(AreEqual(&OptionalProperty0, &OptionalProperty1));
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestDequantizeSerializedInitializedState)
{
	SetNonDefaultInstanceState();
	Quantize();
	Serialize();
	FreeQuantizedState();
	Deserialize();
	Dequantize();
	UE_NET_ASSERT_TRUE(AreEqual(&OptionalProperty0, &OptionalProperty1));
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestSerializeDeltaEqualUnsetInstances)
{
	QuantizeTwoStates();
	SerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestSerializeDeltaNonEqualStates)
{
	OptionalProperty0.Emplace(FTestOptionalPropertyNetSerializer_StructWithOptionals());
	OptionalProperty0->IntValue += 4711U;

	OptionalProperty1.Emplace();
	OptionalProperty1->IntValue += 711U;

	QuantizeTwoStates();
	SerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestDeserializeDeltaEqualUnsetInstances)
{
	QuantizeTwoStates();
	SerializeDelta();
	DeserializeDelta();
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestDeserializeDeltaNonEqualStates)
{
	OptionalProperty0.Emplace(FTestOptionalPropertyNetSerializer_StructWithOptionals());
	OptionalProperty0->IntValue += 4711U;

	OptionalProperty1.Emplace(FTestOptionalPropertyNetSerializer_StructWithOptionals());
	OptionalProperty1->IntValue += 711U;

	QuantizeTwoStates();
	SerializeDelta();
	DeserializeDelta();
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestDequantizeDeltaSerializedUnsetInstances)
{
	QuantizeTwoStates();
	SerializeDelta();
	DeserializeDelta();
	Dequantize();

	UE_NET_ASSERT_TRUE(AreEqual(&OptionalProperty0, &OptionalProperty1));
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestDequantizeDeltaSerializedNonEqualStates)
{
	OptionalProperty0.Emplace(FTestOptionalPropertyNetSerializer_StructWithOptionals());
	OptionalProperty0->IntValue += 4711U;

	OptionalProperty1.Emplace(FTestOptionalPropertyNetSerializer_StructWithOptionals());
	OptionalProperty1->IntValue += 711U;

	QuantizeTwoStates();
	SerializeDelta();
	DeserializeDelta();
	Dequantize();

	UE_NET_ASSERT_TRUE(AreEqual(&OptionalProperty0, &OptionalProperty1));
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestCollectReferencesUninitialized)
{
	Quantize();

	FNetReferenceCollector Collector;

	FNetCollectReferencesArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetOptionalPropertyNetSerializerConfig());
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Collector = NetSerializerValuePointer(&Collector);
	GetOptionalPropertyNetSerializer()->CollectNetReferences(NetSerializationContext, Args);

	UE_NET_ASSERT_EQ(Collector.GetCollectedReferences().Num(), 0);
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestCollectReferencesNoRef)
{
	OptionalProperty0.Emplace(FTestOptionalPropertyNetSerializer_StructWithOptionals());

	Quantize();

	FNetReferenceCollector Collector(ENetReferenceCollectorTraits::IncludeInvalidReferences);

	FNetCollectReferencesArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetOptionalPropertyNetSerializerConfig());
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Collector = NetSerializerValuePointer(&Collector);

	GetOptionalPropertyNetSerializer()->CollectNetReferences(NetSerializationContext, Args);

	UE_NET_ASSERT_EQ(Collector.GetCollectedReferences().Num(), 1);
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestCollectReferencesWithRef)
{
	OptionalProperty0.Emplace(FTestOptionalPropertyNetSerializer_StructWithOptionals());
	OptionalProperty0->ObjectRef = FTestOptionalPropertyNetSerializer_StructWithOptionals::StaticStruct();

	Quantize();

	FNetReferenceCollector Collector(ENetReferenceCollectorTraits::IncludeInvalidReferences);

	FNetCollectReferencesArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetOptionalPropertyNetSerializerConfig());
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Collector = NetSerializerValuePointer(&Collector);

	GetOptionalPropertyNetSerializer()->CollectNetReferences(NetSerializationContext, Args);

	UE_NET_ASSERT_GE(Collector.GetCollectedReferences().Num(), 1);
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestCollectReferencesWithRefAndOptRef)
{
	OptionalProperty0.Emplace(FTestOptionalPropertyNetSerializer_StructWithOptionals());
	OptionalProperty0->ObjectRef = FTestOptionalPropertyNetSerializer_StructWithOptionals::StaticStruct();

	OptionalProperty0->OptObjectRef = FTestOptionalPropertyNetSerializer_StructWithOptional::StaticStruct();

	Quantize();

	FNetReferenceCollector Collector(ENetReferenceCollectorTraits::IncludeInvalidReferences);

	FNetCollectReferencesArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetOptionalPropertyNetSerializerConfig());
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Collector = NetSerializerValuePointer(&Collector);

	GetOptionalPropertyNetSerializer()->CollectNetReferences(NetSerializationContext, Args);

	UE_NET_ASSERT_GE(Collector.GetCollectedReferences().Num(), 2);
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestIsEqualExternal)
{
	constexpr bool bUseQuantizedState = false;

	// Default state compared to default state
	OptionalProperty0.Reset();
	OptionalProperty1.Reset();
	UE_NET_ASSERT_TRUE(IsEqual(bUseQuantizedState));

	// Non-default state compared to default state
	SetNonDefaultInstanceState();
	UE_NET_ASSERT_FALSE(IsEqual(bUseQuantizedState));

	// Non-default state compared to non-default state
	OptionalProperty1 = OptionalProperty0;
	UE_NET_ASSERT_TRUE(IsEqual(bUseQuantizedState));
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestIsEqualQuantized)
{
	constexpr bool bUseQuantizedState = true;

	// Default state compared to default state
	OptionalProperty0.Reset();
	Quantize();
	Clone();
	UE_NET_ASSERT_TRUE(IsEqual(bUseQuantizedState));

	// Non-default state compared to default state
	SetNonDefaultInstanceState();
	Quantize();
	UE_NET_ASSERT_FALSE(IsEqual(bUseQuantizedState));

	// Non-default state compared to non-default state
	Clone();
	UE_NET_ASSERT_TRUE(IsEqual(bUseQuantizedState));
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestValidate)
{
	Validate();
}

UE_NET_TEST_FIXTURE(FTestOptionalPropertyNetSerializerFixture, TestDequantizeSerializedArrayWithOptionals)
{
	SetNonDefaultInstanceState();

	// Add some array entries
	OptionalProperty0->ArrayWithOptionals.SetNum(10);
	
	OptionalProperty0->ArrayWithOptionals[0] = 0;
	OptionalProperty0->ArrayWithOptionals[3] = 3;
	OptionalProperty0->ArrayWithOptionals[9] = 9;

	Quantize();
	Serialize();
	FreeQuantizedState();
	Deserialize();
	Dequantize();
	
	UE_NET_ASSERT_TRUE(AreEqual(&OptionalProperty0, &OptionalProperty1));
}


}

UTestOptionalPropertyNetSerializer_TestObject::UTestOptionalPropertyNetSerializer_TestObject()
: UReplicatedTestObject()
{
}

void UTestOptionalPropertyNetSerializer_TestObject::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = false;

	DOREPLIFETIME_WITH_PARAMS_FAST(UTestOptionalPropertyNetSerializer_TestObject, StructWithOptionals, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UTestOptionalPropertyNetSerializer_TestObject, StructWithNotFullyReplicatedOptionalStruct, Params);
}

void UTestOptionalPropertyNetSerializer_TestObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

namespace UE::Net::Private
{

// Fixture implementation
void FTestOptionalPropertyNetSerializerFixture::SetUp()
{
	FReplicationSystemServerClientTestFixture::SetUp();

	// Init default serialization context
	InternalNetSerializationContext.ReplicationSystem = Server->ReplicationSystem;

	FInternalNetSerializationContext TempInternalNetSerializationContext;
	FInternalNetSerializationContext::FInitParameters TempInternalNetSerializationContextInitParams;
	TempInternalNetSerializationContextInitParams.ReplicationSystem = Server->ReplicationSystem;
	TempInternalNetSerializationContextInitParams.ObjectResolveContext.RemoteNetTokenStoreState = Server->ReplicationSystem->GetNetTokenStore()->GetLocalNetTokenStoreState();
	TempInternalNetSerializationContext.Init(TempInternalNetSerializationContextInitParams);

	InternalNetSerializationContext = MoveTemp(TempInternalNetSerializationContext);
	NetSerializationContext.SetInternalContext(&InternalNetSerializationContext);

	FMemory::Memzero(QuantizedBuffer, sizeof(QuantizedBuffer));

	bHasQuantizedState = false;
	bHasClonedQuantizedState = false;

	if (!StructWithOptionalsNetSerializerConfig.StateDescriptor.IsValid())
	{
		FReplicationStateDescriptorBuilder::FParameters Params;
		StructWithOptionalsNetSerializerConfig.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(FTestOptionalPropertyNetSerializer_StructWithOptional::StaticStruct(), Params);
	}
}

void FTestOptionalPropertyNetSerializerFixture::TearDown()
{
	OptionalProperty0.Reset();
	OptionalProperty1.Reset();

	FreeQuantizedState();

	FReplicationSystemServerClientTestFixture::TearDown();
}

void FTestOptionalPropertyNetSerializerFixture::Serialize()
{
	// Must have run quantize before this
	UE_NET_ASSERT_TRUE(bHasQuantizedState);

	// Serialize data
	Writer.InitBytes(BitStreamBuffer, sizeof(BitStreamBuffer));
	FNetSerializationContext Context(&Writer);
	Context.SetInternalContext(NetSerializationContext.GetInternalContext());

	FNetSerializeArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetOptionalPropertyNetSerializerConfig());
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	GetOptionalPropertyNetSerializer()->Serialize(Context, Args);

	Writer.CommitWrites();

	UE_NET_ASSERT_FALSE(Context.HasError());
	UE_NET_ASSERT_GT(Writer.GetPosBits(), 0U);
}

void FTestOptionalPropertyNetSerializerFixture::Deserialize()
{
	// Check pre-conditions
	UE_NET_ASSERT_FALSE(bHasQuantizedState);
	UE_NET_ASSERT_GT(Writer.GetPosBytes(), 0U);

	Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());

	FNetSerializationContext Context(&Reader);
	Context.SetInternalContext(NetSerializationContext.GetInternalContext());

	FNetDeserializeArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetOptionalPropertyNetSerializerConfig());
	Args.Target = NetSerializerValuePointer(&QuantizedBuffer[0]);
	GetOptionalPropertyNetSerializer()->Deserialize(Context, Args);

	bHasQuantizedState = true;

	UE_NET_ASSERT_FALSE(Context.HasErrorOrOverflow());
	UE_NET_ASSERT_GT(Reader.GetPosBits(), 0U);
}

void FTestOptionalPropertyNetSerializerFixture::SerializeDelta()
{
	// Check pre-conditions
	UE_NET_ASSERT_TRUE(bHasQuantizedState);
	UE_NET_ASSERT_EQ(QuantizedStateCount, 2U);

	// Serialize data
	Writer.InitBytes(BitStreamBuffer, sizeof(BitStreamBuffer));
	FNetSerializationContext Context(&Writer);
	Context.SetInternalContext(NetSerializationContext.GetInternalContext());

	FNetSerializeDeltaArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetOptionalPropertyNetSerializerConfig());
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Prev = NetSerializerValuePointer(&QuantizedBuffer[1]);
	GetOptionalPropertyNetSerializer()->SerializeDelta(Context, Args);

	Writer.CommitWrites();

	UE_NET_ASSERT_FALSE(Context.HasErrorOrOverflow());
	UE_NET_ASSERT_GT(Writer.GetPosBits(), 0U);
}

void FTestOptionalPropertyNetSerializerFixture::DeserializeDelta()
{
	// Check pre-conditions
	UE_NET_ASSERT_GT(Writer.GetPosBytes(), 0U);

	Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());

	FNetSerializationContext Context(&Reader);
	Context.SetInternalContext(NetSerializationContext.GetInternalContext());

	FNetDeserializeDeltaArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetOptionalPropertyNetSerializerConfig());
	Args.Target = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Prev = NetSerializerValuePointer(&QuantizedBuffer[1]);
	GetOptionalPropertyNetSerializer()->DeserializeDelta(Context, Args);

	bHasQuantizedState = true;
	QuantizedStateCount = 1;

	UE_NET_ASSERT_FALSE(Context.HasErrorOrOverflow());
	UE_NET_ASSERT_GT(Reader.GetPosBits(), 0U);
}

void FTestOptionalPropertyNetSerializerFixture::Quantize()
{
	FNetQuantizeArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetOptionalPropertyNetSerializerConfig());
	Args.Target = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Source = NetSerializerValuePointer(&OptionalProperty0);
	GetOptionalPropertyNetSerializer()->Quantize(NetSerializationContext, Args);

	bHasQuantizedState = true;
	QuantizedStateCount = 1;

	UE_NET_ASSERT_FALSE(NetSerializationContext.HasError());
}

void FTestOptionalPropertyNetSerializerFixture::QuantizeTwoStates()
{
	Quantize();

	FNetQuantizeArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetOptionalPropertyNetSerializerConfig());
	Args.Target = NetSerializerValuePointer(&QuantizedBuffer[1]);
	Args.Source = NetSerializerValuePointer(&OptionalProperty1);
	GetOptionalPropertyNetSerializer()->Quantize(NetSerializationContext, Args);

	bHasQuantizedState = true;
	QuantizedStateCount = 2;

	UE_NET_ASSERT_FALSE(NetSerializationContext.HasError());
}

void FTestOptionalPropertyNetSerializerFixture::Clone()
{
	// Check pre-conditions
	UE_NET_ASSERT_TRUE(bHasQuantizedState);

	FMemory::Memcpy(ClonedQuantizedBuffer[0], QuantizedBuffer[0], sizeof(QuantizedBuffer[0]));

	FNetCloneDynamicStateArgs Args = {};
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Target = NetSerializerValuePointer(&ClonedQuantizedBuffer[0]);
	Args.NetSerializerConfig = NetSerializerConfigParam(GetOptionalPropertyNetSerializerConfig());
	GetOptionalPropertyNetSerializer()->CloneDynamicState(NetSerializationContext, Args);

	bHasClonedQuantizedState = true;
	ClonedQuantizedStateCount = 1;
}

void FTestOptionalPropertyNetSerializerFixture::FreeQuantizedState()
{
	FNetFreeDynamicStateArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetOptionalPropertyNetSerializerConfig());

	if (bHasQuantizedState)
	{
		for (uint32 StateIt = 0, StateEndIt = QuantizedStateCount; StateIt != StateEndIt; ++StateIt)
		{
			Args.Source = NetSerializerValuePointer(&QuantizedBuffer[StateIt]);
			GetOptionalPropertyNetSerializer()->FreeDynamicState(NetSerializationContext, Args);

			FMemory::Memzero(&QuantizedBuffer[StateIt], sizeof(QuantizedBuffer[StateIt]));
		}
		bHasQuantizedState = false;
	}

	if (bHasClonedQuantizedState)
	{
		for (uint32 StateIt = 0, StateEndIt = ClonedQuantizedStateCount; StateIt != StateEndIt; ++StateIt)
		{
			Args.Source = NetSerializerValuePointer(&ClonedQuantizedBuffer[StateIt]);
			GetOptionalPropertyNetSerializer()->FreeDynamicState(NetSerializationContext, Args);

			FMemory::Memzero(&ClonedQuantizedBuffer[StateIt], sizeof(ClonedQuantizedBuffer[StateIt]));
		}
		bHasClonedQuantizedState = false;
	}
}

void FTestOptionalPropertyNetSerializerFixture::Dequantize()
{
	UE_NET_ASSERT_TRUE(bHasQuantizedState);

	FNetDequantizeArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetOptionalPropertyNetSerializerConfig());
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Target = NetSerializerValuePointer(&OptionalProperty1);
	GetOptionalPropertyNetSerializer()->Dequantize(NetSerializationContext, Args);
}

bool FTestOptionalPropertyNetSerializerFixture::IsEqual(bool bQuantized)
{
	if (bQuantized)
	{
		UE_NET_EXPECT_TRUE(bHasQuantizedState);
		if (!bHasQuantizedState)
		{
			return false;
		}

		UE_NET_EXPECT_TRUE(bHasClonedQuantizedState);
		if (!bHasClonedQuantizedState)
		{
			return false;
		}
	}

	FNetIsEqualArgs Args = {};
	if (bQuantized)
	{
		Args.Source0 = NetSerializerValuePointer(&QuantizedBuffer[0]);
		Args.Source1 = NetSerializerValuePointer(&ClonedQuantizedBuffer[0]);
	}
	else
	{
		Args.Source0 = NetSerializerValuePointer(&OptionalProperty0);
		Args.Source1 = NetSerializerValuePointer(&OptionalProperty1);
	}
	Args.bStateIsQuantized = bQuantized;
	Args.NetSerializerConfig = NetSerializerConfigParam(GetOptionalPropertyNetSerializerConfig());
	return GetOptionalPropertyNetSerializer()->IsEqual(NetSerializationContext, Args);
}

void FTestOptionalPropertyNetSerializerFixture::Validate()
{
	FNetValidateArgs Args = {};
	Args.NetSerializerConfig = NetSerializerConfigParam(GetOptionalPropertyNetSerializerConfig());
	Args.Source = NetSerializerValuePointer(&OptionalProperty0);

	GetOptionalPropertyNetSerializer()->Validate(NetSerializationContext, Args);
}

void FTestOptionalPropertyNetSerializerFixture::SetNonDefaultInstanceState()
{
	OptionalProperty0.Emplace(FTestOptionalPropertyNetSerializer_StructWithOptionals());
	OptionalProperty0->IntValue += 13;
}

bool FTestOptionalPropertyNetSerializerFixture::AreEqual(const OptionalPropertyType* Value0, const OptionalPropertyType* Value1)
{
	const FOptionalPropertyNetSerializerConfig* Config =  GetOptionalPropertyNetSerializerConfig();

	return Config->Property->Identical(Value0, Value1, 0);
}

const FNetSerializer* FTestOptionalPropertyNetSerializerFixture::GetOptionalPropertyNetSerializer() const
{
	if (const FReplicationStateDescriptor* Desc = StructWithOptionalsNetSerializerConfig.StateDescriptor.GetReference())
	{
		return Desc->MemberSerializerDescriptors[0].Serializer;
	}

	return nullptr;
}

const FOptionalPropertyNetSerializerConfig* FTestOptionalPropertyNetSerializerFixture::GetOptionalPropertyNetSerializerConfig() const
{
	if (const FReplicationStateDescriptor* Desc = StructWithOptionalsNetSerializerConfig.StateDescriptor.GetReference())
	{
		return static_cast<const FOptionalPropertyNetSerializerConfig*>(Desc->MemberSerializerDescriptors[0].SerializerConfig);
	}

	return nullptr;
}

}


