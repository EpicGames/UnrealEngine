// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Misc/Optional.h"
#include "UObject/ObjectMacros.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "TestOptionalPropertyNetSerializer.generated.h"

USTRUCT()
struct FTestOptionalPropertyNetSerializer_SimpleStruct
{
	GENERATED_BODY()

public:
	UPROPERTY(transient)
	int32 IntA = 0;
};

USTRUCT()
struct FTestOptionalPropertyNetSerializer_NotFullyReplicatedStruct
{
	GENERATED_BODY()

public:
	UPROPERTY(transient)
	int32 IntA = 0;

	UPROPERTY(transient, NotReplicated)
	int32 NotReplicatedInt = 0;
};

USTRUCT()
struct FTestOptionalPropertyNetSerializer_StructWithObjectReference
{
	GENERATED_BODY()

public:
	UPROPERTY(transient)
	TObjectPtr<UObject> ObjectRef = nullptr;
};

USTRUCT()
struct FTestOptionalPropertyNetSerializer_StructWithOptionals
{
	GENERATED_BODY()

public:
	UPROPERTY(transient)
	int32 IntValue = 0;

	UPROPERTY(transient)
	TArray<TOptional<int32>> ArrayWithOptionals;

	UPROPERTY(transient)
	TObjectPtr<UObject> ObjectRef = nullptr;

	UPROPERTY(transient)
	TOptional<int32> OptInt;

	UPROPERTY(transient)
	TOptional<TObjectPtr<UObject>> OptObjectRef;

	UPROPERTY(transient)
	TOptional<TArray<int32>> OptIntArray;

	UPROPERTY(transient)
	TOptional<TArray<TOptional<int32>>> OptNestOptIntArray;

	UPROPERTY(transient)
	TOptional<FTestOptionalPropertyNetSerializer_SimpleStruct> OptStruct;
};

// Struct that we use to grab NetSerializer and config for TOptional<FTestOptionalPropertyNetSerializer_StructWithOptionals>
USTRUCT()
struct FTestOptionalPropertyNetSerializer_StructWithOptional
{
	GENERATED_BODY()

public:
	UPROPERTY(transient)
	TOptional<FTestOptionalPropertyNetSerializer_StructWithOptionals> OptInt;
};

USTRUCT()
struct FTestOptionalPropertyNetSerializer_StructWithNotFullyReplicatedOptionalStruct
{
	GENERATED_BODY()

public:
	UPROPERTY(transient)
	TOptional<FTestOptionalPropertyNetSerializer_NotFullyReplicatedStruct> OptStruct;
};

UCLASS()
class UTestOptionalPropertyNetSerializer_TestObject : public UReplicatedTestObject
{
	GENERATED_BODY()
public:
	UTestOptionalPropertyNetSerializer_TestObject();

	void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	UPROPERTY(Transient, Replicated)
	FTestOptionalPropertyNetSerializer_StructWithOptionals StructWithOptionals;

	UPROPERTY(Transient, Replicated)
	FTestOptionalPropertyNetSerializer_StructWithNotFullyReplicatedOptionalStruct StructWithNotFullyReplicatedOptionalStruct;

// Not supported at the moment due to UHT.
//	UPROPERTY(Transient, Replicated)
//	TOptional<Int32> OptInt;
};
