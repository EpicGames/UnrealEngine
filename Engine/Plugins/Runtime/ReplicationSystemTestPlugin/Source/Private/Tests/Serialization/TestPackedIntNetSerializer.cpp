// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/PackedIntNetSerializers.h"
#include <limits>

namespace UE::Net::Private
{

static FTestMessage& PrintPackedIntNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

template<typename SourceType>
class TTestPackedIntNetSerializer : public TTestNetSerializerFixture<PrintPackedIntNetSerializerConfig, SourceType>
{
	typedef TTestNetSerializerFixture<PrintPackedIntNetSerializerConfig, SourceType> Super;

public:
	TTestPackedIntNetSerializer(const FNetSerializer& Serializer) : Super(Serializer) {}

	void TestIsEqual();
	void TestSerialize();
	void TestSerializeDelta();
	void TestSmallValuesArePacked();

protected:
	static const SourceType Values[];
	static const SIZE_T ValueCount;
	const FPackedInt32NetSerializerConfig Config;
};

#define UE_NET_IMPLEMENT_PACKEDINT_NETSERIALIZER_TEST(TestClassName, NetSerializerType, SourceType) \
class TestClassName : public TTestPackedIntNetSerializer<SourceType> \
{ \
public: \
	TestClassName() : TTestPackedIntNetSerializer<SourceType>(UE_NET_GET_SERIALIZER(NetSerializerType)) {} \
}; \
\
UE_NET_TEST_FIXTURE(TestClassName, HasTestValues) \
{ \
	UE_NET_ASSERT_GT_MSG(ValueCount, SIZE_T(0), "No test values found"); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestIsEqual) \
{ \
	TestIsEqual(); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestSerialize) \
{ \
	TestSerialize(); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestSerializeDelta) \
{ \
	TestSerializeDelta(); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestSmallValuesArePacked) \
{ \
	TestSmallValuesArePacked(); \
}

UE_NET_IMPLEMENT_PACKEDINT_NETSERIALIZER_TEST(FTestPackedInt64NetSerializer, FPackedInt64NetSerializer, int64);
UE_NET_IMPLEMENT_PACKEDINT_NETSERIALIZER_TEST(FTestPackedInt32NetSerializer, FPackedInt32NetSerializer, int32);

#undef UE_NET_IMPLEMENT_PACKEDINT_NETSERIALIZER_TEST

//
template<typename SourceType> const SourceType TTestPackedIntNetSerializer<SourceType>::Values[] =
{
	std::numeric_limits<SourceType>::min(), std::numeric_limits<SourceType>::max(), SourceType(0), SourceType(-1), (std::numeric_limits<SourceType>::max() + std::numeric_limits<SourceType>::min())/SourceType(2) - SourceType(15), SourceType(2048458 % std::numeric_limits<SourceType>::max())
};
template<typename SourceType> const SIZE_T TTestPackedIntNetSerializer<SourceType>::ValueCount = sizeof(Values)/sizeof(Values[0]);

template<typename SourceType>
void TTestPackedIntNetSerializer<SourceType>::TestIsEqual()
{
	SourceType CompareValues[2][sizeof(Values)/sizeof(Values[0])];
	bool ExpectedResults[2][sizeof(Values)/sizeof(Values[0])];
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		CompareValues[0][ValueIt] = Values[ValueIt];
		ExpectedResults[0][ValueIt] = true;
	}
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		CompareValues[1][ValueIt] = ~Values[ValueIt];
		ExpectedResults[1][ValueIt] = false;
	}

	// Do two rounds of testing, one where we compare each value with itself and one where we compare against a guaranteed non-equal value.
	// We will only do an unquantized compare since we know these serializers uses a default quantize implementation.
	for (SIZE_T TestRoundIt = 0, TestRoundEndIt = 2; TestRoundIt != TestRoundEndIt; ++TestRoundIt)
	{
		constexpr bool bQuantizedCompare = false;
		const bool bSuccess = Super::TestIsEqual(Values, CompareValues[TestRoundIt], ExpectedResults[TestRoundIt], ValueCount, Config, bQuantizedCompare);
		if (!bSuccess)
		{
			return;
		}
	}
}

template<typename SourceType>
void TTestPackedIntNetSerializer<SourceType>::TestSerialize()
{
	SourceType ExpectedValues[sizeof(Values) / sizeof(Values[0])];
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		ExpectedValues[ValueIt] = Values[ValueIt];
	}

	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		constexpr bool bQuantizedCompare = false;
		const bool bSuccess = Super::TestSerialize(Values, ExpectedValues, ValueCount, Config, bQuantizedCompare);
		if (!bSuccess)
		{
			return;
		}
	}
}

template<typename SourceType>
void TTestPackedIntNetSerializer<SourceType>::TestSerializeDelta()
{
	Super::TestSerializeDelta(Values, ValueCount, Config);
}

template<typename SourceType>
void TTestPackedIntNetSerializer<SourceType>::TestSmallValuesArePacked()
{
	constexpr SourceType SmallValue = SourceType(0);
	constexpr SourceType LargeValue = std::numeric_limits<SourceType>::max();

	const bool bSerializedSmallValue = this->Serialize(Config, NetSerializerValuePointer(&SmallValue));
	UE_NET_ASSERT_TRUE_MSG(bSerializedSmallValue, "Failed serializing " << SmallValue);
	const uint32 SmallValueBits = this->Writer.GetPosBits();

	const bool bSerializedLargeValue = this->Serialize(Config, NetSerializerValuePointer(&LargeValue));
	UE_NET_ASSERT_TRUE_MSG(bSerializedLargeValue, "Failed serializing " << LargeValue);
	const uint32 LargeValueBits = this->Writer.GetPosBits();

	UE_NET_ASSERT_LT_MSG(SmallValueBits, LargeValueBits, "Bits serialized for small value (" << SmallValue << ") should've been less than for large value (" << LargeValue << ")");
}

}
