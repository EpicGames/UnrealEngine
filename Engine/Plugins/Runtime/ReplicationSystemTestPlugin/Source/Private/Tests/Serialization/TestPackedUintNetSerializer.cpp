// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/PackedIntNetSerializers.h"
#include <limits>

namespace UE::Net::Private
{

static FTestMessage& PrintPackedUintNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

template<typename SourceType>
class TTestPackedUintNetSerializer : public TTestNetSerializerFixture<PrintPackedUintNetSerializerConfig, SourceType>
{
	typedef TTestNetSerializerFixture<PrintPackedUintNetSerializerConfig, SourceType> Super;

public:
	TTestPackedUintNetSerializer(const FNetSerializer& Serializer) : Super(Serializer) {}

	void TestIsEqual();
	void TestSerialize();
	void TestSerializeDelta();
	void TestSmallValuesArePacked();

protected:
	static const SourceType Values[];
	static const SIZE_T ValueCount;
	const FPackedUint32NetSerializerConfig Config;
};

#define UE_NET_IMPLEMENT_PACKEDUINT_NETSERIALIZER_TEST(TestClassName, NetSerializerType, SourceType) \
class TestClassName : public TTestPackedUintNetSerializer<SourceType> \
{ \
public: \
	TestClassName() : TTestPackedUintNetSerializer<SourceType>(UE_NET_GET_SERIALIZER(NetSerializerType)) {} \
}; \
\
UE_NET_TEST_FIXTURE(TestClassName, HasTestValues) \
{ \
	UE_NET_ASSERT_GT_MSG(ValueCount, SIZE_T(0), "No test values found"); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestIsEqual) \
{ \
	TestIsEqual(); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestSerialize) \
{ \
	TestSerialize(); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestSerializeDelta) \
{ \
	TestSerializeDelta(); \
} \
\
UE_NET_TEST_FIXTURE(TestClassName, TestSmallValuesArePacked) \
{ \
	TestSmallValuesArePacked(); \
}

UE_NET_IMPLEMENT_PACKEDUINT_NETSERIALIZER_TEST(FTestPackedUint64NetSerializer, FPackedUint64NetSerializer, uint64);
UE_NET_IMPLEMENT_PACKEDUINT_NETSERIALIZER_TEST(FTestPackedUint32NetSerializer, FPackedUint32NetSerializer, uint32);

#undef UE_NET_IMPLEMENT_PACKEDUINT_NETSERIALIZER_TEST

//
template<typename SourceType> const SourceType TTestPackedUintNetSerializer<SourceType>::Values[] =
{
	std::numeric_limits<SourceType>::min(), std::numeric_limits<SourceType>::max(), SourceType(0), (std::numeric_limits<SourceType>::max() + std::numeric_limits<SourceType>::min())/SourceType(2) - SourceType(15), SourceType(2048458 % std::numeric_limits<SourceType>::max())
};
template<typename SourceType> const SIZE_T TTestPackedUintNetSerializer<SourceType>::ValueCount = sizeof(Values)/sizeof(Values[0]);

template<typename SourceType>
void TTestPackedUintNetSerializer<SourceType>::TestIsEqual()
{
	SourceType CompareValues[2][sizeof(Values)/sizeof(Values[0])];
	bool ExpectedResults[2][sizeof(Values)/sizeof(Values[0])];
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		CompareValues[0][ValueIt] = Values[ValueIt];
		ExpectedResults[0][ValueIt] = true;
	}
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		CompareValues[1][ValueIt] = ~Values[ValueIt];
		ExpectedResults[1][ValueIt] = false;
	}

	// Do two rounds of testing, one where we compare each value with itself and one where we compare against a guaranteed non-equal value.
	// We will only do an unquantized compare since we know these serializers uses a default quantize implementation.
	for (SIZE_T TestRoundIt = 0, TestRoundEndIt = 2; TestRoundIt != TestRoundEndIt; ++TestRoundIt)
	{
		constexpr bool bQuantizedCompare = false;
		const bool bSuccess = Super::TestIsEqual(Values, CompareValues[TestRoundIt], ExpectedResults[TestRoundIt], ValueCount, Config, bQuantizedCompare);
		if (!bSuccess)
		{
			return;
		}
	}
}

template<typename SourceType>
void TTestPackedUintNetSerializer<SourceType>::TestSerialize()
{
	SourceType ExpectedValues[sizeof(Values) / sizeof(Values[0])];
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		ExpectedValues[ValueIt] = Values[ValueIt];
	}

	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		constexpr bool bQuantizedCompare = false;
		const bool bSuccess = Super::TestSerialize(Values, ExpectedValues, ValueCount, Config, bQuantizedCompare);
		if (!bSuccess)
		{
			return;
		}
	}
}

template<typename SourceType>
void TTestPackedUintNetSerializer<SourceType>::TestSerializeDelta()
{
	Super::TestSerializeDelta(Values, ValueCount, Config);
}

template<typename SourceType>
void TTestPackedUintNetSerializer<SourceType>::TestSmallValuesArePacked()
{
	constexpr SourceType SmallValue = SourceType(0);
	constexpr SourceType LargeValue = std::numeric_limits<SourceType>::max();

	const bool bSerializedSmallValue = this->Serialize(Config, NetSerializerValuePointer(&SmallValue));
	UE_NET_ASSERT_TRUE_MSG(bSerializedSmallValue, "Failed serializing " << SmallValue);
	const uint32 SmallValueBits = this->Writer.GetPosBits();

	const bool bSerializedLargeValue = this->Serialize(Config, NetSerializerValuePointer(&LargeValue));
	UE_NET_ASSERT_TRUE_MSG(bSerializedLargeValue, "Failed serializing " << LargeValue);
	const uint32 LargeValueBits = this->Writer.GetPosBits();

	UE_NET_ASSERT_LT_MSG(SmallValueBits, LargeValueBits, "Bits serialized for small value (" << SmallValue << ") should've been less than for large value (" << LargeValue << ")");
}

}
