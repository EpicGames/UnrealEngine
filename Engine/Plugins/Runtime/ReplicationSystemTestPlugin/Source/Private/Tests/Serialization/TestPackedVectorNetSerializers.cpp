// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/PackedVectorNetSerializers.h"
#include "Math/UnrealMathUtility.h"
#include <limits>

namespace UE::Net::Private
{

static FTestMessage& PrintPackedVectorNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

class FTestPackedVectorNetSerializerBase : public TTestNetSerializerFixture<PrintPackedVectorNetSerializerConfig, FVector>
{
	typedef TTestNetSerializerFixture<PrintPackedVectorNetSerializerConfig, FVector> Super;

public:
	static bool IsAlmostEqual(NetSerializerValuePointer Value0, NetSerializerValuePointer Value1, float MaxAbsDiff, int32 MaxUlpDiff);

protected:
	FTestPackedVectorNetSerializerBase(const FNetSerializer& NetSerializer, const FNetSerializerConfig& NetSerializerConfig);

	void TestIsEqual();
	void TestSerialize(float MaxAbsDiff, int32 MaxUlpDiff);
	void TestSerializeBadValues();
	void TestSerializeDelta();
	void TestValidate();

protected:
	using ScalarType = decltype(FVector::X);

	struct FFloatTriplet
	{
		ScalarType X;
		ScalarType Y;
		ScalarType Z;
	};

	static const FFloatTriplet BadValues[];
	static const FFloatTriplet ExpectedBadValues[];
	static const SIZE_T BadValueCount;
	static const FVector Values[];
	static const SIZE_T ValueCount;

	const FNetSerializerConfig& NetSerializerConfig;
};

const FTestPackedVectorNetSerializerBase::FFloatTriplet FTestPackedVectorNetSerializerBase::BadValues[] =
{
	{-std::numeric_limits<ScalarType>::infinity(), std::numeric_limits<ScalarType>::signaling_NaN(), -std::numeric_limits<ScalarType>::signaling_NaN()},
	{-std::numeric_limits<ScalarType>::infinity(), -std::numeric_limits<ScalarType>::infinity(), -std::numeric_limits<ScalarType>::infinity()},
	{std::numeric_limits<ScalarType>::signaling_NaN(), std::numeric_limits<ScalarType>::signaling_NaN(), std::numeric_limits<ScalarType>::signaling_NaN()},
	{-std::numeric_limits<ScalarType>::signaling_NaN(), -std::numeric_limits<ScalarType>::signaling_NaN(), -std::numeric_limits<ScalarType>::signaling_NaN()},
	{0.0f, std::numeric_limits<ScalarType>::signaling_NaN(), 0.0f},
};
const SIZE_T FTestPackedVectorNetSerializerBase::BadValueCount = sizeof(FTestPackedVectorNetSerializerBase::BadValues)/sizeof(FTestPackedVectorNetSerializerBase::BadValues[0]);
const FTestPackedVectorNetSerializerBase::FFloatTriplet FTestPackedVectorNetSerializerBase::ExpectedBadValues[FTestPackedVectorNetSerializerBase::BadValueCount] = {};

const FVector FTestPackedVectorNetSerializerBase::Values[] =
{
	FVector(0.0f),
	FVector(std::numeric_limits<ScalarType>::max()),
	FVector(std::numeric_limits<ScalarType>::lowest()),
	FVector(-1E-23, +1E23, 1E22), 
	FVector(9872.331244f, 127673.452381f, -4711.4711f),
	FVector(1.26698556E-23),
	FVector(ScalarType(std::numeric_limits<int32>::max()), ScalarType(std::numeric_limits<int32>::max() >> 3U), ScalarType(std::numeric_limits<int32>::max() >> 7)),
	FVector(ScalarType(std::numeric_limits<int32>::min()), -ScalarType(std::numeric_limits<int32>::min() >> 3U), -ScalarType(std::numeric_limits<int32>::min() >> 7U)),
	FVector(ScalarType(1 << 24U), -ScalarType(1 << 21U), ScalarType(1 << 17)),
	FVector(47114711.987654321f, -71147114.123456789f, 32768.111111f),
	FVector(-180817.42f, -FMath::Exp2(25.0f), 47.11f),
	FVector(+180720.42f, -19751216.0f, FPlatformMath::Exp2(31.0f)),
};
const SIZE_T FTestPackedVectorNetSerializerBase::ValueCount = sizeof(FTestPackedVectorNetSerializerBase::Values)/sizeof(FTestPackedVectorNetSerializerBase::Values[0]);

class FTestVectorNetQuantizeNetSerializer : public FTestPackedVectorNetSerializerBase
{
	typedef FTestPackedVectorNetSerializerBase Super;

public:
	FTestVectorNetQuantizeNetSerializer();
};

class FTestVectorNetQuantize10NetSerializer : public FTestPackedVectorNetSerializerBase
{
	typedef FTestPackedVectorNetSerializerBase Super;

public:
	FTestVectorNetQuantize10NetSerializer();
};

class FTestVectorNetQuantize100NetSerializer : public FTestPackedVectorNetSerializerBase
{
	typedef FTestPackedVectorNetSerializerBase Super;

public:
	FTestVectorNetQuantize100NetSerializer();
};

class FTestVectorNetQuantizeNormalNetSerializer : public TTestNetSerializerFixture<PrintPackedVectorNetSerializerConfig, FVector>
{
	typedef TTestNetSerializerFixture<PrintPackedVectorNetSerializerConfig, FVector> Super;

protected:
	FTestVectorNetQuantizeNormalNetSerializer();

	void TestIsEqual();
	void TestSerialize(float MaxAbsDiff, int32 MaxUlpDiff);
	void TestSerializeDelta();
	void TestValidate();

protected:
	using ScalarType = decltype(FVector::X);

	struct FFloatTriplet
	{
		ScalarType X;
		ScalarType Y;
		ScalarType Z;
	};

	static const FFloatTriplet BadValues[7];
	static const SIZE_T BadValueCount;
	static const FVector Values[5];
	static const SIZE_T ValueCount;
	static const FVector ExactValues[3];
	static const SIZE_T ExactValueCount;

	const FNetSerializerConfig& NetSerializerConfig;
};

const FTestVectorNetQuantizeNormalNetSerializer::FFloatTriplet FTestVectorNetQuantizeNormalNetSerializer::BadValues[7] =
{
	{-std::numeric_limits<ScalarType>::infinity(), std::numeric_limits<ScalarType>::signaling_NaN(), -std::numeric_limits<ScalarType>::signaling_NaN()},
	{-std::numeric_limits<ScalarType>::infinity(), -std::numeric_limits<ScalarType>::infinity(), -std::numeric_limits<ScalarType>::infinity()},
	{std::numeric_limits<ScalarType>::signaling_NaN(), std::numeric_limits<ScalarType>::signaling_NaN(), std::numeric_limits<ScalarType>::signaling_NaN()},
	{-std::numeric_limits<ScalarType>::signaling_NaN(), -std::numeric_limits<ScalarType>::signaling_NaN(), -std::numeric_limits<ScalarType>::signaling_NaN()},
	{0.0f, std::numeric_limits<ScalarType>::signaling_NaN(), 0.0f},
	{-1.000001f, 2.0f, -3.33f},
	{0.0f, -5.0f, 0.0f},
};
const SIZE_T FTestVectorNetQuantizeNormalNetSerializer::BadValueCount = sizeof(FTestVectorNetQuantizeNormalNetSerializer::BadValues)/sizeof(FTestVectorNetQuantizeNormalNetSerializer::BadValues[0]);

const FVector FTestVectorNetQuantizeNormalNetSerializer::Values[5] =
{
	FVector(0.70710678f, -0.70710678f, 1.0f - 0.70710678f),
	FVector(-0.8660254f, 1.0f - 0.8660254f, 0.8660254f),
	FVector(0.5f, -0.5f, 0.5f),
	FVector(-0.1f, -0.2f, -0.3f),
	FVector(0.25f, 0.75f, 1.0f),
};
const SIZE_T FTestVectorNetQuantizeNormalNetSerializer::ValueCount = sizeof(FTestVectorNetQuantizeNormalNetSerializer::Values)/sizeof(FTestVectorNetQuantizeNormalNetSerializer::Values[0]);

const FVector FTestVectorNetQuantizeNormalNetSerializer::ExactValues[3] =
{
	FVector(-1.0f),
	FVector(+0.0f),
	FVector(+1.0f),
};
const SIZE_T FTestVectorNetQuantizeNormalNetSerializer::ExactValueCount = sizeof(FTestVectorNetQuantizeNormalNetSerializer::ExactValues)/sizeof(FTestVectorNetQuantizeNormalNetSerializer::ExactValues[0]);

// VectorNetQuantizeNetSerializer tests
UE_NET_TEST_FIXTURE(FTestVectorNetQuantizeNetSerializer, TestSerialize)
{
	TestSerialize(1.0f, 1);
}

// Disabled until we figured out how to silence errors when it is intentional.
//UE_NET_TEST_FIXTURE(FTestVectorNetQuantizeNetSerializer, TestSerializeBadValues)
//{
//	TestSerializeBadValues();
//}

UE_NET_TEST_FIXTURE(FTestVectorNetQuantizeNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestVectorNetQuantizeNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestVectorNetQuantizeNetSerializer, TestValidate)
{
	TestValidate();
}

// VectorNetQuantize10NetSerializer tests
UE_NET_TEST_FIXTURE(FTestVectorNetQuantize10NetSerializer, TestSerialize)
{
	TestSerialize(0.125f, 1);
}

UE_NET_TEST_FIXTURE(FTestVectorNetQuantize10NetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestVectorNetQuantize10NetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestVectorNetQuantize10NetSerializer, TestValidate)
{
	TestValidate();
}

// VectorNetQuantize100NetSerializer tests
UE_NET_TEST_FIXTURE(FTestVectorNetQuantize100NetSerializer, TestSerialize)
{
	TestSerialize(0.0078125f, 1);
}

UE_NET_TEST_FIXTURE(FTestVectorNetQuantize100NetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestVectorNetQuantize100NetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestVectorNetQuantize100NetSerializer, TestValidate)
{
	TestValidate();
}

// VectorNetQuantizeNormalNetSerializer tests
UE_NET_TEST_FIXTURE(FTestVectorNetQuantizeNormalNetSerializer, TestSerialize)
{
	TestSerialize(1.0f/32767.0f, 1);
}

UE_NET_TEST_FIXTURE(FTestVectorNetQuantizeNormalNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestVectorNetQuantizeNormalNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestVectorNetQuantizeNormalNetSerializer, TestValidate)
{
	TestValidate();
}

// 
FTestPackedVectorNetSerializerBase::FTestPackedVectorNetSerializerBase(const FNetSerializer& NetSerializer, const FNetSerializerConfig& InNetSerializerConfig)
: Super(NetSerializer)
, NetSerializerConfig(InNetSerializerConfig)
{
}

void FTestPackedVectorNetSerializerBase::TestSerialize(float MaxAbsDiff, int32 MaxUlpDiff)
{
	const auto& EqualityFunc = [MaxAbsDiff, MaxUlpDiff](NetSerializerValuePointer Value0, NetSerializerValuePointer Value1) -> bool { return IsAlmostEqual(Value0, Value1, MaxAbsDiff, MaxUlpDiff); };

	for (const bool bQuantizedCompare : {false, true})
	{
		TOptional<TFunctionRef<bool(NetSerializerValuePointer Value0, NetSerializerValuePointer Value1)>> CompareFunc;
		if (!bQuantizedCompare)
		{
			CompareFunc = EqualityFunc;
		}
		Super::TestSerialize(Values, Values, ValueCount, NetSerializerConfig, bQuantizedCompare, CompareFunc);
	}
}

void FTestPackedVectorNetSerializerBase::TestSerializeBadValues()
{
	for (const bool bQuantizedCompare : {false, true})
	{
		Super::TestSerialize(reinterpret_cast<const FVector*>(BadValues), reinterpret_cast<const FVector*>(ExpectedBadValues), BadValueCount, NetSerializerConfig, bQuantizedCompare);
	}
}

void FTestPackedVectorNetSerializerBase::TestSerializeDelta()
{
	Super::TestSerializeDelta(Values, ValueCount, NetSerializerConfig);
}

void FTestPackedVectorNetSerializerBase::TestIsEqual()
{
	FVector CompareValues[2][sizeof(Values)/sizeof(Values[0])];
	bool ExpectedResults[2][sizeof(Values)/sizeof(Values[0])];

	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		// Make sure nothing messes with our test values.
		FPlatformMemory::Memcpy(&CompareValues[0][ValueIt], &Values[ValueIt], sizeof(FVector));
		ExpectedResults[0][ValueIt] = true;
	}
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		// Make sure nothing messes with our test values.
		FPlatformMemory::Memcpy(&CompareValues[1][ValueIt], &Values[(ValueIt + 1) % ValueCount], sizeof(FVector));
		ExpectedResults[1][ValueIt] = false;
	}

	for (SIZE_T TestRoundIt = 0, TestRoundEndIt = 2; TestRoundIt != TestRoundEndIt; ++TestRoundIt)
	{
		constexpr bool bQuantizedCompare = true;
		bool bSuccess = Super::TestIsEqual(Values, CompareValues[TestRoundIt], ExpectedResults[TestRoundIt], ValueCount, NetSerializerConfig, bQuantizedCompare);
		if (!bSuccess)
		{
			return;
		}
	}
}

void FTestPackedVectorNetSerializerBase::TestValidate()
{
	// Test bad values
	{
		static_assert(sizeof(BadValues[0]) == sizeof(FVector), "Expected FVector to have the same size as three floats.");
		const bool ExpectedResults[BadValueCount] = {};
		const bool bSuccess = Super::TestValidate(reinterpret_cast<const FVector*>(BadValues), ExpectedResults, BadValueCount, NetSerializerConfig);
		if (!bSuccess)
		{
			return;
		}
	}

	// Test conforming values
	{
		bool ExpectedResults[ValueCount];
		for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
		{
			ExpectedResults[ValueIt] = true;
		}

		const bool bSuccess = Super::TestValidate(Values, ExpectedResults, ValueCount, NetSerializerConfig);
		if (!bSuccess)
		{
			return;
		}
	}
}

bool FTestPackedVectorNetSerializerBase::IsAlmostEqual(NetSerializerValuePointer Value0, NetSerializerValuePointer Value1, float MaxAbsDiff, int32 MaxUlpDiff)
{
	// Allow ULPs difference and absolute difference
	const FVector V0 =  *reinterpret_cast<FVector*>(Value0);
	const FVector V1 =  *reinterpret_cast<FVector*>(Value1);
	const ScalarType AbsDiffX = FMath::Abs(V0.X - V1.X);
	const ScalarType AbsDiffY = FMath::Abs(V0.Y - V1.Y);
	const ScalarType AbsDiffZ = FMath::Abs(V0.Z - V1.Z);

	const bool EqualUlpX = FMath::IsNearlyEqualByULP(V0.X, V1.X, MaxUlpDiff);
	const bool EqualUlpY = FMath::IsNearlyEqualByULP(V0.Y, V1.Y, MaxUlpDiff);
	const bool EqualUlpZ = FMath::IsNearlyEqualByULP(V0.Z, V1.Z, MaxUlpDiff);
	if (((AbsDiffX <= MaxAbsDiff) || EqualUlpX) && ((AbsDiffY <= MaxAbsDiff) || EqualUlpY) && ((AbsDiffZ <= MaxAbsDiff) || EqualUlpZ))
	{
		return true;
	}
		
	return false;
}

FTestVectorNetQuantizeNormalNetSerializer::FTestVectorNetQuantizeNormalNetSerializer()
: Super(UE_NET_GET_SERIALIZER(FVectorNetQuantizeNormalNetSerializer))
, NetSerializerConfig(*UE_NET_GET_SERIALIZER(FVectorNetQuantizeNormalNetSerializer).DefaultConfig)
{
}

void FTestVectorNetQuantizeNormalNetSerializer::TestSerialize(float MaxAbsDiff, int32 MaxUlpDiff)
{
	const auto& EqualityFunc = [MaxAbsDiff, MaxUlpDiff](NetSerializerValuePointer Value0, NetSerializerValuePointer Value1) -> bool { return FTestPackedVectorNetSerializerBase::IsAlmostEqual(Value0, Value1, MaxAbsDiff, MaxUlpDiff); };

	for (const bool bQuantizedCompare : {false, true})
	{
		TOptional<TFunctionRef<bool(NetSerializerValuePointer Value0, NetSerializerValuePointer Value1)>> CompareFunc;
		if (!bQuantizedCompare)
		{
			CompareFunc = EqualityFunc;
		}
		Super::TestSerialize(Values, Values, ValueCount, NetSerializerConfig, bQuantizedCompare, CompareFunc);
	}

	const auto& ExactEqualityFunc = [MaxAbsDiff, MaxUlpDiff](NetSerializerValuePointer Value0, NetSerializerValuePointer Value1) -> bool { return *reinterpret_cast<const FVector*>(Value0) == *reinterpret_cast<const FVector*>(Value1); };

	// Exact values are expected to be dequantized to the exact same value as was originally quantized.
	for (const bool bQuantizedCompare : {false, true})
	{
		TOptional<TFunctionRef<bool(NetSerializerValuePointer Value0, NetSerializerValuePointer Value1)>> CompareFunc;
		if (!bQuantizedCompare)
		{
			CompareFunc = ExactEqualityFunc;
		}
		Super::TestSerialize(ExactValues, ExactValues, ExactValueCount, NetSerializerConfig, bQuantizedCompare, CompareFunc);
	}
}

void FTestVectorNetQuantizeNormalNetSerializer::TestSerializeDelta()
{
	Super::TestSerializeDelta(Values, ValueCount, NetSerializerConfig);
}

void FTestVectorNetQuantizeNormalNetSerializer::TestIsEqual()
{
	TArray<FVector> AllValues;
	AllValues.Reserve(ValueCount + ExactValueCount);
	AllValues.Append(Values, ValueCount);
	AllValues.Append(ExactValues, ExactValueCount);

	TArray<FVector> CompareValues[2];
	TArray<bool> ExpectedResults[2];
	const int32 AllValueCount = AllValues.Num();

	{
		// Make sure nothing messes with our test values.
		CompareValues[0].SetNumUninitialized(AllValueCount);
		FPlatformMemory::Memcpy(CompareValues[0].GetData(), AllValues.GetData(), AllValueCount*sizeof(FVector));
		ExpectedResults[0].Init(true, AllValueCount);
	}

	{
		CompareValues[1].SetNumUninitialized(AllValueCount);
		ExpectedResults[1].Init(false, AllValueCount);
		for (SIZE_T ValueIt = 0; ValueIt < AllValueCount; ++ValueIt)
		{
			// Make sure nothing messes with our test values.
			FPlatformMemory::Memcpy(CompareValues[1].GetData() + ValueIt, AllValues.GetData() + ((ValueIt + 1) % AllValueCount), sizeof(FVector));
		}
	}

	for (SIZE_T TestRoundIt = 0, TestRoundEndIt = 2; TestRoundIt != TestRoundEndIt; ++TestRoundIt)
	{
		constexpr bool bQuantizedCompare = true;
		bool bSuccess = Super::TestIsEqual(AllValues.GetData(), CompareValues[TestRoundIt].GetData(), ExpectedResults[TestRoundIt].GetData(), AllValueCount, NetSerializerConfig, bQuantizedCompare);
		if (!bSuccess)
		{
			return;
		}
	}
}

void FTestVectorNetQuantizeNormalNetSerializer::TestValidate()
{
	// Test bad values
	{
		static_assert(sizeof(BadValues[0]) == sizeof(FVector), "Expected FVector to have the same size as three floats.");
		const bool ExpectedResults[BadValueCount] = {};
		const bool bSuccess = Super::TestValidate(reinterpret_cast<const FVector*>(BadValues), ExpectedResults, BadValueCount, NetSerializerConfig);
		if (!bSuccess)
		{
			return;
		}
	}

	// Test conforming values
	{
		{
			bool ExpectedResults[ValueCount];
			for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
			{
				ExpectedResults[ValueIt] = true;
			}

			const bool bSuccess = Super::TestValidate(Values, ExpectedResults, ValueCount, NetSerializerConfig);
			if (!bSuccess)
			{
				return;
			}
		}

		{
			bool ExpectedResults[ExactValueCount];
			for (SIZE_T ValueIt = 0; ValueIt < ExactValueCount; ++ValueIt)
			{
				ExpectedResults[ValueIt] = true;
			}

			const bool bSuccess = Super::TestValidate(ExactValues, ExpectedResults, ExactValueCount, NetSerializerConfig);
			if (!bSuccess)
			{
				return;
			}
		}
	}
}

// FTestVectorNetQuantizeNetSerializer implementation
FTestVectorNetQuantizeNetSerializer::FTestVectorNetQuantizeNetSerializer()
: Super(UE_NET_GET_SERIALIZER(FVectorNetQuantizeNetSerializer), *UE_NET_GET_SERIALIZER(FVectorNetQuantizeNetSerializer).DefaultConfig)
{
}

// FTestVectorNetQuantize10NetSerializer implementation
FTestVectorNetQuantize10NetSerializer::FTestVectorNetQuantize10NetSerializer()
: Super(UE_NET_GET_SERIALIZER(FVectorNetQuantize10NetSerializer), *UE_NET_GET_SERIALIZER(FVectorNetQuantize10NetSerializer).DefaultConfig)
{
}

// FTestVectorNetQuantize100NetSerializer implementation
FTestVectorNetQuantize100NetSerializer::FTestVectorNetQuantize100NetSerializer()
: Super(UE_NET_GET_SERIALIZER(FVectorNetQuantize100NetSerializer), *UE_NET_GET_SERIALIZER(FVectorNetQuantize100NetSerializer).DefaultConfig)
{
}

}
