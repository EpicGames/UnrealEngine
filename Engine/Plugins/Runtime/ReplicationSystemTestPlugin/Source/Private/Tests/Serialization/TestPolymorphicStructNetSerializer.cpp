// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestPolymorphicStructNetSerializer.h"
#include "TestNetSerializerFixture.h"

#include "Iris/Serialization/PolymorphicNetSerializerImpl.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Iris/ReplicationSystem/NetExports.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/Serialization/NetExportContext.h"
#include "Iris/Serialization/NetReferenceCollector.h"
#include "Net/UnrealNetwork.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"

namespace UE::Net
{

FTestMessage& operator<<(FTestMessage& Message, const FExamplePolymorphicArrayStruct& Value)
{
	return Message << "FExamplePolymorphicArrayStruct of size " << Value.Num();
}

struct FTestPolymorphicArrayStructNetSerializer : public TPolymorphicArrayStructNetSerializerImpl<FExamplePolymorphicArrayStruct, FExamplePolymorphicStructBase, FExamplePolymorphicArrayStruct::GetArray, FExamplePolymorphicArrayStruct::SetArrayNum>
{
	typedef TPolymorphicArrayStructNetSerializerImpl<FExamplePolymorphicArrayStruct, FExamplePolymorphicStructBase, FExamplePolymorphicArrayStruct::GetArray, FExamplePolymorphicArrayStruct::SetArrayNum> InternalNetSerializerType;

	static const uint32 Version = 0;
	static const ConfigType DefaultConfig;
};

const FTestPolymorphicArrayStructNetSerializer::ConfigType FTestPolymorphicArrayStructNetSerializer::DefaultConfig;

UE_NET_IMPLEMENT_SERIALIZER(FTestPolymorphicArrayStructNetSerializer);

struct FTestPolymorphicStructNetSerializer : public TPolymorphicStructNetSerializerImpl<FExamplePolymorphicStruct, FExamplePolymorphicStructBase, FExamplePolymorphicStruct::GetItem>
{
	typedef TPolymorphicStructNetSerializerImpl<FExamplePolymorphicStruct, FExamplePolymorphicStructBase, FExamplePolymorphicStruct::GetItem> InternalNetSerializerType;

	static void InitTypeCache()
	{
		InternalNetSerializerType::InitTypeCache<FTestPolymorphicStructNetSerializer>();
	}

	static const uint32 Version = 0;
	static inline const ConfigType DefaultConfig;
};

UE_NET_IMPLEMENT_SERIALIZER(FTestPolymorphicStructNetSerializer);

}

namespace UE::Net::Private
{

FTestMessage& PrintPolymorphicArrayStructNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

class FTestPolymorphicArrayStructNetSerializerFixture : public FReplicationSystemServerClientTestFixture
{
public:
	FTestPolymorphicArrayStructNetSerializerFixture() : FReplicationSystemServerClientTestFixture() {}

protected:
	virtual void SetUp() override;
	virtual void TearDown() override;
	void ValidateExpectedState(const FExamplePolymorphicArrayStruct& Struct);
	void SetExpectedState(FExamplePolymorphicArrayStruct& Target);

	// Composable operations for testing the serializer
	void Serialize();
	void Deserialize();
	void SerializeDelta();
	void DeserializeDelta();
	void Quantize();
	void QuantizeTwoStates();
	void Dequantize();
	bool IsEqual(bool bQuantized);
	void Clone();
	void Validate();
	void FreeQuantizedState();

	// Test-only access into the cache's private storage. The fixture is friended on
	// FPolymorphicNetSerializerScriptStructCache so this can poke private members; tests
	// (which run on a class derived from the fixture) call through this helper.
	static int32 RemoveTypeFromCache(FPolymorphicNetSerializerScriptStructCache& Cache, FName StructName)
	{
		return Cache.RegisteredTypes.Remove(StructName);
	}

protected:
	FNetSerializationContext NetSerializationContext;
	FInternalNetSerializationContext InternalNetSerializationContext;

	FTestPolymorphicArrayStructNetSerializerConfig PolymorphicNetSerializerConfig;
	const FNetSerializer* PolymorphicNetSerializer = &UE_NET_GET_SERIALIZER(FTestPolymorphicArrayStructNetSerializer);

	FTestPolymorphicStructNetSerializerConfig PolymorphicStructNetSerializerConfig;
	const FNetSerializer* PolymorphicStructNetSerializer = &UE_NET_GET_SERIALIZER(FTestPolymorphicStructNetSerializer);

	FExamplePolymorphicArrayStruct PolymorphicStructInstance0;
	FExamplePolymorphicArrayStruct PolymorphicStructInstance1;

	FExamplePolymorphicStructA* StructA;
	FExamplePolymorphicStructB* StructB;
	FExamplePolymorphicStructC* StructC;
	FExamplePolymorphicStructD* StructD;
	FExamplePolymorphicStructD_Derived* StructD_Derived;

	alignas(8) uint8 QuantizedBuffer[2][2048];
	alignas(8) uint8 ClonedQuantizedBuffer[2][2048];
	alignas(8) uint8 BitStreamBuffer[2048];

	bool bHasQuantizedState = false;
	bool bHasClonedQuantizedState = false;

	uint32 QuantizedStateCount = 0;
	uint32 ClonedQuantizedStateCount = 0;

	FNetBitStreamWriter Writer;
	FNetBitStreamReader Reader;

	// Export context
	UE::Net::Private::FNetExports NetExports;

	inline static bool bHasRegisteredStructDNetSerializer = false;
};

class FTestPolymorphicArrayStructNetSerializerDeltaSerializationFixture : public TTestNetSerializerFixture<PrintPolymorphicArrayStructNetSerializerConfig, FExamplePolymorphicArrayStruct>
{
protected:
	using Super = TTestNetSerializerFixture<PrintPolymorphicArrayStructNetSerializerConfig, FExamplePolymorphicArrayStruct>;

	FTestPolymorphicArrayStructNetSerializerDeltaSerializationFixture() : Super(UE_NET_GET_SERIALIZER(FTestPolymorphicArrayStructNetSerializer)) {}

	virtual void SetUp() override;
	virtual void TearDown() override;

	void SetArbitraryState(FExamplePolymorphicArrayStruct& Target);

protected:
	FTestPolymorphicArrayStructNetSerializerConfig Config;

	FInternalNetSerializationContext InternalNetSerializationContext;

	// Export context
	UE::Net::Private::FNetExports NetExports;

	FDataStreamTestUtil DataStreamUtil;
	FReplicationSystemTestServer* Server = nullptr;
};

}

namespace UE::Net
{

// Custom serializer for FExamplePolymorphicStructD
struct FExamplePolymorphicStructDNetSerializer
{
	static const uint32 Version = 0;

	typedef FExamplePolymorphicStructD SourceType;
	typedef uint32 QuantizedType;
	typedef FExamplePolymorphicStructDNetSerializerConfig ConfigType;

	static inline const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args);
};

void FExamplePolymorphicStructDNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const uint32 Value = *reinterpret_cast<const uint32*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	Writer->WriteBits(Value, 32U);
}

void FExamplePolymorphicStructDNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	const uint32 Value = Reader->ReadBits(32U);

	uint32& TargetValue = *reinterpret_cast<uint32*>(Args.Target);
	
	TargetValue = Value;
}

void FExamplePolymorphicStructDNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	uint32& Target = *reinterpret_cast<uint32*>(Args.Target);
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);

	Target = Source.SomeValue;
}

void FExamplePolymorphicStructDNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const uint32& Source = *reinterpret_cast<const uint32*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	Target.SomeValue = Source;	
}

bool FExamplePolymorphicStructDNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const uint32& Value0 = *reinterpret_cast<const uint32*>(Args.Source0);
		const uint32& Value1 = *reinterpret_cast<const uint32*>(Args.Source1);

		return Value0 == Value1;
	}
	else
	{
		const SourceType Value0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType Value1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		return Value0.SomeValue == Value1.SomeValue;
	}
}

UE_NET_IMPLEMENT_SERIALIZER(FExamplePolymorphicStructDNetSerializer);

}

namespace UE::Net::Private
{

static const FName PropertyNetSerializerRegistry_NAME_ExamplePolymorphicStructD(TEXT("ExamplePolymorphicStructD"));
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_ExamplePolymorphicStructD, FExamplePolymorphicStructDNetSerializer);
static const FName PropertyNetSerializerRegistry_NAME_TestPolymorphicStructNetSerializer(TEXT("ExamplePolymorphicStruct"));
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_TestPolymorphicStructNetSerializer, FTestPolymorphicStructNetSerializer);

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestQuantize)
{
	Quantize();
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestDequantize)
{
	Quantize();
	Dequantize();
	ValidateExpectedState(PolymorphicStructInstance1);
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestSerialize)
{
	Quantize();
	Serialize();
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestDeserialize)
{
	Quantize();
	Serialize();
	FreeQuantizedState();
	Deserialize();
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestDequantizeSerializedState)
{
	Quantize();
	Serialize();
	FreeQuantizedState();
	Deserialize();
	Dequantize();
	ValidateExpectedState(PolymorphicStructInstance1);
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestSerializeDelta)
{
	QuantizeTwoStates();
	SerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestDeserializeDelta)
{
	QuantizeTwoStates();
	SerializeDelta();
	DeserializeDelta();
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestDequantizeDeltaSerializedState)
{
	QuantizeTwoStates();
	SerializeDelta();
	DeserializeDelta();
	Dequantize();
	ValidateExpectedState(PolymorphicStructInstance1);
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestCollectReferencesNoRef)
{
	Quantize();

	FNetReferenceCollector Collector;

	FNetCollectReferencesArgs Args = {};
	Args.Version = PolymorphicNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&PolymorphicNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Collector = NetSerializerValuePointer(&Collector);

	PolymorphicNetSerializer->CollectNetReferences(NetSerializationContext, Args);	

	UE_NET_ASSERT_EQ(0, Collector.GetCollectedReferences().Num());
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestCollectReferencesRef)
{
	Quantize();

	FNetReferenceCollector Collector(ENetReferenceCollectorTraits::IncludeInvalidReferences);

	FNetCollectReferencesArgs Args = {};
	Args.Version = PolymorphicNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&PolymorphicNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Collector = NetSerializerValuePointer(&Collector);

	PolymorphicNetSerializer->CollectNetReferences(NetSerializationContext, Args);	

	UE_NET_ASSERT_EQ(1, Collector.GetCollectedReferences().Num());
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestIsEqualExternal)
{
	constexpr bool bUseQuantizedState = false;
	UE_NET_ASSERT_FALSE(IsEqual(bUseQuantizedState));

	// Copy the state using the defined shallow copy
	PolymorphicStructInstance1 = PolymorphicStructInstance0;
	UE_NET_ASSERT_TRUE(IsEqual(bUseQuantizedState));

	// Set the same values, that would be equal if we used a deep copy
	// Due to the way the struct we try to mimic is implemented when it comes to compare we do expect this to fail even though the values are the same
	// It only does a shallow copy by default, and also only relies on comparing pointers for detecting changes which probably can fail pretty nicely
	SetExpectedState(PolymorphicStructInstance1);

	UE_NET_ASSERT_FALSE(IsEqual(bUseQuantizedState));
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestIsEqualQuantized)
{
	constexpr bool bUseQuantizedState = true;

	Quantize();
	Clone();
	UE_NET_ASSERT_TRUE(IsEqual(bUseQuantizedState));
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestValidate)
{
	Validate();
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestCacheLookupByName)
{
	// Two independently initialized caches must produce equivalent FName lookups for the same
	// struct, and lookups for unregistered names must return nullptr (not mis-resolve to
	// some other entry the way an array index would).
	FPolymorphicNetSerializerScriptStructCache CacheA;
	FPolymorphicNetSerializerScriptStructCache CacheB;
	CacheA.InitForType(FExamplePolymorphicStructBase::StaticStruct());
	CacheB.InitForType(FExamplePolymorphicStructBase::StaticStruct());

	const FName StructAName = FPolymorphicNetSerializerScriptStructCache::MakeStructName(FExamplePolymorphicStructA::StaticStruct());
	UE_NET_ASSERT_FALSE(StructAName.IsNone());

	const FPolymorphicNetSerializerScriptStructCache::FTypeInfo* InfoA = CacheA.GetTypeInfo(StructAName);
	const FPolymorphicNetSerializerScriptStructCache::FTypeInfo* InfoB = CacheB.GetTypeInfo(StructAName);
	UE_NET_ASSERT_NE(InfoA, static_cast<const FPolymorphicNetSerializerScriptStructCache::FTypeInfo*>(nullptr));
	UE_NET_ASSERT_NE(InfoB, static_cast<const FPolymorphicNetSerializerScriptStructCache::FTypeInfo*>(nullptr));
	UE_NET_ASSERT_TRUE(InfoA->ScriptStruct == InfoB->ScriptStruct);
	UE_NET_ASSERT_TRUE(InfoA->ScriptStruct == FExamplePolymorphicStructA::StaticStruct());

	// Lookup by ScriptStruct pointer must agree with lookup by FName.
	UE_NET_ASSERT_EQ(CacheA.GetTypeInfo(FExamplePolymorphicStructA::StaticStruct()), InfoA);

	// Unknown names resolve to nullptr (the bug we are fixing: under TypeIndex this would
	// silently land on an unrelated entry).
	UE_NET_ASSERT_EQ(CacheA.GetTypeInfo(FName()), static_cast<const FPolymorphicNetSerializerScriptStructCache::FTypeInfo*>(nullptr));
	UE_NET_ASSERT_EQ(CacheA.GetTypeInfo(FName(TEXT("/Script/UnloadedPlugin.SomeStructThatDoesNotExist"))), static_cast<const FPolymorphicNetSerializerScriptStructCache::FTypeInfo*>(nullptr));
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestCacheDivergenceMatchingStruct)
{
	// Server has the full set of polymorphic structs registered. Client is missing
	// FExamplePolymorphicStructA (simulating a client build that didn't load that module).
	// Server quantize+serialize an array containing only StructB, which both sides have.
	// Client deserialize+dequantize must succeed and round-trip the value.
	FTestPolymorphicArrayStructNetSerializerConfig ServerConfig;
	FTestPolymorphicArrayStructNetSerializerConfig ClientConfig;
	ServerConfig.RegisteredTypes.InitForType(FExamplePolymorphicStructBase::StaticStruct());
	ClientConfig.RegisteredTypes.InitForType(FExamplePolymorphicStructBase::StaticStruct());

	const FName StructAName = FPolymorphicNetSerializerScriptStructCache::MakeStructName(FExamplePolymorphicStructA::StaticStruct());
	UE_NET_ASSERT_NE(RemoveTypeFromCache(ClientConfig.RegisteredTypes, StructAName), 0);

	FExamplePolymorphicArrayStruct ServerArray;
	{
		FExamplePolymorphicStructB* StructBLocal = new FExamplePolymorphicStructB;
		StructBLocal->SomeFloat = 7.0f;
		ServerArray.Add(StructBLocal);
	}

	FMemory::Memzero(QuantizedBuffer[0], sizeof(QuantizedBuffer[0]));

	{
		FNetQuantizeArgs Args = {};
		Args.Version = PolymorphicNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(&ServerConfig);
		Args.Target = NetSerializerValuePointer(&QuantizedBuffer[0]);
		Args.Source = NetSerializerValuePointer(&ServerArray);
		PolymorphicNetSerializer->Quantize(NetSerializationContext, Args);
	}
	UE_NET_ASSERT_FALSE(NetSerializationContext.HasError());

	Writer.InitBytes(BitStreamBuffer, sizeof(BitStreamBuffer));
	{
		FNetSerializationContext Context(&Writer);
		Context.SetInternalContext(NetSerializationContext.GetInternalContext());
		// Setup export scope to capture and serialize exports
		FNetExportContext::FBatchExports CurrentPacketBatchExports;
		FNetExports::FExportScope ExportScope = NetExports.MakeExportScope(Context, CurrentPacketBatchExports);

		FNetSerializeArgs Args = {};
		Args.Version = PolymorphicNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(&ServerConfig);
		Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
		PolymorphicNetSerializer->Serialize(Context, Args);
		Writer.CommitWrites();
		UE_NET_ASSERT_FALSE(Context.HasError());
	}

	{
		FNetFreeDynamicStateArgs Args = {};
		Args.Version = PolymorphicNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(&ServerConfig);
		Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
		PolymorphicNetSerializer->FreeDynamicState(NetSerializationContext, Args);
	}
	FMemory::Memzero(QuantizedBuffer[0], sizeof(QuantizedBuffer[0]));

	Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());
	{
		FNetSerializationContext Context(&Reader);
		Context.SetInternalContext(NetSerializationContext.GetInternalContext());
		FNetDeserializeArgs Args = {};
		Args.Version = PolymorphicNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(&ClientConfig);
		Args.Target = NetSerializerValuePointer(&QuantizedBuffer[0]);
		PolymorphicNetSerializer->Deserialize(Context, Args);
		UE_NET_ASSERT_FALSE(Context.HasError());
	}

	FExamplePolymorphicArrayStruct ClientArray;
	{
		FNetDequantizeArgs Args = {};
		Args.Version = PolymorphicNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(&ClientConfig);
		Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
		Args.Target = NetSerializerValuePointer(&ClientArray);
		PolymorphicNetSerializer->Dequantize(NetSerializationContext, Args);
	}

	UE_NET_ASSERT_EQ(ClientArray.Num(), 1);
	const FExamplePolymorphicStructBase* RoundTrippedBase = ClientArray.Get(0);
	UE_NET_ASSERT_NE(RoundTrippedBase, static_cast<const FExamplePolymorphicStructBase*>(nullptr));
	UE_NET_ASSERT_EQ(RoundTrippedBase->GetScriptStruct(), FExamplePolymorphicStructB::StaticStruct());
	const FExamplePolymorphicStructB* RoundTripped = static_cast<const FExamplePolymorphicStructB*>(RoundTrippedBase);
	UE_NET_ASSERT_EQ(RoundTripped->SomeFloat, 7.0f);

	{
		FNetFreeDynamicStateArgs Args = {};
		Args.Version = PolymorphicNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(&ClientConfig);
		Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
		PolymorphicNetSerializer->FreeDynamicState(NetSerializationContext, Args);
	}
	FMemory::Memzero(QuantizedBuffer[0], sizeof(QuantizedBuffer[0]));
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestCacheDivergenceMissingStruct)
{
	// Server sends a struct that the client's cache does not contain (the client did not load
	// that module). The deserializer must report NetError_PolymorphicStructNetSerializer_InvalidStructType
	// rather than mis-resolving to a different struct.
	FTestPolymorphicArrayStructNetSerializerConfig ServerConfig;
	FTestPolymorphicArrayStructNetSerializerConfig ClientConfig;
	ServerConfig.RegisteredTypes.InitForType(FExamplePolymorphicStructBase::StaticStruct());
	ClientConfig.RegisteredTypes.InitForType(FExamplePolymorphicStructBase::StaticStruct());

	const FName StructAName = FPolymorphicNetSerializerScriptStructCache::MakeStructName(FExamplePolymorphicStructA::StaticStruct());
	UE_NET_ASSERT_NE(RemoveTypeFromCache(ClientConfig.RegisteredTypes, StructAName), 0);

	FExamplePolymorphicArrayStruct ServerArray;
	{
		FExamplePolymorphicStructA* StructALocal = new FExamplePolymorphicStructA;
		StructALocal->SomeInt = 99;
		ServerArray.Add(StructALocal);
	}

	FMemory::Memzero(QuantizedBuffer[0], sizeof(QuantizedBuffer[0]));

	{
		FNetQuantizeArgs Args = {};
		Args.Version = PolymorphicNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(&ServerConfig);
		Args.Target = NetSerializerValuePointer(&QuantizedBuffer[0]);
		Args.Source = NetSerializerValuePointer(&ServerArray);
		PolymorphicNetSerializer->Quantize(NetSerializationContext, Args);
	}
	UE_NET_ASSERT_FALSE(NetSerializationContext.HasError());

	Writer.InitBytes(BitStreamBuffer, sizeof(BitStreamBuffer));
	{
		FNetSerializationContext Context(&Writer);
		Context.SetInternalContext(NetSerializationContext.GetInternalContext());
		// Setup export scope to capture and serialize exports
		FNetExportContext::FBatchExports CurrentPacketBatchExports;
		FNetExports::FExportScope ExportScope = NetExports.MakeExportScope(Context, CurrentPacketBatchExports);

		FNetSerializeArgs Args = {};
		Args.Version = PolymorphicNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(&ServerConfig);
		Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
		PolymorphicNetSerializer->Serialize(Context, Args);
		Writer.CommitWrites();
		UE_NET_ASSERT_FALSE(Context.HasError());
	}

	{
		FNetFreeDynamicStateArgs Args = {};
		Args.Version = PolymorphicNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(&ServerConfig);
		Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
		PolymorphicNetSerializer->FreeDynamicState(NetSerializationContext, Args);
	}
	FMemory::Memzero(QuantizedBuffer[0], sizeof(QuantizedBuffer[0]));

	Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());
	{
		FNetSerializationContext Context(&Reader);
		Context.SetInternalContext(NetSerializationContext.GetInternalContext());
		FNetDeserializeArgs Args = {};
		Args.Version = PolymorphicNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(&ClientConfig);
		Args.Target = NetSerializerValuePointer(&QuantizedBuffer[0]);
		PolymorphicNetSerializer->Deserialize(Context, Args);
		UE_NET_ASSERT_TRUE(Context.HasError());
		UE_NET_ASSERT_EQ(Context.GetError(), NetError_PolymorphicStructNetSerializer_InvalidStructType);
	}
	FMemory::Memzero(QuantizedBuffer[0], sizeof(QuantizedBuffer[0]));
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerDeltaSerializationFixture, TestDeltaSerialization)
{
	TArray<FExamplePolymorphicArrayStruct> Values;

	{
		// Add empty value
		{
			FExamplePolymorphicArrayStruct EmptyArray;
			Values.Emplace(MoveTemp(EmptyArray));
		}

		// Add arbitrary array
		{
			FExamplePolymorphicArrayStruct ArbitraryArray;
			SetArbitraryState(ArbitraryArray);
			Values.Emplace(ArbitraryArray);
		}

		// Add "arbitrary array" without first value
		{
			FExamplePolymorphicArrayStruct ArbitraryArrayWithoutFirstElement;
			SetArbitraryState(ArbitraryArrayWithoutFirstElement);
			if (ArbitraryArrayWithoutFirstElement.Num() > 0)
			{
				ArbitraryArrayWithoutFirstElement.RemoveAt(0);
			}
			Values.Emplace(MoveTemp(ArbitraryArrayWithoutFirstElement));
		}

		// Add "arbitrary array" without last value
		{
			FExamplePolymorphicArrayStruct ArbitraryArrayWithoutLastElement;
			SetArbitraryState(ArbitraryArrayWithoutLastElement);
			if (ArbitraryArrayWithoutLastElement.Num() > 0)
			{
				ArbitraryArrayWithoutLastElement.RemoveAt(ArbitraryArrayWithoutLastElement.Num() - 1);
			}
			Values.Emplace(ArbitraryArrayWithoutLastElement);
		}
	}

	// Setup export scope to capture and serialize exports
	FNetExportContext::FBatchExports CurrentPacketBatchExports;
	FNetExports::FExportScope ExportScope = NetExports.MakeExportScope(Context, CurrentPacketBatchExports);

	Super::TestSerializeDelta(Values.GetData(), Values.Num(), Config);
}


// FTestPolymorphicArrayStructNetSerializerFixture implementation
void FTestPolymorphicArrayStructNetSerializerFixture::SetExpectedState(FExamplePolymorphicArrayStruct& Target)
{
	StructA = new FExamplePolymorphicStructA;
	StructA->SomeInt = 1;
	StructB = new FExamplePolymorphicStructB;
	StructB->SomeFloat = 1.0f;

	StructC = new FExamplePolymorphicStructC;
	StructC->SomeObjectRef = nullptr;
	StructC->SomeBool = false;

	StructD = new FExamplePolymorphicStructD;
	StructD->SomeValue = 0x7E577E57;
	
	StructD_Derived = new FExamplePolymorphicStructD_Derived;
	StructD_Derived->SomeValue = 0xC0DEC0DE;
	StructD_Derived->FloatInD_Derived = 4711.0f;

	// Add some data, we need some structs to send as well
	Target.Add(StructA);
	Target.Add(StructB);
	Target.Add(StructC);
	Target.Add(StructD);
	Target.Add(StructD_Derived);
}


void FTestPolymorphicArrayStructNetSerializerFixture::SetUp()
{
	FReplicationSystemServerClientTestFixture::SetUp();

	// Init default serialization context
	InternalNetSerializationContext.ReplicationSystem = Server->ReplicationSystem;

	FInternalNetSerializationContext TempInternalNetSerializationContext;
	FInternalNetSerializationContext::FInitParameters TempInternalNetSerializationContextInitParams;
	TempInternalNetSerializationContextInitParams.ReplicationSystem = Server->ReplicationSystem;
	TempInternalNetSerializationContextInitParams.ObjectResolveContext.RemoteNetTokenStoreState = Server->ReplicationSystem->GetNetTokenStore()->GetLocalNetTokenStoreState();
	TempInternalNetSerializationContext.Init(TempInternalNetSerializationContextInitParams);

	InternalNetSerializationContext = MoveTemp(TempInternalNetSerializationContext);
	NetSerializationContext.SetInternalContext(&InternalNetSerializationContext);

	FMemory::Memzero(QuantizedBuffer, sizeof(QuantizedBuffer));

	SetExpectedState(PolymorphicStructInstance0);

	bHasQuantizedState = false;
	bHasClonedQuantizedState = false;

	// Register custom serializer
	if (!bHasRegisteredStructDNetSerializer)
	{
		UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_ExamplePolymorphicStructD);
		UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_TestPolymorphicStructNetSerializer);

		bHasRegisteredStructDNetSerializer = true;
	}

	// Init type registry, note that we set it up for the context we provide rather than the default config
	PolymorphicNetSerializerConfig.RegisteredTypes.InitForType(FExamplePolymorphicStructBase::StaticStruct());
	PolymorphicStructNetSerializerConfig.RegisteredTypes.InitForType(FExamplePolymorphicStructBase::StaticStruct());
}

void FTestPolymorphicArrayStructNetSerializerFixture::ValidateExpectedState(const FExamplePolymorphicArrayStruct& StructInstance)
{
	UE_NET_ASSERT_EQ(PolymorphicStructInstance0.Num(), StructInstance.Num());

	auto AInstance = static_cast<const FExamplePolymorphicStructA*>(StructInstance.Get(0));
	auto BInstance = static_cast<const FExamplePolymorphicStructB*>(StructInstance.Get(1));
	auto CInstance = static_cast<const FExamplePolymorphicStructC*>(StructInstance.Get(2));
	auto DInstance = static_cast<const FExamplePolymorphicStructD*>(StructInstance.Get(3));
	auto D_DerivedInstance = static_cast<const FExamplePolymorphicStructD_Derived*>(StructInstance.Get(4));

	// Validate types
	UE_NET_ASSERT_TRUE(AInstance->GetScriptStruct() == FExamplePolymorphicStructA::StaticStruct());
	UE_NET_ASSERT_TRUE(BInstance->GetScriptStruct() == FExamplePolymorphicStructB::StaticStruct());
	UE_NET_ASSERT_TRUE(CInstance->GetScriptStruct() == FExamplePolymorphicStructC::StaticStruct());
	UE_NET_ASSERT_TRUE(DInstance->GetScriptStruct() == FExamplePolymorphicStructD::StaticStruct());
	UE_NET_ASSERT_TRUE(D_DerivedInstance->GetScriptStruct() == FExamplePolymorphicStructD_Derived::StaticStruct());

	// Validate actual data	
	UE_NET_ASSERT_EQ(StructA->SomeInt, AInstance->SomeInt);
	UE_NET_ASSERT_EQ(StructB->SomeFloat, BInstance->SomeFloat);
	UE_NET_ASSERT_EQ(StructC->SomeBool, CInstance->SomeBool);
	UE_NET_ASSERT_EQ(ToRawPtr(StructC->SomeObjectRef), ToRawPtr(CInstance->SomeObjectRef));
	UE_NET_ASSERT_EQ(StructD->SomeValue, DInstance->SomeValue);
	UE_NET_ASSERT_EQ(StructD_Derived->SomeValue, D_DerivedInstance->SomeValue);
	UE_NET_ASSERT_EQ(StructD_Derived->FloatInD_Derived, D_DerivedInstance->FloatInD_Derived);
}

void FTestPolymorphicArrayStructNetSerializerFixture::TearDown()
{
	PolymorphicStructInstance0.Clear();
	PolymorphicStructInstance1.Clear();

	FreeQuantizedState();

	FReplicationSystemServerClientTestFixture::TearDown();
}

void FTestPolymorphicArrayStructNetSerializerFixture::Serialize()
{
	// Must have run quantize before this
	UE_NET_ASSERT_TRUE(bHasQuantizedState);
	
	// Serialize data
	{
		Writer.InitBytes(BitStreamBuffer, sizeof(BitStreamBuffer));
		FNetSerializationContext Context(&Writer);
		Context.SetInternalContext(NetSerializationContext.GetInternalContext());

		// Setup export scope to capture and serialize exports
		FNetExportContext::FBatchExports CurrentPacketBatchExports;
		FNetExports::FExportScope ExportScope = NetExports.MakeExportScope(Context, CurrentPacketBatchExports);

		FNetSerializeArgs Args = {};
		Args.Version = PolymorphicNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(&PolymorphicNetSerializerConfig);
		Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
		PolymorphicNetSerializer->Serialize(Context, Args);

		Writer.CommitWrites();

		UE_NET_ASSERT_FALSE(Context.HasError());
		UE_NET_ASSERT_TRUE(Writer.GetPosBits() > 0U);
	}
}

void FTestPolymorphicArrayStructNetSerializerFixture::Deserialize()
{
	// Check pre-conditions
	UE_NET_ASSERT_FALSE(bHasQuantizedState);
	UE_NET_ASSERT_TRUE(Writer.GetPosBytes() > 0U);
	
	Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());

	FNetSerializationContext Context(&Reader);
	Context.SetInternalContext(NetSerializationContext.GetInternalContext());

	FNetDeserializeArgs Args = {};
	Args.Version = PolymorphicNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&PolymorphicNetSerializerConfig);
	Args.Target = NetSerializerValuePointer(&QuantizedBuffer[0]);
	PolymorphicNetSerializer->Deserialize(Context, Args);

	bHasQuantizedState = true;
}

void FTestPolymorphicArrayStructNetSerializerFixture::SerializeDelta()
{
	// Check pre-conditions
	UE_NET_ASSERT_TRUE(bHasQuantizedState);
	UE_NET_ASSERT_EQ(QuantizedStateCount, 2U);
	
	// Serialize data
	{
		Writer.InitBytes(BitStreamBuffer, sizeof(BitStreamBuffer));
		FNetSerializationContext Context(&Writer);
		Context.SetInternalContext(NetSerializationContext.GetInternalContext());

		// Setup export scope to capture and serialize exports
		FNetExportContext::FBatchExports CurrentPacketBatchExports;
		FNetExports::FExportScope ExportScope = NetExports.MakeExportScope(Context, CurrentPacketBatchExports);

		FNetSerializeDeltaArgs Args = {};
		Args.Version = PolymorphicNetSerializer->Version;
		Args.NetSerializerConfig = NetSerializerConfigParam(&PolymorphicNetSerializerConfig);
		Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
		Args.Prev = NetSerializerValuePointer(&QuantizedBuffer[1]);

		PolymorphicNetSerializer->SerializeDelta(Context, Args);

		Writer.CommitWrites();

		UE_NET_ASSERT_FALSE(Context.HasError());
		UE_NET_ASSERT_TRUE(Writer.GetPosBits() > 0U);
	}
}

void FTestPolymorphicArrayStructNetSerializerFixture::DeserializeDelta()
{
	// Check pre-conditions
	UE_NET_ASSERT_TRUE(Writer.GetPosBytes() > 0U);
	
	Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());

	FNetSerializationContext Context(&Reader);
	Context.SetInternalContext(NetSerializationContext.GetInternalContext());

	FNetDeserializeDeltaArgs Args = {};
	Args.Version = PolymorphicNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&PolymorphicNetSerializerConfig);
	Args.Target = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Prev = NetSerializerValuePointer(&QuantizedBuffer[1]);
	PolymorphicNetSerializer->DeserializeDelta(Context, Args);

	bHasQuantizedState = true;
	QuantizedStateCount = 1;
}

void FTestPolymorphicArrayStructNetSerializerFixture::Quantize()
{
	FNetQuantizeArgs Args = {};
	Args.Version = PolymorphicNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&PolymorphicNetSerializerConfig);
	Args.Target = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Source = NetSerializerValuePointer(&PolymorphicStructInstance0);
	PolymorphicNetSerializer->Quantize(NetSerializationContext, Args);

	bHasQuantizedState = true;
	QuantizedStateCount = 1;
}

void FTestPolymorphicArrayStructNetSerializerFixture::QuantizeTwoStates()
{
	Quantize();

	FNetQuantizeArgs Args = {};
	Args.Version = PolymorphicNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&PolymorphicNetSerializerConfig);
	Args.Target = NetSerializerValuePointer(&QuantizedBuffer[1]);
	Args.Source = NetSerializerValuePointer(&PolymorphicStructInstance1);
	PolymorphicNetSerializer->Quantize(NetSerializationContext, Args);

	bHasQuantizedState = true;
	QuantizedStateCount = 2;
}


void FTestPolymorphicArrayStructNetSerializerFixture::Clone()
{
	// Check pre-conditions
	UE_NET_ASSERT_TRUE(bHasQuantizedState);

	FNetCloneDynamicStateArgs Args = {};
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Target = NetSerializerValuePointer(&ClonedQuantizedBuffer[0]);
	Args.NetSerializerConfig = NetSerializerConfigParam(&PolymorphicNetSerializerConfig);
	PolymorphicNetSerializer->CloneDynamicState(NetSerializationContext, Args);

	bHasClonedQuantizedState = true;
	ClonedQuantizedStateCount = 1;
}

void FTestPolymorphicArrayStructNetSerializerFixture::FreeQuantizedState()
{
	FNetFreeDynamicStateArgs Args = {};
	Args.Version = PolymorphicNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&PolymorphicNetSerializerConfig);

	if (bHasQuantizedState)
	{
		for (uint32 StateIt = 0, StateEndIt = QuantizedStateCount; StateIt != StateEndIt; ++StateIt)
		{
			Args.Source = NetSerializerValuePointer(&QuantizedBuffer[StateIt]);
			PolymorphicNetSerializer->FreeDynamicState(NetSerializationContext, Args);
		}
		bHasQuantizedState = false;
	}
		
	if (bHasClonedQuantizedState)
	{
		for (uint32 StateIt = 0, StateEndIt = ClonedQuantizedStateCount; StateIt != StateEndIt; ++StateIt)
		{
			Args.Source = NetSerializerValuePointer(&ClonedQuantizedBuffer[StateIt]);
			PolymorphicNetSerializer->FreeDynamicState(NetSerializationContext, Args);
		}
		bHasClonedQuantizedState = false;
	}
}

void FTestPolymorphicArrayStructNetSerializerFixture::Dequantize()
{
	UE_NET_ASSERT_TRUE(bHasQuantizedState);

	FNetDequantizeArgs Args = {};
	Args.Version = PolymorphicNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&PolymorphicNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(&QuantizedBuffer[0]);
	Args.Target = NetSerializerValuePointer(&PolymorphicStructInstance1);
	PolymorphicNetSerializer->Dequantize(NetSerializationContext, Args);
}

bool FTestPolymorphicArrayStructNetSerializerFixture::IsEqual(bool bQuantized)
{
	if (bQuantized)
	{
		UE_NET_EXPECT_TRUE(bHasQuantizedState);
		if (!bHasQuantizedState)
		{
			return false;
		}

		UE_NET_EXPECT_TRUE(bHasClonedQuantizedState);
		if (!bHasClonedQuantizedState)
		{
			return false;
		}
	}

	FNetSerializationContext Context;
	FNetIsEqualArgs Args = {};
	Args.Version = PolymorphicNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&PolymorphicNetSerializerConfig);
	Args.Source0 = bQuantized ? NetSerializerValuePointer(&QuantizedBuffer[0]) : NetSerializerValuePointer(&PolymorphicStructInstance0);
	Args.Source1 = bQuantized ? NetSerializerValuePointer(&ClonedQuantizedBuffer[0]) : NetSerializerValuePointer(&PolymorphicStructInstance1);
	Args.bStateIsQuantized = bQuantized;

	return PolymorphicNetSerializer->IsEqual(Context, Args);
}

void FTestPolymorphicArrayStructNetSerializerFixture::Validate()
{
	FNetValidateArgs Args = {};
	Args.Version = PolymorphicNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&PolymorphicNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(&PolymorphicStructInstance0);

	PolymorphicNetSerializer->Validate(NetSerializationContext, Args);
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestPolymorphicStructNetSerializer_Modify)
{
	// InitTypeCache as we do actual serialization
	FTestPolymorphicStructNetSerializer::InitTypeCache();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestPolymorphicStructNetSerializer_TestObject* ServerObject = Server->CreateObject<UTestPolymorphicStructNetSerializer_TestObject>();
	
	ServerObject->PolyStruct.Raise<FExamplePolymorphicStructB>();
	ServerObject->PolyStruct.GetAs<FExamplePolymorphicStructB>().SomeFloat = 12.0f;

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	auto ClientObject = Client->GetObjectAs<UTestPolymorphicStructNetSerializer_TestObject>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	CA_ASSUME(ClientObject != nullptr);
	UE_NET_ASSERT_EQ(ClientObject->PolyStruct.GetAs<FExamplePolymorphicStructB>().SomeFloat, 12.0f);

	// Modify
	ServerObject->PolyStruct.GetAs<FExamplePolymorphicStructB>().SomeFloat += 1.0f;

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verify that we detected the modification
	UE_NET_ASSERT_TRUE(ClientObject->PolyStruct.GetAs<FExamplePolymorphicStructB>().SomeFloat == ServerObject->PolyStruct.GetAs<FExamplePolymorphicStructB>().SomeFloat);

	// Switch type
	ServerObject->PolyStruct.Raise<FExamplePolymorphicStructD>();
	ServerObject->PolyStruct.GetAs<FExamplePolymorphicStructD>().SomeValue = 100;

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	UE_NET_ASSERT_TRUE(ClientObject->PolyStruct.GetAs<FExamplePolymorphicStructD>().SomeValue == ServerObject->PolyStruct.GetAs<FExamplePolymorphicStructD>().SomeValue);	
}

UE_NET_TEST_FIXTURE(FTestPolymorphicArrayStructNetSerializerFixture, TestPolymorphicStructNetSerializer_ModifyFastArray)
{
	// InitTypeCache as we do actual serialization
	FTestPolymorphicStructNetSerializer::InitTypeCache();

	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestPolymorphicStructNetSerializer_TestObject* ServerObject = Server->CreateObject<UTestPolymorphicStructNetSerializer_TestObject>();

	// Add a few entries for the fastarray
	FExamplePolymorphicStructFastArrayItem Item;

	Item.PolyStruct.Raise<FExamplePolymorphicStructA>();
	ServerObject->PolyStructFastArray.Edit().Add(Item);

	Item.PolyStruct.Raise<FExamplePolymorphicStructB>();
	ServerObject->PolyStructFastArray.Edit().Add(Item);

	Item.PolyStruct.Raise<FExamplePolymorphicStructC>();
	ServerObject->PolyStructFastArray.Edit().Add(Item);

	Item.PolyStruct.Raise<FExamplePolymorphicStructD>();
	ServerObject->PolyStructFastArray.Edit().Add(Item);
	ServerObject->PolyStructFastArray.Edit()[3].PolyStruct.GetAs<FExamplePolymorphicStructD>().SomeValue = 100;
	
	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();
	
	auto ClientObject = Client->GetObjectAs<UTestPolymorphicStructNetSerializer_TestObject>(ServerObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject, nullptr);
	CA_ASSUME(ClientObject != nullptr);
	UE_NET_ASSERT_EQ(ClientObject->PolyStructFastArray.GetItemArray().Num(), ServerObject->PolyStructFastArray.GetItemArray().Num());
	UE_NET_ASSERT_EQ(ClientObject->PolyStructFastArray.GetItemArray()[3].PolyStruct.GetAs<FExamplePolymorphicStructD>().SomeValue, 100U);

	// Modify value and see that it is replicated as expected
	ServerObject->PolyStructFastArray.Edit()[3].PolyStruct.GetAs<FExamplePolymorphicStructD>().SomeValue += 3;

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verified that the client got the modified value
	UE_NET_ASSERT_EQ(ClientObject->PolyStructFastArray.GetItemArray()[3].PolyStruct.GetAs<FExamplePolymorphicStructD>().SomeValue, 103U);

	// Switch type
	ServerObject->PolyStructFastArray.Edit()[2].PolyStruct.Raise<FExamplePolymorphicStructD>();
	ServerObject->PolyStructFastArray.Edit()[2].PolyStruct.GetAs<FExamplePolymorphicStructD>().SomeValue = 1;

	// Replicate
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, DeliverPacket);
	Server->PostSendUpdate();

	// Verified that the client got the modified value
	UE_NET_ASSERT_EQ(ClientObject->PolyStructFastArray.GetItemArray()[2].PolyStruct.GetAs<FExamplePolymorphicStructD>().SomeValue, 1U);
}

// FTestPolymorphicArrayStructNetSerializerDeltaSerializationFixture implementation
void FTestPolymorphicArrayStructNetSerializerDeltaSerializationFixture::SetUp()
{
	Super::SetUp();

	Config.RegisteredTypes.InitForType(FExamplePolymorphicStructBase::StaticStruct());

	Server = new FReplicationSystemTestServer(GetName());

	// Init NetSerializationContext
	FInternalNetSerializationContext TempInternalNetSerializationContext;
	FInternalNetSerializationContext::FInitParameters TempInternalNetSerializationContextInitParams;
	TempInternalNetSerializationContextInitParams.ReplicationSystem = Server->ReplicationSystem;
	TempInternalNetSerializationContextInitParams.ObjectResolveContext.RemoteNetTokenStoreState = Server->ReplicationSystem->GetNetTokenStore()->GetLocalNetTokenStoreState();
	TempInternalNetSerializationContext.Init(TempInternalNetSerializationContextInitParams);

	InternalNetSerializationContext = MoveTemp(TempInternalNetSerializationContext);
	Context.SetInternalContext(&InternalNetSerializationContext);
}

void FTestPolymorphicArrayStructNetSerializerDeltaSerializationFixture::TearDown()
{
	delete Server;
	Server = nullptr;

	Super::TearDown();
}

void FTestPolymorphicArrayStructNetSerializerDeltaSerializationFixture::SetArbitraryState(FExamplePolymorphicArrayStruct& Target)
{
	FExamplePolymorphicStructA* StructA = new FExamplePolymorphicStructA;
	StructA->SomeInt = -29837492;
	
	FExamplePolymorphicStructB* StructB = new FExamplePolymorphicStructB;
	StructB->SomeFloat = 17.0f;

	FExamplePolymorphicStructC* StructC = new FExamplePolymorphicStructC;
	StructC->SomeObjectRef = nullptr;
	StructC->SomeBool = false;

	FExamplePolymorphicStructD* StructD = new FExamplePolymorphicStructD;
	StructD->SomeValue = 0xBEEEEEEF;

	FExamplePolymorphicStructD_Derived* StructD_Derived = new FExamplePolymorphicStructD_Derived;
	StructD_Derived->SomeValue = 0xDEEDDEED;
	StructD_Derived->FloatInD_Derived = 12345.0f;

	// Add values to target
	Target.Add(StructA);
	Target.Add(StructB);
	Target.Add(StructC);
	Target.Add(StructD_Derived);
	Target.Add(StructD);
}

}

void FExamplePolymorphicStructFastArrayItem::PostReplicatedAdd(const struct FExamplePolymorphicStructFastArraySerializer& InArraySerializer)
{
	const_cast<FExamplePolymorphicStructFastArraySerializer&>(InArraySerializer).bHitReplicatedAdd = true;
}

void FExamplePolymorphicStructFastArrayItem::PostReplicatedChange(const struct FExamplePolymorphicStructFastArraySerializer& InArraySerializer)
{
	const_cast<FExamplePolymorphicStructFastArraySerializer&>(InArraySerializer).bHitReplicatedChange = true;
}

void FExamplePolymorphicStructFastArrayItem::PreReplicatedRemove(const struct FExamplePolymorphicStructFastArraySerializer& InArraySerializer)
{
	const_cast<FExamplePolymorphicStructFastArraySerializer&>(InArraySerializer).bHitReplicatedRemove = true;
}

UTestPolymorphicStructNetSerializer_TestObject::UTestPolymorphicStructNetSerializer_TestObject()
: UReplicatedTestObject()
{
}

void UTestPolymorphicStructNetSerializer_TestObject::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty>& OutLifetimeProps) const
{
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = false;

	DOREPLIFETIME_WITH_PARAMS_FAST(UTestPolymorphicStructNetSerializer_TestObject, PolyStruct, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UTestPolymorphicStructNetSerializer_TestObject, PolyStructFastArray, Params);
}

void UTestPolymorphicStructNetSerializer_TestObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}
