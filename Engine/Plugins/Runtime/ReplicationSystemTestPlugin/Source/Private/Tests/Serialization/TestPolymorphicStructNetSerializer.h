// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/PolymorphicNetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "Iris/ReplicationState/IrisFastArraySerializer.h"
#include "Iris/ReplicationState/Private/IrisFastArraySerializerInternal.h"
#include "TestPolymorphicStructNetSerializer.generated.h"

// Example BaseStruct used to support replication of polymorphic structs
// Note: this is not an optimal setup but is needed to ensure that we support existing setups used by existing engine code.
USTRUCT()
struct FExamplePolymorphicStructBase
{
	GENERATED_BODY()

	virtual ~FExamplePolymorphicStructBase() { }

	/** Returns the serialization data, must always be overridden */
	virtual UScriptStruct* GetScriptStruct() const
	{
		return FExamplePolymorphicStructBase::StaticStruct();
	}
};

//////////////////////////////////////////////////////////////////////////

// Test struct based on examples from existing engine code, 
// This is a very specific type of replication where structs contained in the array only are replicated based on the stored pointer.
// i.e. it does not compare the actual instance data stored in the polymorphic struct, but instead expects user to explicitly clone and insert a new struct to trigger replication of the data contained in the struct
USTRUCT()
struct FExamplePolymorphicArrayStruct
{
	GENERATED_BODY()

	FExamplePolymorphicArrayStruct() { }

	FExamplePolymorphicArrayStruct(FExamplePolymorphicStructBase* DataPtr)
	{
		Data.Add(TSharedPtr<FExamplePolymorphicStructBase>(DataPtr));
	}

	FExamplePolymorphicArrayStruct(FExamplePolymorphicArrayStruct&& Other) : Data(MoveTemp(Other.Data))	{ }
	FExamplePolymorphicArrayStruct(const FExamplePolymorphicArrayStruct& Other) : Data(Other.Data) { }

	FExamplePolymorphicArrayStruct& operator=(FExamplePolymorphicArrayStruct&& Other) { Data = MoveTemp(Other.Data); return *this; }
	FExamplePolymorphicArrayStruct& operator=(const FExamplePolymorphicArrayStruct& Other) { Data = Other.Data; return *this; }

public:
	void Clear()
	{
		Data.Reset();
	}

	int32 Num() const
	{
		return Data.Num();
	}

	/** Returns true if index is valid */
	bool IsValid(int32 Index) const
	{
		return (Index < Data.Num() && Data[Index].IsValid());
	}

	/** Returns data at index, or nullptr if invalid */
	const FExamplePolymorphicStructBase* Get(int32 Index) const
	{
		return IsValid(Index) ? Data[Index].Get() : nullptr;
	}

	/** Returns data at index, or nullptr if invalid */
	FExamplePolymorphicStructBase* Get(int32 Index)
	{
		return IsValid(Index) ? Data[Index].Get() : nullptr;
	}

	void RemoveAt(int32 Index)
	{
		if (Index > 0 && Index < Num())
		{
			Data.RemoveAt(Index);
		}
	}

	/** Adds a new FExamplePolymorphicArrayStruct data to FExamplePolymorphicArrayStruct, it must have been created with new */
	void Add(FExamplePolymorphicStructBase* DataPtr)
	{
		Data.Add(TSharedPtr<FExamplePolymorphicStructBase>(DataPtr));
	}

	/** Does a shallow copy of data from one handle to another */
	void Append(const FExamplePolymorphicArrayStruct& OtherHandle)
	{
		Data.Append(OtherHandle.Data);
	}

	/** Comparison operator */
	bool operator==(const FExamplePolymorphicArrayStruct& Other) const
	{
		// This comparison operator only compares actual instance pointers
		if (Data.Num() != Other.Data.Num())
		{
			return false;
		}

		for (int32 It = 0, EndIt = Data.Num(); It < EndIt; ++It)
		{
			if (Data[It].IsValid() != Other.Data[It].IsValid())
			{
				return false;
			}
			if (Data[It].Get() != Other.Data[It].Get())
			{
				return false;
			}
		}
		return true;
	}

	/** Comparison operator */
	bool operator!=(const FExamplePolymorphicArrayStruct& Other) const
	{
		return !(FExamplePolymorphicArrayStruct::operator==(Other));
	}

	/** For PolymorphicStructArrayNetSerializer */
	static TArrayView<TSharedPtr<FExamplePolymorphicStructBase>> GetArray(FExamplePolymorphicArrayStruct& ArrayContainer)
	{
		return MakeArrayView(ArrayContainer.Data);
	}

	static void SetArrayNum(FExamplePolymorphicArrayStruct& ArrayContainer, SIZE_T Num)
	{
		ArrayContainer.Data.SetNum(IntCastChecked<int32>(Num));
	}

private:
	/** Raw storage of target data, do not modify this directly */
	TArray<TSharedPtr<FExamplePolymorphicStructBase>, TInlineAllocator<1> >	Data;
};

template<>
struct TStructOpsTypeTraits<FExamplePolymorphicArrayStruct> : public TStructOpsTypeTraitsBase2<FExamplePolymorphicArrayStruct>
{
	enum
	{
		WithCopy = true,		// Necessary so that TSharedPtr<FExamplePolymorphicArrayItem> Data is copied around
		WithIdenticalViaEquality = true,
	};
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FTestPolymorphicArrayStructNetSerializer, REPLICATIONSYSTEMTESTPLUGIN_API);

}

USTRUCT()
struct FTestPolymorphicArrayStructNetSerializerConfig : public FPolymorphicArrayStructNetSerializerConfig
{
	GENERATED_BODY()
};

//////////////////////////////////////////////////////////////////////////

// Example of how to setup a Polymorphic struct that can work as a replicated property
USTRUCT()
struct FExamplePolymorphicStruct
{
	GENERATED_BODY()

	FExamplePolymorphicStruct() { }

	FExamplePolymorphicStruct(const FExamplePolymorphicStruct& Other)
	{
		*this = Other;
	}

	FExamplePolymorphicStruct& operator=(FExamplePolymorphicStruct&& Other)
	{
		Data = MoveTemp(Other.Data); return *this;
	}

	FExamplePolymorphicStruct& operator=(const FExamplePolymorphicStruct& Other) 
	{
		// When used as a replicated property we need to do deep copy as we store copies of the state in order to detect changes
		if (this != &Other)
		{
			const UScriptStruct* DstScriptStruct = Data.IsValid() ? Data->GetScriptStruct() : nullptr;
			const UScriptStruct* SrcScriptStruct = Other.Data.IsValid() ? Other.Data->GetScriptStruct() : nullptr;

			const FExamplePolymorphicStructBase* SrcPolymorphicStruct = Other.Data.IsValid() ? Other.Data.Get() : nullptr;
			if (SrcPolymorphicStruct)
			{
				CA_ASSUME(SrcScriptStruct != nullptr);
				FExamplePolymorphicStructBase* const DstPolymorphicStruct = Data.IsValid() ? Data.Get() : nullptr;
				if (DstPolymorphicStruct == SrcPolymorphicStruct)
				{
					SrcScriptStruct->CopyScriptStruct(DstPolymorphicStruct, SrcPolymorphicStruct);
				}
				else
				{
					FExamplePolymorphicStructBase* NewPolymorphicStruct = static_cast<FExamplePolymorphicStructBase*>(FMemory::Malloc(SrcScriptStruct->GetStructureSize(), SrcScriptStruct->GetMinAlignment()));
					SrcScriptStruct->InitializeStruct(NewPolymorphicStruct);
					SrcScriptStruct->CopyScriptStruct(NewPolymorphicStruct, SrcPolymorphicStruct);
					Data = MakeShareable(NewPolymorphicStruct);								
				}				
			}
			else
			{
				Data.Reset();
			}
		}

		return *this;
	}

	void Reset()
	{
		Data.Reset();
	}

	bool IsValid() const
	{
		return Data.IsValid();
	}

	/** Initalize Data to a struct of wanted type */
	template<typename T>
	void Raise()
	{
		Data = TSharedPtr<T>(new T);
	}

	/** Get data as expected type */
	template<typename T>
	T& GetAs()
	{
		check(Data.Get()->GetScriptStruct() == T::StaticStruct());
		return *static_cast<T*>(Data.Get());
	}

	/** Comparison operator that compares instance data */
	bool operator==(const FExamplePolymorphicStruct& Other) const
	{
		// When used as replicated property and we want to detect changes of the intance data we need to compare the data stored in the inner structs.
		const FExamplePolymorphicStructBase* PolymorphicStruct = Data.Get();
		const FExamplePolymorphicStructBase* OtherPolymorphicStruct = Other.Data.Get();
		
		const UScriptStruct* PolymorphicStructScriptStruct = PolymorphicStruct ? PolymorphicStruct->GetScriptStruct() : nullptr;
		const UScriptStruct* OtherPolymorphicStructScriptStruct = OtherPolymorphicStruct ? OtherPolymorphicStruct->GetScriptStruct() : nullptr;

		if (PolymorphicStructScriptStruct != OtherPolymorphicStructScriptStruct)
		{
			return false;
		}

		if (PolymorphicStructScriptStruct)
		{
			// Compare actual struct data	
			return PolymorphicStructScriptStruct->CompareScriptStruct(PolymorphicStruct, OtherPolymorphicStruct, 0);
		}
		else
		{
			return false;
		}		
	}

	bool operator!=(const FExamplePolymorphicStruct& Other) const
	{
		return !(FExamplePolymorphicStruct::operator==(Other));
	}

	static TSharedPtr<FExamplePolymorphicStructBase>& GetItem(FExamplePolymorphicStruct& Source)
	{
		return Source.Data;
	}

private:
	TSharedPtr<FExamplePolymorphicStructBase> Data;
};

template<>
struct TStructOpsTypeTraits<FExamplePolymorphicStruct> : public TStructOpsTypeTraitsBase2<FExamplePolymorphicStruct>
{
	enum
	{
		WithCopy = true,					// Necessary to do a deep copy
		WithIdenticalViaEquality = true,	// We have a custom compare operator
	};
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FTestPolymorphicStructNetSerializer, REPLICATIONSYSTEMTESTPLUGIN_API);

}

USTRUCT()
struct FTestPolymorphicStructNetSerializerConfig : public FPolymorphicStructNetSerializerConfig
{
	GENERATED_BODY()
};

//////////////////////////////////////////////////////////////////////////

// FastArray declaration to test using a Polymorphic struct in a fastarray.
USTRUCT()
struct FExamplePolymorphicStructFastArrayItem : public FFastArraySerializerItem
{
	GENERATED_BODY()

	UPROPERTY()
	FExamplePolymorphicStruct PolyStruct;

	// Callbacks
	void PostReplicatedAdd(const struct FExamplePolymorphicStructFastArraySerializer& InArraySerializer);
	void PostReplicatedChange(const struct FExamplePolymorphicStructFastArraySerializer& InArraySerializer);
	void PreReplicatedRemove(const struct FExamplePolymorphicStructFastArraySerializer& InArraySerializer);
};

USTRUCT()
struct FExamplePolymorphicStructFastArraySerializer : public FIrisFastArraySerializer
{
	GENERATED_BODY()

	FExamplePolymorphicStructFastArraySerializer()
	: FIrisFastArraySerializer()
	, bHitReplicatedAdd(false)
	, bHitReplicatedChange(false)
	, bHitReplicatedRemove(false)
	, bHitPostReplicatedReceive(false)
	, bPostReplicatedReceiveWasHitWithUnresolvedReferences(false)
	{
	}

	typedef TArray<FExamplePolymorphicStructFastArrayItem> ItemArrayType;
	const ItemArrayType& GetItemArray() const
	{
		return Items;
	}

	ItemArrayType& GetItemArray()
	{
		return Items;
	}

	typedef UE::Net::TIrisFastArrayEditor<FExamplePolymorphicStructFastArraySerializer> FFastArrayEditor;
	FFastArrayEditor Edit()
	{
		return FFastArrayEditor(*this);
	}
	
	void PostReplicatedReceive(const FFastArraySerializer::FPostReplicatedReceiveParameters& Parameters)
	{
		bHitPostReplicatedReceive = 1U;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bPostReplicatedReceiveWasHitWithUnresolvedReferences = Parameters.bHasMoreUnmappedReferences;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

public:
	uint8 bHitReplicatedAdd    : 1;
	uint8 bHitReplicatedChange : 1;
	uint8 bHitReplicatedRemove : 1;
	uint8 bHitPostReplicatedReceive : 1;

	bool bPostReplicatedReceiveWasHitWithUnresolvedReferences;

private:
	UPROPERTY()
	TArray<FExamplePolymorphicStructFastArrayItem> Items;
};

//////////////////////////////////////////////////////////////////////////

// Polymorphic structs based on FExamplePolymorphicStructBase used by test code
USTRUCT()
struct FExamplePolymorphicStructA : public FExamplePolymorphicStructBase
{
	GENERATED_BODY()

	UPROPERTY()
	int32 SomeInt;

	/** Returns the serialization data, must always be overridden */
	virtual UScriptStruct* GetScriptStruct() const override
	{ 
		return FExamplePolymorphicStructA::StaticStruct();
	}
};

USTRUCT()
struct FExamplePolymorphicStructB : public FExamplePolymorphicStructBase
{
	GENERATED_BODY()

	UPROPERTY()
	float SomeFloat;

	/** Returns the serialization data, must always be overridden */
	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FExamplePolymorphicStructB::StaticStruct();
	}
};

USTRUCT()
struct FExamplePolymorphicStructC : public FExamplePolymorphicStructBase
{
	GENERATED_BODY()

	UPROPERTY()
	bool SomeBool;

	UPROPERTY()
	TObjectPtr<UObject> SomeObjectRef = nullptr;

	/** Returns the serialization data, must always be overridden */
	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FExamplePolymorphicStructC::StaticStruct();
	}
};

// Example of a struct that does not have any properties, but relies on an explicit NetSerializer for serialization
USTRUCT()
struct FExamplePolymorphicStructD : public FExamplePolymorphicStructBase
{
	GENERATED_BODY()
	
	// Since we do not have any properties, we need a custom compare operator and mark the struct with WithIdenticalViaEquality Trait
	// Alternatively we could mark SomeValue as a non-replicated property (UProperty(NotReplicated))
	bool operator==(const FExamplePolymorphicStructD& Other) const { return SomeValue == Other.SomeValue; }

	uint32 SomeValue;

	/** Returns the serialization data, must always be overridden */
	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FExamplePolymorphicStructD::StaticStruct();
	}
};

// Since this struct does not have any properties, it cannot be compared using properties and we must therefore set the trait to use native compare
template<>
struct TStructOpsTypeTraits<FExamplePolymorphicStructD> : public TStructOpsTypeTraitsBase2<FExamplePolymorphicStructD>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};

// Derives from a struct with custom NetSerializer and extra properties
USTRUCT()
struct FExamplePolymorphicStructD_Derived : public FExamplePolymorphicStructD
{
	GENERATED_BODY()

	UPROPERTY()
	float FloatInD_Derived = 0.0f;

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FExamplePolymorphicStructD_Derived::StaticStruct();
	}
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FExamplePolymorphicStructDNetSerializer, REPLICATIONSYSTEMTESTPLUGIN_API);

}

USTRUCT()
struct FExamplePolymorphicStructDNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

//////////////////////////////////////////////////////////////////////////

// Test Object using FExamplePolymorphicStruct and FExamplePolymorphicStructFastArraySerializer used in tests
UCLASS()
class UTestPolymorphicStructNetSerializer_TestObject : public UReplicatedTestObject
{
	GENERATED_BODY()
public:
	UTestPolymorphicStructNetSerializer_TestObject();

	void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Fragments, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;

	UPROPERTY(Replicated)
	FExamplePolymorphicStruct PolyStruct;

	UPROPERTY(Replicated)
	FExamplePolymorphicStructFastArraySerializer PolyStructFastArray;
};
