// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/QuatNetSerializers.h"
#include "Math/UnrealMathUtility.h"
#include "Traits/IntType.h"

namespace UE::Net
{

FTestMessage& operator<<(FTestMessage& Message, const FQuat4f& Value)
{
	return Message << Value.ToString();
}

FTestMessage& operator<<(FTestMessage& Message, const FQuat4d& Value)
{
	return Message << Value.ToString();
}

}

namespace UE::Net::Private
{

static FTestMessage& PrintQuatNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

template<typename QuatType>
class FTestQuatNetSerializerBase : public TTestNetSerializerFixture<PrintQuatNetSerializerConfig, QuatType>
{
	typedef TTestNetSerializerFixture<PrintQuatNetSerializerConfig, QuatType> Super;
	using FloatType = decltype(QuatType::X);
	using RotatorType = typename Math::TRotator<FloatType>;
	using UintType = typename TUnsignedIntType<sizeof(FloatType)>::Type;

protected:
	FTestQuatNetSerializerBase(const FNetSerializer& NetSerializer, const FNetSerializerConfig& NetSerializerConfig);

	void TestIsEqual();
	void TestSerialize(FloatType MaxComponentAbsDiff, int32 MaxComponentUlpDiff);
	void TestValidate();

	static FloatType DegToRad(FloatType Value);
	static FloatType RadToDeg(FloatType Value);
	static bool SignBit(FloatType Value);

	static bool IsAlmostEqual(NetSerializerValuePointer Value0, NetSerializerValuePointer Value1, FloatType MaxComponentAbsDiff, int32 MaxComponentUlpDiff);

	void InitializeClass();

protected:
	TArray<QuatType> Values;

	const FNetSerializerConfig& NetSerializerConfig;
};

class FTestUnitQuat4fNetSerializer : public FTestQuatNetSerializerBase<FQuat4f>
{
	typedef FTestQuatNetSerializerBase<FQuat4f> Super;

public:
	FTestUnitQuat4fNetSerializer();
};

class FTestUnitQuat4dNetSerializer : public FTestQuatNetSerializerBase<FQuat4d>
{
	typedef FTestQuatNetSerializerBase<FQuat4d> Super;

public:
	FTestUnitQuat4dNetSerializer();
};

// UnitQuat4fNetSerializer tests
UE_NET_TEST_FIXTURE(FTestUnitQuat4fNetSerializer, TestSerialize)
{
	// Tolerate a bit more than 2^-23 difference.
	constexpr float MaxValueDiff = 0.0000002f;
	constexpr int32 MaxUlpDiff = 1;
	TestSerialize(MaxValueDiff, MaxUlpDiff);
}

UE_NET_TEST_FIXTURE(FTestUnitQuat4fNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestUnitQuat4fNetSerializer, TestValidate)
{
	TestValidate();
}

// UnitQuat4dNetSerializer tests
UE_NET_TEST_FIXTURE(FTestUnitQuat4dNetSerializer, TestSerialize)
{
	// Tolerate a bit more than 2^-52 difference.
	constexpr double MaxValueDiff = 4E-16;
	constexpr int32 MaxUlpDiff = 1;
	TestSerialize(MaxValueDiff, MaxUlpDiff);
}

UE_NET_TEST_FIXTURE(FTestUnitQuat4dNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestUnitQuat4dNetSerializer, TestValidate)
{
	TestValidate();
}

//
template<typename QuatType>
FTestQuatNetSerializerBase<QuatType>::FTestQuatNetSerializerBase(const FNetSerializer& NetSerializer, const FNetSerializerConfig& InNetSerializerConfig)
: Super(NetSerializer)
, NetSerializerConfig(InNetSerializerConfig)
{
	InitializeClass();
}

template<typename QuatType>
void FTestQuatNetSerializerBase<QuatType>::TestSerialize(FloatType MaxComponentAbsDiff, int32 MaxComponentUlpDiff)
{
	const auto& EqualityFunc = [MaxComponentAbsDiff, MaxComponentUlpDiff](NetSerializerValuePointer Value0, NetSerializerValuePointer Value1) -> bool { return IsAlmostEqual(Value0, Value1, MaxComponentAbsDiff, MaxComponentUlpDiff); };

	for (const bool bQuantizedCompare : {false, true})
	{
		TOptional<TFunctionRef<bool(NetSerializerValuePointer Value0, NetSerializerValuePointer Value1)>> CompareFunc;
		if (!bQuantizedCompare)
		{
			CompareFunc = EqualityFunc;
		}
		Super::TestSerialize(Values.GetData(), Values.GetData(), Values.Num(), NetSerializerConfig, bQuantizedCompare, CompareFunc);
	}
}

template<typename QuatType>
void FTestQuatNetSerializerBase<QuatType>::TestIsEqual()
{
	TArray<QuatType> CompareValues[2];
	TArray<bool> ExpectedResults[2];
	const int32 ValueCount = Values.Num();

	{
		// Make sure nothing messes with our test values.
		CompareValues[0].SetNumUninitialized(Values.Num());
		FPlatformMemory::Memcpy(CompareValues[0].GetData(), Values.GetData(), Values.Num()*sizeof(QuatType));
		ExpectedResults[0].Init(true, ValueCount);
	}

	{
		CompareValues[1].SetNumUninitialized(ValueCount);
		ExpectedResults[1].Init(false, ValueCount);
		for (int32 ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
		{
			// Make sure nothing messes with our test values.
			FPlatformMemory::Memcpy(CompareValues[1].GetData() + ValueIt, Values.GetData() + ((ValueIt + 1) % ValueCount), sizeof(QuatType));
		}
	}

	for (SIZE_T TestRoundIt = 0, TestRoundEndIt = 2; TestRoundIt != TestRoundEndIt; ++TestRoundIt)
	{
		constexpr bool bQuantizedCompare = true;
		bool bSuccess = Super::TestIsEqual(Values.GetData(), CompareValues[TestRoundIt].GetData(), ExpectedResults[TestRoundIt].GetData(), ValueCount, NetSerializerConfig, bQuantizedCompare);
		if (!bSuccess)
		{
			return;
		}
	}
}

template<typename QuatType>
void FTestQuatNetSerializerBase<QuatType>::TestValidate()
{
	// Test conforming values
	{
		TArray<bool> ExpectedResults;
		ExpectedResults.Init(true, Values.Num());

		const bool bSuccess = Super::TestValidate(Values.GetData(), ExpectedResults.GetData(), Values.Num(), NetSerializerConfig);
		if (!bSuccess)
		{
			return;
		}
	}
}

template<typename QuatType>
inline typename FTestQuatNetSerializerBase<QuatType>::FloatType FTestQuatNetSerializerBase<QuatType>::DegToRad(FloatType Value)
{
	return Value*PI/180.0f;
}

template<typename QuatType>
inline typename FTestQuatNetSerializerBase<QuatType>::FloatType FTestQuatNetSerializerBase<QuatType>::RadToDeg(FloatType Value)
{
	return Value*180.0f/PI;
}

template<typename QuatType>
inline bool FTestQuatNetSerializerBase<QuatType>::SignBit(FloatType Value)
{
	union FFloatAndUint
	{
		FloatType Float;
		UintType Uint;
	};

	FFloatAndUint FloatToInt;
	FloatToInt.Float = Value;
	return FloatToInt.Uint >> (sizeof(UintType)*8U - 1);
}

template<typename QuatType>
bool FTestQuatNetSerializerBase<QuatType>::IsAlmostEqual(NetSerializerValuePointer Value0, NetSerializerValuePointer Value1, FloatType MaxComponentAbsDiff, int32 MaxComponentUlpDiff)
{
	const QuatType V0 =  *reinterpret_cast<QuatType*>(Value0);
	const QuatType V1 =  *reinterpret_cast<QuatType*>(Value1);

	const bool bAreUnitQuats = V0.IsNormalized() && V1.IsNormalized();
	const bool WSignIsEqual = SignBit(V0.W) == SignBit(V1.W);

	const FloatType AbsDiffX = FMath::Abs(V0.X - V1.X);
	const FloatType AbsDiffY = FMath::Abs(V0.Y - V1.Y);
	const FloatType AbsDiffZ = FMath::Abs(V0.Z - V1.Z);

	const bool EqualUlpX = FMath::IsNearlyEqualByULP(V0.X, V1.X, MaxComponentUlpDiff);
	const bool EqualUlpY = FMath::IsNearlyEqualByULP(V0.Y, V1.Y, MaxComponentUlpDiff);
	const bool EqualUlpZ = FMath::IsNearlyEqualByULP(V0.Z, V1.Z, MaxComponentUlpDiff);
	if (bAreUnitQuats && WSignIsEqual && ((AbsDiffX <= MaxComponentAbsDiff) || EqualUlpX) && ((AbsDiffY <= MaxComponentAbsDiff) || EqualUlpY) && ((AbsDiffZ <= MaxComponentAbsDiff) || EqualUlpZ))
	{
		return true;
	}
		
	return false;
}

template<typename QuatType>
void FTestQuatNetSerializerBase<QuatType>::InitializeClass()
{	
	Values.Reserve(1024);

	Values.Add(QuatType(1.0f, 0.0f, 0.0f, 0.0f));
	Values.Add(QuatType(-1.0f, 0.0f, 0.0f, 0.0f));
	Values.Add(QuatType(0.0f, 1.0f, 0.0f, 0.0f));
	Values.Add(QuatType(0.0f, -1.0f, 0.0f, 0.0f));
	Values.Add(QuatType(0.0f, 0.0f, 1.0f, 0.0f));
	Values.Add(QuatType(0.0f, 0.0f, -1.0f, 0.0f));
	Values.Add(QuatType(0.0f, 0.0f, 0.0f, 1.0f));
	Values.Add(QuatType(0.0f, 0.0f, 0.0f, -1.0f));

	{
		RotatorType Rotator;
		const float Angles[] = { 0.0f, 45.0f, 90.0f, 135.0f, 180.0f, 225.0f, 270.0f, 315.0f };
		for (const float Pitch : Angles)
		{
			Rotator.Pitch = Pitch;
			for (const float Yaw : Angles)
			{
				Rotator.Yaw = Yaw;
				for (const float Roll : Angles)
				{
					Rotator.Roll = Roll;
					Values.Add(QuatType(Rotator));
				}
			}
		}
	}

	{
		RotatorType Rotator;

		Rotator.Pitch = 1.0f;
		Rotator.Yaw = 2.0f;
		Rotator.Roll = 3.0f;
		Values.Add(QuatType(Rotator));

		Rotator.Pitch = 357.777777f;
		Rotator.Yaw = 358.888888f;
		Rotator.Roll = 359.999999f;
		Values.Add(QuatType(Rotator));
	}
}

// FTestUnitQuat4fNetSerializer implementation
FTestUnitQuat4fNetSerializer::FTestUnitQuat4fNetSerializer()
: Super(UE_NET_GET_SERIALIZER(FUnitQuat4fNetSerializer), *UE_NET_GET_SERIALIZER(FUnitQuat4fNetSerializer).DefaultConfig)
{
}

// FTestUnitQuat4dNetSerializer implementation
FTestUnitQuat4dNetSerializer::FTestUnitQuat4dNetSerializer()
: Super(UE_NET_GET_SERIALIZER(FUnitQuat4dNetSerializer), *UE_NET_GET_SERIALIZER(FUnitQuat4dNetSerializer).DefaultConfig)
{
}

}
