// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/RemoteObjectIdNetSerializer.h"

namespace UE::Net::Private
{

static FTestMessage& PrintRemoteObjectIdNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

class FTestRemoteObjectIdNetSerializer : public TTestNetSerializerFixture<PrintRemoteObjectIdNetSerializerConfig, FRemoteObjectId>
{
public:
	FTestRemoteObjectIdNetSerializer() : TTestNetSerializerFixture<PrintRemoteObjectIdNetSerializerConfig, FRemoteObjectId>(UE_NET_GET_SERIALIZER(FRemoteObjectIdNetSerializer)) {}

	void TestIsEqual();
	void TestValidate();
	void TestQuantize();
	void TestSerialize();

protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	void SetUpTestValues();

	typedef TTestNetSerializerFixture<PrintRemoteObjectIdNetSerializerConfig, FRemoteObjectId> Super;

	TArray<FRemoteObjectId> TestValues;

	static FRemoteObjectIdNetSerializerConfig SerializerConfig;
};

FRemoteObjectIdNetSerializerConfig FTestRemoteObjectIdNetSerializer::SerializerConfig;

UE_NET_TEST_FIXTURE(FTestRemoteObjectIdNetSerializer, HasTestValues)
{
	UE_NET_ASSERT_GT_MSG(TestValues.Num(), 0, "No test values found");
}

UE_NET_TEST_FIXTURE(FTestRemoteObjectIdNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestRemoteObjectIdNetSerializer, TestValidate)
{
	TestValidate();
}

UE_NET_TEST_FIXTURE(FTestRemoteObjectIdNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestRemoteObjectIdNetSerializer, TestSerialize)
{
	TestSerialize();
}

void FTestRemoteObjectIdNetSerializer::SetUp()
{
	Super::SetUp();

	if (TestValues.Num() == 0)
	{
		SetUpTestValues();
	}
}

void FTestRemoteObjectIdNetSerializer::TearDown()
{
	TestValues.Reset();

	Super::TearDown();
}

void FTestRemoteObjectIdNetSerializer::SetUpTestValues()
{
	TestValues.Add(FRemoteObjectId());

	TestValues.Add(FRemoteObjectId(FRemoteServerId(ERemoteServerIdConstants::FirstValid), 0));
	TestValues.Add(FRemoteObjectId(FRemoteServerId(ERemoteServerIdConstants::FirstValid), 1));
	TestValues.Add(FRemoteObjectId(FRemoteServerId(ERemoteServerIdConstants::FirstValid), MAX_REMOTE_OBJECT_SERIAL_NUMBER / 2));
	TestValues.Add(FRemoteObjectId(FRemoteServerId(ERemoteServerIdConstants::FirstValid), MAX_REMOTE_OBJECT_SERIAL_NUMBER - 1));
	TestValues.Add(FRemoteObjectId(FRemoteServerId(ERemoteServerIdConstants::FirstValid), MAX_REMOTE_OBJECT_SERIAL_NUMBER));

	TestValues.Add(FRemoteObjectId(FRemoteServerId(ERemoteServerIdConstants::Max), 0));
	TestValues.Add(FRemoteObjectId(FRemoteServerId(ERemoteServerIdConstants::Max), 1));
	TestValues.Add(FRemoteObjectId(FRemoteServerId(ERemoteServerIdConstants::Max), MAX_REMOTE_OBJECT_SERIAL_NUMBER / 2));
	TestValues.Add(FRemoteObjectId(FRemoteServerId(ERemoteServerIdConstants::Max), MAX_REMOTE_OBJECT_SERIAL_NUMBER - 1));
	TestValues.Add(FRemoteObjectId(FRemoteServerId(ERemoteServerIdConstants::Max), MAX_REMOTE_OBJECT_SERIAL_NUMBER));
}

void FTestRemoteObjectIdNetSerializer::TestIsEqual()
{
	const int32 TestValueCount = TestValues.Num();

	TArray<FRemoteObjectId> CompareValues[2];
	TArray<bool> ExpectedResults[2];

	CompareValues[0] = TestValues;
	ExpectedResults[0].Init(true, TestValueCount);

	for (int32 ValueIt = 0; ValueIt < TestValueCount; ++ValueIt)
	{
		const int32 NextValueIndex = (ValueIt + 1U) % TestValueCount;

		CompareValues[1].Add(TestValues[NextValueIndex]);
		ExpectedResults[1].Add(TestValues[ValueIt] == TestValues[NextValueIndex]);
	}

	// Do two rounds of testing per config, one where we compare each value with itself and one where we compare against another value.
	for (int32 TestRoundIt : {0, 1})
	{
		// Do both quantized and regular compares
		for (const bool bQuantizedCompare : {true, false})
		{
			const bool bSuccess = Super::TestIsEqual(TestValues.GetData(), CompareValues[TestRoundIt].GetData(), ExpectedResults[TestRoundIt].GetData(), TestValueCount, SerializerConfig, bQuantizedCompare);
			if (!bSuccess)
			{
				return;
			}
		}
	}
}

void FTestRemoteObjectIdNetSerializer::TestValidate()
{
	TArray<bool> ExpectedResults;
	ExpectedResults.Init(true, TestValues.Num());

	Super::TestValidate(TestValues.GetData(), ExpectedResults.GetData(), TestValues.Num(), SerializerConfig);
}

void FTestRemoteObjectIdNetSerializer::TestQuantize()
{
	Super::TestQuantize(TestValues.GetData(), TestValues.Num(), SerializerConfig);
}

void FTestRemoteObjectIdNetSerializer::TestSerialize()
{
	constexpr bool bQuantizedCompare = false;
	Super::TestSerialize(TestValues.GetData(), TestValues.GetData(), TestValues.Num(), SerializerConfig, bQuantizedCompare);
}

}
