// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestRemoteObjectReferenceNetSerializer.h"
#include "Iris/Core/NetObjectReference.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/ObjectNetSerializer.h"
#include "Tests/ReplicationSystem/RPC/RPCTestFixture.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "UObject/Package.h"

void UTestReplicatedObjectWithRemoteReference::GetLifetimeReplicatedProps(TArray<class FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
}

void UTestReplicatedObjectWithRemoteReference::RemoteRPCWithRemoteReferenceParam_Implementation(FRemoteObjectReference RemoteReference)
{
	RemoteRPCWithRemoteReferenceParamCallCount++;
	LastReceivedRemoteReference = RemoteReference;
}

// These tests require remote reference support to be compiled in
#if UE_WITH_REMOTE_OBJECT_HANDLE

namespace UE::Net::Private
{

UE_NET_TEST_FIXTURE(FRPCTestFixture, TestRemoteObjectReference)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn objects on server
	UTestReplicatedObjectWithRemoteReference* TestObject = Server->CreateObject<UTestReplicatedObjectWithRemoteReference>();
	UReplicatedTestObject* ReferencedObject = Server->CreateObject<UReplicatedTestObject>();

	TestObject->bIsServerObject = true;
	TestObject->ReplicationSystem = Server->GetReplicationSystem();
	Server->ReplicationSystem->SetOwningNetConnection(TestObject->NetRefHandle, Client->ConnectionIdOnServer);

	TestObject->RemoteReferenceProperty = FRemoteObjectReference(ReferencedObject);
	TestObject->RemoteRPCWithRemoteReferenceParam(FRemoteObjectReference(ReferencedObject));
	
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	UTestReplicatedObjectWithRemoteReference* ClientTestObject = Client->GetObjectAs<UTestReplicatedObjectWithRemoteReference>(TestObject->NetRefHandle);

	UE_NET_ASSERT_NE(ClientTestObject, nullptr);
	UE_NET_ASSERT_EQ(ClientTestObject->RemoteRPCWithRemoteReferenceParamCallCount, 1);

	// Verify that we received valid references
	UE_NET_ASSERT_TRUE(ClientTestObject->LastReceivedRemoteReference.GetRemoteId().IsValid());
	UE_NET_ASSERT_TRUE(ClientTestObject->RemoteReferenceProperty.GetRemoteId().IsValid());

	// Verify that the replicated references resolve to the correct object
	UReplicatedTestObject* ResolvedRPCParameter = Cast<UReplicatedTestObject>(ClientTestObject->LastReceivedRemoteReference.Resolve());
	UReplicatedTestObject* ResolvedProperty = Cast<UReplicatedTestObject>(ClientTestObject->RemoteReferenceProperty.Resolve());

	// These are expected to resolve to to server object in this test case, since both the client and server
	// exist in the same engine instance.
	UE_NET_ASSERT_EQ(ResolvedRPCParameter, ReferencedObject);
	UE_NET_ASSERT_EQ(ResolvedProperty, ReferencedObject);
}

UE_NET_TEST_FIXTURE(FRPCTestFixture, TestRemoteObjectReferencePaths)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Don't want these to trigger in this test
	FScopedRemoteDelegateOverride ScopedDelegateOverride;

	// Spawn replicated object on server to do the replication
	UTestReplicatedObjectWithRemoteReference* TestObject = Server->CreateObject<UTestReplicatedObjectWithRemoteReference>();

	// Spawn a non-replicated object with a deterministic path, it will be the one referenced by the FRemoteObjectReferece
	UTestNamedObject* ServerReferencedObject = NewObject<UTestNamedObject>(GetTransientPackage(), UTestNamedObject::StaticClass(), "TestNamedObject");

	TestObject->bIsServerObject = true;
	TestObject->ReplicationSystem = Server->GetReplicationSystem();
	Server->ReplicationSystem->SetOwningNetConnection(TestObject->NetRefHandle, Client->ConnectionIdOnServer);

	TestObject->RemoteReferenceProperty = FRemoteObjectReference(ServerReferencedObject);
	TestObject->RemoteRPCWithRemoteReferenceParam(FRemoteObjectReference(ServerReferencedObject));

	Server->NetUpdate();
	Server->SendTo(Client);
	Server->PostSendUpdate();

	// Simulate having different instances of objects on the client & server, this forces the FRemoteObjectReference
	// to look up the object by path on the client. We accomplish this by destroying, GCing, and recreating the object.
	ServerReferencedObject->MarkAsGarbage();
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	ServerReferencedObject = nullptr;

	UTestNamedObject* ClientReferencedObject = NewObject<UTestNamedObject>(GetTransientPackage(), UTestNamedObject::StaticClass(), "TestNamedObject");

	// Deliver to client now that the referenced object has been recreated
	Server->DeliverTo(Client, true);

	UTestReplicatedObjectWithRemoteReference* ClientTestObject = Client->GetObjectAs<UTestReplicatedObjectWithRemoteReference>(TestObject->NetRefHandle);

	UE_NET_ASSERT_NE(ClientTestObject, nullptr);
	UE_NET_ASSERT_EQ(ClientTestObject->RemoteRPCWithRemoteReferenceParamCallCount, 1);

	// Verify that we received a valid reference
	UE_NET_ASSERT_TRUE(ClientTestObject->LastReceivedRemoteReference.GetRemoteId().IsValid());

	// Client should be able to resolve the remote reference by path
	UTestNamedObject* ClientResolvedObjectRPC = Cast<UTestNamedObject>(ClientTestObject->LastReceivedRemoteReference.Resolve());
	UE_NET_ASSERT_EQ(ClientResolvedObjectRPC, ClientReferencedObject);

	UTestNamedObject* ClientResolvedObjectProperty = Cast<UTestNamedObject>(ClientTestObject->RemoteReferenceProperty.Resolve());
	UE_NET_ASSERT_EQ(ClientResolvedObjectProperty, ClientReferencedObject);
}

}

#endif
