// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Tests/ReplicationSystem/RPC/ReplicatedTestObjectWithRPC.h"
#include "UObject/RemoteObjectTransfer.h"
#include "TestRemoteObjectReferenceNetSerializer.generated.h"

/** Just need an empty object the test can spawn with a stable name. Can't use UObject directly since it' abstract. */
UCLASS()
class UTestNamedObject : public UObject
{
	GENERATED_BODY()
};

UCLASS()
class UTestReplicatedObjectWithRemoteReference : public UTestReplicatedObjectWithRPC
{
	GENERATED_BODY()

public:
	void GetLifetimeReplicatedProps(TArray<class FLifetimeProperty>& OutLifetimeProps) const;

	UPROPERTY(Replicated)
	FRemoteObjectReference RemoteReferenceProperty;

	UFUNCTION(Client, Reliable)
	void RemoteRPCWithRemoteReferenceParam(FRemoteObjectReference RemoteReference);

	int32 RemoteRPCWithRemoteReferenceParamCallCount = 0;

	FRemoteObjectReference LastReceivedRemoteReference;
};

namespace UE::Net::Private
{

// Overrides the StoreRemoteObjectDataDelegate and RestoreRemoteObjectDataDelegate to be no-ops for a scope
// for test cases that don't need them, avoids crashing in FPlatformFile while it's only a stub for the test program
class FScopedRemoteDelegateOverride
{
public:
	FScopedRemoteDelegateOverride()
		: OldStoreRemoteObjectDataDelegate(UE::RemoteObject::Transfer::StoreRemoteObjectDataDelegate)
		, OldRestoreRemoteObjectDataDelegate(UE::RemoteObject::Transfer::RestoreRemoteObjectDataDelegate)
	{
		UE::RemoteObject::Transfer::StoreRemoteObjectDataDelegate.BindLambda([](const UE::RemoteObject::Transfer::FMigrateSendParams&) {});
		UE::RemoteObject::Transfer::RestoreRemoteObjectDataDelegate.BindLambda([](const FUObjectMigrationContext&) {});
	}

	~FScopedRemoteDelegateOverride()
	{
		UE::RemoteObject::Transfer::StoreRemoteObjectDataDelegate = OldStoreRemoteObjectDataDelegate;
		UE::RemoteObject::Transfer::RestoreRemoteObjectDataDelegate = OldRestoreRemoteObjectDataDelegate;
	}

private:
	TDelegate<void(const UE::RemoteObject::Transfer::FMigrateSendParams&)> OldStoreRemoteObjectDataDelegate;
	TDelegate<void(const FUObjectMigrationContext&)> OldRestoreRemoteObjectDataDelegate;
};

}