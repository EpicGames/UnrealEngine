// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/RemoteServerIdNetSerializer.h"

namespace UE::Net::Private
{

static FTestMessage& PrintRemoteServerIdNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

class FTestRemoteServerIdNetSerializer : public TTestNetSerializerFixture<PrintRemoteServerIdNetSerializerConfig, FRemoteServerId>
{
public:
	FTestRemoteServerIdNetSerializer() : TTestNetSerializerFixture<PrintRemoteServerIdNetSerializerConfig, FRemoteServerId>(UE_NET_GET_SERIALIZER(FRemoteServerIdNetSerializer)) {}

	void TestIsEqual();
	void TestValidate();
	void TestQuantize();
	void TestSerialize();

protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	void SetUpTestValues();

	typedef TTestNetSerializerFixture<PrintRemoteServerIdNetSerializerConfig, FRemoteServerId> Super;

	TArray<FRemoteServerId> TestValues;

	static FRemoteServerIdNetSerializerConfig SerializerConfig;
};

FRemoteServerIdNetSerializerConfig FTestRemoteServerIdNetSerializer::SerializerConfig;

UE_NET_TEST_FIXTURE(FTestRemoteServerIdNetSerializer, HasTestValues)
{
	UE_NET_ASSERT_GT_MSG(TestValues.Num(), 0, "No test values found");
}

UE_NET_TEST_FIXTURE(FTestRemoteServerIdNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestRemoteServerIdNetSerializer, TestValidate)
{
	TestValidate();
}

UE_NET_TEST_FIXTURE(FTestRemoteServerIdNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestRemoteServerIdNetSerializer, TestSerialize)
{
	TestSerialize();
}

void FTestRemoteServerIdNetSerializer::SetUp()
{
	Super::SetUp();

	if (TestValues.Num() == 0)
	{
		SetUpTestValues();
	}
}

void FTestRemoteServerIdNetSerializer::TearDown()
{
	TestValues.Reset();

	Super::TearDown();
}

void FTestRemoteServerIdNetSerializer::SetUpTestValues()
{
	TestValues.Add(FRemoteServerId());

	TestValues.Add(FRemoteServerId(ERemoteServerIdConstants::FirstValid));
	TestValues.Add(FRemoteServerId::FromIdNumber(static_cast<uint32>(ERemoteServerIdConstants::FirstValid) + 1));
	TestValues.Add(FRemoteServerId(ERemoteServerIdConstants::Database));
	TestValues.Add(FRemoteServerId(ERemoteServerIdConstants::Asset));
	TestValues.Add(FRemoteServerId(ERemoteServerIdConstants::Local));
	TestValues.Add(FRemoteServerId::FromIdNumber(static_cast<uint32>(ERemoteServerIdConstants::Max) / 2));
	TestValues.Add(FRemoteServerId::FromIdNumber(static_cast<uint32>(ERemoteServerIdConstants::FirstReserved) - 1));
	TestValues.Add(FRemoteServerId(ERemoteServerIdConstants::FirstReserved));
}

void FTestRemoteServerIdNetSerializer::TestIsEqual()
{
	const int32 TestValueCount = TestValues.Num();

	TArray<FRemoteServerId> CompareValues[2];
	TArray<bool> ExpectedResults[2];

	CompareValues[0] = TestValues;
	ExpectedResults[0].Init(true, TestValueCount);

	for (int32 ValueIt = 0; ValueIt < TestValueCount; ++ValueIt)
	{
		const int32 NextValueIndex = (ValueIt + 1U) % TestValueCount;

		CompareValues[1].Add(TestValues[NextValueIndex]);
		ExpectedResults[1].Add(TestValues[ValueIt] == TestValues[NextValueIndex]);
	}

	// Do two rounds of testing per config, one where we compare each value with itself and one where we compare against another value.
	for (int32 TestRoundIt : {0, 1})
	{
		// Do both quantized and regular compares
		for (const bool bQuantizedCompare : {true, false})
		{
			const bool bSuccess = Super::TestIsEqual(TestValues.GetData(), CompareValues[TestRoundIt].GetData(), ExpectedResults[TestRoundIt].GetData(), TestValueCount, SerializerConfig, bQuantizedCompare);
			if (!bSuccess)
			{
				return;
			}
		}
	}
}

void FTestRemoteServerIdNetSerializer::TestValidate()
{
	TArray<bool> ExpectedResults;
	ExpectedResults.Init(true, TestValues.Num());

	Super::TestValidate(TestValues.GetData(), ExpectedResults.GetData(), TestValues.Num(), SerializerConfig);
}

void FTestRemoteServerIdNetSerializer::TestQuantize()
{
	Super::TestQuantize(TestValues.GetData(), TestValues.Num(), SerializerConfig);
}

void FTestRemoteServerIdNetSerializer::TestSerialize()
{
	constexpr bool bQuantizedCompare = false;
	Super::TestSerialize(TestValues.GetData(), TestValues.GetData(), TestValues.Num(), SerializerConfig, bQuantizedCompare);
}

}
