// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "RepMovementNetSerializer.h"
#include "Engine/ReplicatedState.h"

UE::Net::FTestMessage& operator<<(UE::Net::FTestMessage& Message, const FRepMovement& Value)
{
	return Message << 
		"LinearVelocity: " << Value.LinearVelocity << '\n' <<
		"AngularVelocity: " << Value.AngularVelocity << '\n' <<
		"Location: " << Value.Location << '\n' <<
		"Rotation: " << Value.Rotation.ToString() << '\n' <<
		"Acceleration: " << Value.Acceleration << '\n' <<
		"bSimulatedPhysicSleep: " << Value.bSimulatedPhysicSleep << '\n' <<
		"bRepPhysics: " << Value.bRepPhysics << '\n' <<
		"bRepAcceleration: " << Value.bRepAcceleration << '\n' <<
		"ServerFrame: " << Value.ServerFrame << '\n' <<
		"ServerPhysicsHandle: " << Value.ServerPhysicsHandle << '\n' <<
		"LocationQuantizationLevel: " << (uint32)Value.LocationQuantizationLevel << '\n' <<
		"VelocityQuantizationLevel: " << (uint32)Value.VelocityQuantizationLevel << '\n' <<
		"RotationQuantizationLevel: " << (uint32)Value.RotationQuantizationLevel;
}

namespace UE::Net::Private
{

static FTestMessage& PrintRepMovementNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

class FTestRepMovementNetSerializer : public TTestNetSerializerFixture<PrintRepMovementNetSerializerConfig, FRepMovement>
{
	typedef TTestNetSerializerFixture<PrintRepMovementNetSerializerConfig, FRepMovement> Super;

public:
	FTestRepMovementNetSerializer() : Super(UE_NET_GET_SERIALIZER(FRepMovementNetSerializer)) {}

	void TestIsEqual();
	void TestSerialize();
	void TestSerializeDelta();

protected:
	virtual void SetUp() override;

	inline static bool bIsClassInitialized = false;
	inline static TArray<FRepMovement> Values;

private:
	void OneTimeClassInitialization();
};

UE_NET_TEST_FIXTURE(FTestRepMovementNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestRepMovementNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestRepMovementNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

void FTestRepMovementNetSerializer::SetUp()
{
	Super::SetUp();
	OneTimeClassInitialization();
}

void FTestRepMovementNetSerializer::TestIsEqual()
{
	constexpr int32 TestRoundCount = 2;
	TArray<FRepMovement> CompareValues[TestRoundCount];
	TArray<bool> ExpectedResults[TestRoundCount];

	const int32 ValueCount = Values.Num();
	for (int32 TestRoundIt = 0, TestRoundEndIt = TestRoundCount; TestRoundIt != TestRoundEndIt; ++TestRoundIt)
	{
		CompareValues[TestRoundIt].Reserve(ValueCount);
		ExpectedResults[TestRoundIt].Reserve(ValueCount);

		const int32 SourceValueItOffset = TestRoundIt;
		const bool ExpectedResult = (TestRoundIt == 0U);
		for (int32 ValueIt = 0, ValueEndIt = ValueCount; ValueIt < ValueEndIt; ++ValueIt)
		{
			CompareValues[TestRoundIt].Push(Values[(ValueIt + SourceValueItOffset) % ValueEndIt]);
			ExpectedResults[TestRoundIt].Push(ExpectedResult);
		}
	}

	const FRepMovementNetSerializerConfig Config;
	for (SIZE_T TestRoundIt = 0, TestRoundEndIt = TestRoundCount; TestRoundIt != TestRoundEndIt; ++TestRoundIt)
	{
		constexpr bool bQuantizedCompare = true;
		bool bSuccess = Super::TestIsEqual(Values.GetData(), CompareValues[TestRoundIt].GetData(), ExpectedResults[TestRoundIt].GetData(), CompareValues[TestRoundIt].Num(), Config, bQuantizedCompare);
		if (!bSuccess)
		{
			return;
		}
	}
}

void FTestRepMovementNetSerializer::TestSerialize()
{
	const FRepMovementNetSerializerConfig Config;
	constexpr bool bQuantizedCompare = true;
	Super::TestSerialize(Values.GetData(), Values.GetData(), Values.Num(), Config, bQuantizedCompare);
}

void FTestRepMovementNetSerializer::TestSerializeDelta()
{
	const FRepMovementNetSerializerConfig Config;
	Super::TestSerializeDelta(Values.GetData(), Values.Num(), Config);
}

void FTestRepMovementNetSerializer::OneTimeClassInitialization()
{
	if (bIsClassInitialized)
	{
		return;
	}

	// Default value
	{
		Values.Emplace();
	}

	// Values with location, velocities, rotation, ServerFrame and physics handle
	{
		FRepMovement& Value = Values.Emplace_GetRef();
		Value.LinearVelocity = FVector(10.01, -20.23, 40.05);
		Value.AngularVelocity = FVector(1000.23, -2000.05, 4000.01);
		Value.Location = FVector(100000.05, -200000.01, 400000.23);
		Value.Rotation = FRotator(213.16, 23.75, 109.12);

		FRepMovement ValueWithServerFrame = Value;
		ValueWithServerFrame.ServerFrame = 1908246313;
		Values.Add(ValueWithServerFrame);

		FRepMovement ValueWithPhysicsHandle = Value;
		ValueWithPhysicsHandle.ServerPhysicsHandle = 4711;
		Values.Add(ValueWithPhysicsHandle);
	}

	// Values with location, velocities, rotation, bRepPhysics and bSimulatedPhysicSleep
	{
		FRepMovement& Value = Values.Emplace_GetRef();
		Value.LinearVelocity = FVector(10.01, 40.05, -20.23);
		Value.AngularVelocity = FVector(-2000.05, 1000.23, 4000.01);
		Value.Location = FVector(400000.23, 100000.05, -200000.01);
		Value.Rotation = FRotator(213.16, 23.75, 109.12);
		Value.bRepPhysics = true;

		FRepMovement ValueWithSimulatedPhysicSleep = Value;
		ValueWithSimulatedPhysicSleep.bSimulatedPhysicSleep = true;
		Values.Add(ValueWithSimulatedPhysicSleep);
	}

	// Values with location, velocities, rotation, acceleration, ServerFrame, and physics handle
	{
		FRepMovement& Value = Values.Emplace_GetRef();
		Value.LinearVelocity = FVector(11.33, -50.23, 41.05);
		Value.AngularVelocity = FVector(599.23, -2001.05, 4010.01);
		Value.Location = FVector(100342.05, -190200.01, 298000.43);
		Value.Rotation = FRotator(99.16, 10.75, 0.01);
		Value.bRepAcceleration = true;
		Value.Acceleration = FVector(1501.11, 1.33, 10.11);

		FRepMovement ValueWithServerFrame = Value;
		ValueWithServerFrame.ServerFrame = 1908246315;
		Values.Add(ValueWithServerFrame);

		FRepMovement ValueWithPhysicsHandle = Value;
		ValueWithPhysicsHandle.ServerPhysicsHandle = 4718;
		Values.Add(ValueWithPhysicsHandle);
	}


	bIsClassInitialized = true;
}

}
