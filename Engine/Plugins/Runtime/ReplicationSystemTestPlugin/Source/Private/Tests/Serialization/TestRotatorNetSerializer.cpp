// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/RotatorNetSerializers.h"
#include <limits>

namespace UE::Net
{

FTestMessage& operator<<(FTestMessage& Message, const FRotator3f& Rotator)
{
	return Message << Rotator.ToString();
}

FTestMessage& operator<<(FTestMessage& Message, const FRotator3d& Rotator)
{
	return Message << Rotator.ToString();
}

}

namespace UE::Net::Private
{

static FTestMessage& PrintRotatorNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

template<typename RotatorType>
class FTestRotatorNetSerializerBase : public TTestNetSerializerFixture<PrintRotatorNetSerializerConfig, RotatorType>
{
private:
	typedef TTestNetSerializerFixture<PrintRotatorNetSerializerConfig, RotatorType> Super;

public:
	FTestRotatorNetSerializerBase(const FNetSerializer& Serializer) : Super(Serializer) {}

	void TestSerialize();
	void TestSerializeDelta();
	void TestIsEqual();
	void TestValidate();

protected:
	using ScalarType = decltype(RotatorType::Pitch);

	struct FFloatTriplet
	{
		ScalarType X;
		ScalarType Y;
		ScalarType Z;
	};

	static const FFloatTriplet BadValues[];
	static const SIZE_T BadValueCount;
	static const RotatorType Values[];
	static const SIZE_T ValueCount;
	float ValueTolerance = 0.0f;
	const FNetSerializerConfig* NetSerializerConfig = nullptr;
};

// In case there's any sort of sanity checking in the FRotator constructors this will avoid them.
template<typename RotatorType>
const typename FTestRotatorNetSerializerBase<RotatorType>::FFloatTriplet FTestRotatorNetSerializerBase<RotatorType>::BadValues[] =
{
	{-INFINITY, NAN, -NAN},
	{-INFINITY, INFINITY, -INFINITY},
	{NAN, NAN, NAN},
	{-NAN, -NAN, -NAN},
};

template<typename RotatorType>
const SIZE_T FTestRotatorNetSerializerBase<RotatorType>::BadValueCount = sizeof(BadValues)/sizeof(BadValues[0]);

template<typename RotatorType>
const RotatorType FTestRotatorNetSerializerBase<RotatorType>::Values[] =
{
	RotatorType(0.0f),
	RotatorType(1.0f, 0.0f, 0.0f),
	RotatorType(0.0f, 1.0f, 0.0f),
	RotatorType(0.0f, 0.0f, 1.0f),
	RotatorType(-0.0f), // 0.0f == -0.0f so we expect this to be handled gracefully
	RotatorType(359.999f, 129.130f, 45.0f),
	RotatorType(1.26698556E-23f),
	RotatorType(47.11f),
};
template<typename RotatorType>
const SIZE_T FTestRotatorNetSerializerBase<RotatorType>::ValueCount = sizeof(Values)/sizeof(Values[0]);

class FTestRotatorAsShortNetSerializer : public FTestRotatorNetSerializerBase<FRotator>
{
private:
	typedef FTestRotatorNetSerializerBase<FRotator> Super;

public:
	FTestRotatorAsShortNetSerializer()
	: Super(UE_NET_GET_SERIALIZER(FRotatorAsShortNetSerializer))
	{
		ValueTolerance = 1.5f*(360.0f/65536.0f);
		NetSerializerConfig = UE_NET_GET_SERIALIZER_DEFAULT_CONFIG(FRotatorAsShortNetSerializer);
	}
};

class FTestRotatorAsByteNetSerializer : public FTestRotatorNetSerializerBase<FRotator>
{
private:
	typedef FTestRotatorNetSerializerBase<FRotator> Super;

public:
	FTestRotatorAsByteNetSerializer()
	: Super(UE_NET_GET_SERIALIZER(FRotatorAsByteNetSerializer))
	{
		ValueTolerance = 1.5f*(360.0f/256.0f);
		NetSerializerConfig = UE_NET_GET_SERIALIZER_DEFAULT_CONFIG(FRotatorAsByteNetSerializer);
	}
};

class FTestRotator3fNetSerializer : public FTestRotatorNetSerializerBase<FRotator3f>
{
private:
	typedef FTestRotatorNetSerializerBase<FRotator3f> Super;

public:
	FTestRotator3fNetSerializer()
	: Super(UE_NET_GET_SERIALIZER(FRotator3fNetSerializer))
	{
		ValueTolerance = 1.5f*(360.0f/65536.0f);
		NetSerializerConfig = UE_NET_GET_SERIALIZER_DEFAULT_CONFIG(FRotator3fNetSerializer);
	}
};

class FTestRotator3dNetSerializer : public FTestRotatorNetSerializerBase<FRotator3d>
{
private:
	typedef FTestRotatorNetSerializerBase<FRotator3d> Super;

public:
	FTestRotator3dNetSerializer()
	: Super(UE_NET_GET_SERIALIZER(FRotator3dNetSerializer))
	{
		ValueTolerance = 1.5f*(360.0f/65536.0f);
		NetSerializerConfig = UE_NET_GET_SERIALIZER_DEFAULT_CONFIG(FRotator3dNetSerializer);
	}
};

// FRotatorAsShortNetSerializer tests
UE_NET_TEST_FIXTURE(FTestRotatorAsShortNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestRotatorAsShortNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestRotatorAsShortNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestRotatorAsShortNetSerializer, TestValidate)
{
	TestValidate();
}

// FRotatorAsByteNetSerializer tests
UE_NET_TEST_FIXTURE(FTestRotatorAsByteNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestRotatorAsByteNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestRotatorAsByteNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestRotatorAsByteNetSerializer, TestValidate)
{
	TestValidate();
}

// FRotator3fNetSerializer tests
UE_NET_TEST_FIXTURE(FTestRotator3fNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestRotator3fNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestRotator3fNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestRotator3fNetSerializer, TestValidate)
{
	TestValidate();
}

// FRotator3dNetSerializer tests
UE_NET_TEST_FIXTURE(FTestRotator3dNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestRotator3dNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestRotator3dNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestRotator3dNetSerializer, TestValidate)
{
	TestValidate();
}

template<typename RotatorType>
void FTestRotatorNetSerializerBase<RotatorType>::TestIsEqual()
{
	RotatorType CompareValues[2][sizeof(Values)/sizeof(Values[0])];
	bool ExpectedResults[2][sizeof(Values)/sizeof(Values[0])];

	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		// We need to memcopy the data to avoid any validation/fixing in the constructor
		FPlatformMemory::Memcpy(&CompareValues[0][ValueIt], &Values[ValueIt], sizeof(RotatorType));
		ExpectedResults[0][ValueIt] = true;
	}
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		FPlatformMemory::Memcpy(&CompareValues[1][ValueIt], &Values[(ValueIt + 1) % ValueCount], sizeof(RotatorType));
		ExpectedResults[1][ValueIt] = false;
	}

	for (SIZE_T TestRoundIt = 0, TestRoundEndIt = 2; TestRoundIt != TestRoundEndIt; ++TestRoundIt)
	{
		constexpr bool bQuantizedCompare = true;
		bool bSuccess = Super::TestIsEqual(Values, CompareValues[TestRoundIt], ExpectedResults[TestRoundIt], ValueCount, *NetSerializerConfig, bQuantizedCompare);
		if (!bSuccess)
		{
			return;
		}
	}
}

template<typename RotatorType>
void FTestRotatorNetSerializerBase<RotatorType>::TestSerialize()
{
	constexpr bool bQuantizedCompare = true;
	Super::TestSerialize(Values, Values, ValueCount, *NetSerializerConfig, bQuantizedCompare);
}

template<typename RotatorType>
void FTestRotatorNetSerializerBase<RotatorType>::TestSerializeDelta()
{
	Super::TestSerializeDelta(Values, ValueCount, *NetSerializerConfig);
}

template<typename RotatorType>
void FTestRotatorNetSerializerBase<RotatorType>::TestValidate()
{
	// Test bad values
	{
		static_assert(sizeof(BadValues[0]) == sizeof(RotatorType), "BadValues and RotatorType type mismatch.");
		bool ExpectedResults[BadValueCount] = {};
		const bool bSuccess = Super::TestValidate(reinterpret_cast<const RotatorType*>(BadValues), ExpectedResults, BadValueCount, *NetSerializerConfig);
		if (!bSuccess)
		{
			return;
		}
	}

	// Test conforming values
	{
		bool ExpectedResults[ValueCount];
		for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
		{
			ExpectedResults[ValueIt] = true;
		}

		const bool bSuccess = Super::TestValidate(Values, ExpectedResults, ValueCount, *NetSerializerConfig);
		if (!bSuccess)
		{
			return;
		}
	}
}

}
