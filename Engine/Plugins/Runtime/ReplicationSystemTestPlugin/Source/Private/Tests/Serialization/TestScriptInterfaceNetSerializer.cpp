// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Serialization/ScriptInterfaceTestTypes.h"
#include "Tests/Serialization/TestNetSerializerFixture.h"
#include "Tests/ReplicationSystem/ReplicationSystemServerClientTestFixture.h"
#include "Iris/Serialization/ObjectNetSerializer.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"

namespace UE::Net::Private
{

static FTestMessage& PrintScriptInterfaceNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	const FScriptInterfaceNetSerializerConfig& Config = static_cast<const FScriptInterfaceNetSerializerConfig&>(InConfig);
	return Message << TEXT("Interface: ") << Config.PropertyClass->GetName();
}

class FTestScriptInterfaceNetSerializer : public FReplicationSystemServerClientTestFixture
{
public:
};

UE_NET_TEST_FIXTURE(FTestScriptInterfaceNetSerializer, TestNoScriptInterface)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestObjectReferencingScriptInterface* Object = Server->CreateObject<UTestObjectReferencingScriptInterface>();

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify object was created
	UTestObjectReferencingScriptInterface* ClientObject = Client->GetObjectAs<UTestObjectReferencingScriptInterface>(Object->NetRefHandle);	
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Verify the interface objects are identical, i.e. not set.
	UE_NET_ASSERT_EQ(Object->InterfaceObject.GetObject(), ClientObject->InterfaceObject.GetObject());
	UE_NET_ASSERT_EQ(Object->InterfaceObject.GetInterface(), ClientObject->InterfaceObject.GetInterface());
}

UE_NET_TEST_FIXTURE(FTestScriptInterfaceNetSerializer, TestInterfaceToReplicatedObject)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestObjectReferencingScriptInterface* Object = Server->CreateObject<UTestObjectReferencingScriptInterface>();
	// Spawn interface object on server
	UTestScriptInterfaceReplicatedObject* InterfaceObject = Server->CreateObject<UTestScriptInterfaceReplicatedObject>();

	// Set the interface
	Object->InterfaceObject = InterfaceObject;

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify object was created
	UTestObjectReferencingScriptInterface* ClientObject = Client->GetObjectAs<UTestObjectReferencingScriptInterface>(Object->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Verify interface object was created
	UTestScriptInterfaceReplicatedObject* ClientInterfaceObject = Client->GetObjectAs<UTestScriptInterfaceReplicatedObject>(InterfaceObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientInterfaceObject, nullptr);

	// Verify the interface object is set
	const TScriptInterface<IIrisTestInterface> ComparisonInterface(ClientInterfaceObject);
	UE_NET_ASSERT_EQ(ClientObject->InterfaceObject.GetObject(), ComparisonInterface.GetObject());
	UE_NET_ASSERT_EQ(ClientObject->InterfaceObject.GetInterface(), ComparisonInterface.GetInterface());
}

UE_NET_TEST_FIXTURE(FTestScriptInterfaceNetSerializer, TestInterfaceToNamedObject)
{
	// Add a client
	FReplicationSystemTestClient* Client = CreateClient();

	// Spawn object on server
	UTestObjectReferencingScriptInterface* Object = Server->CreateObject<UTestObjectReferencingScriptInterface>();
	// Spawn object with subobject implementing interface
	UTestScriptInterfaceReplicatedObjectWithDefaultSubobject* ObjectWithInterfaceSubObject = Server->CreateObject<UTestScriptInterfaceReplicatedObjectWithDefaultSubobject>();

	// Set the interface
	Object->InterfaceObject = ObjectWithInterfaceSubObject->DefaultSubobjectWithInterface.Get();

	// Send and deliver packet
	Server->NetUpdate();
	Server->SendAndDeliverTo(Client, true);
	Server->PostSendUpdate();

	// Verify object was created
	UTestObjectReferencingScriptInterface* ClientObject = Client->GetObjectAs<UTestObjectReferencingScriptInterface>(Object->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObject, nullptr);

	// Verify the object with the interface subobject was created
	UTestScriptInterfaceReplicatedObjectWithDefaultSubobject* ClientObjectWithInterfaceSubObject = Client->GetObjectAs<UTestScriptInterfaceReplicatedObjectWithDefaultSubobject>(ObjectWithInterfaceSubObject->NetRefHandle);
	UE_NET_ASSERT_NE(ClientObjectWithInterfaceSubObject, nullptr);

	// Verify the interface object is set
	const TScriptInterface<IIrisTestInterface> ComparisonInterface(ClientObjectWithInterfaceSubObject->DefaultSubobjectWithInterface.Get());
	UE_NET_ASSERT_EQ(ClientObject->InterfaceObject.GetObject(), ComparisonInterface.GetObject());
	UE_NET_ASSERT_EQ(ClientObject->InterfaceObject.GetInterface(), ComparisonInterface.GetInterface());
}

}
