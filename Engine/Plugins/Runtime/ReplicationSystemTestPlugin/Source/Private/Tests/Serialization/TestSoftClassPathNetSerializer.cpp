// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/SoftObjectNetSerializers.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "UObject/SoftObjectPath.h"

namespace UE::Net
{

FTestMessage& operator<<(FTestMessage& Message, const FSoftClassPath& SoftClassPath)
{
	return Message << SoftClassPath.ToString();
}

}

namespace UE::Net::Private
{

static FTestMessage& PrintSoftClassPathNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

class FTestSoftClassPathNetSerializer : public TTestNetSerializerFixture<PrintSoftClassPathNetSerializerConfig, FSoftClassPath>
{
public:
	FTestSoftClassPathNetSerializer() : Super(UE_NET_GET_SERIALIZER(FSoftClassPathNetSerializer)) {}

	void TestIsEqual();
	void TestValidate();
	void TestQuantize();
	void TestSerialize();
	void TestCloneDynamicState();

protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	void SetUpTestValues();

	typedef TTestNetSerializerFixture<PrintSoftClassPathNetSerializerConfig, FSoftClassPath> Super;

	TArray<FSoftClassPath> TestValues;

	// Serializer
	static FSoftClassPathNetSerializerConfig SerializerConfig;
};

FSoftClassPathNetSerializerConfig FTestSoftClassPathNetSerializer::SerializerConfig;

UE_NET_TEST_FIXTURE(FTestSoftClassPathNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestSoftClassPathNetSerializer, TestValidate)
{
	TestValidate();
}

UE_NET_TEST_FIXTURE(FTestSoftClassPathNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestSoftClassPathNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestSoftClassPathNetSerializer, TestCloneDynamicState)
{
	TestCloneDynamicState();
}

void FTestSoftClassPathNetSerializer::SetUp()
{
	Super::SetUp();

	if (TestValues.Num() == 0)
	{
		SetUpTestValues();
	}
}

void FTestSoftClassPathNetSerializer::TearDown()
{
	TestValues.Reset();

	Super::TearDown();
}

void FTestSoftClassPathNetSerializer::SetUpTestValues()
{
	TestValues.Add(FSoftClassPath());
	TestValues.Add(FSoftClassPath(TEXT("/Script/NonExistingPlugin.NonExistingClass")));
	TestValues.Add(FSoftClassPath(TEXT("/Script/ReplicationSystemTestPlugin.MockDataStream")));
	check(TestValues[TestValues.Num() - 1].ResolveClass() != nullptr);
}

void FTestSoftClassPathNetSerializer::TestValidate()
{
	{
		TArray<bool> ExpectedResults;
		ExpectedResults.Init(true, TestValues.Num());

		const bool bSuccess = Super::TestValidate(TestValues.GetData(), ExpectedResults.GetData(), TestValues.Num(), SerializerConfig);
		if (!bSuccess)
		{
			return;
		}
	}
}

void FTestSoftClassPathNetSerializer::TestIsEqual()
{
	const int32 TestValueCount = TestValues.Num();

	TArray<FSoftClassPath> CompareValues[2];
	TArray<bool> ExpectedResults[2];

	CompareValues[0] = TestValues;
	ExpectedResults[0].Init(true, TestValueCount);

	CompareValues[1].Reserve(TestValueCount);
	ExpectedResults[1].Reserve(TestValueCount);
	for (const FSoftClassPath& Value : TestValues)
	{
		const int32 ValueIndex = static_cast<int32>(&Value - TestValues.GetData());
		const int32 NextValueIndex = (ValueIndex + 1U) % TestValueCount;
		CompareValues[1].Add(TestValues[NextValueIndex]);
		ExpectedResults[1].Add(TestValues[ValueIndex] == TestValues[NextValueIndex]);
	}

	// Do two rounds of testing per config, one where we compare each value with itself and one where we compare against another value.
	for (SIZE_T TestRoundIt : {0, 1})
	{
		// Do both quantized and regular compares
		for (const bool bQuantizedCompare : {true, false})
		{
			const bool bSuccess = Super::TestIsEqual(TestValues.GetData(), CompareValues[TestRoundIt].GetData(), ExpectedResults[TestRoundIt].GetData(), TestValueCount, SerializerConfig, bQuantizedCompare);
			if (!bSuccess)
			{
				return;
			}
		}
	}
}

void FTestSoftClassPathNetSerializer::TestQuantize()
{
	const bool bSuccess = Super::TestQuantize(TestValues.GetData(), TestValues.Num(), SerializerConfig);
	if (!bSuccess)
	{
		return;
	}
}

void FTestSoftClassPathNetSerializer::TestSerialize()
{
	constexpr bool bQuantizedCompare = false;
	const bool bSuccess = Super::TestSerialize(TestValues.GetData(), TestValues.GetData(), TestValues.Num(), SerializerConfig, bQuantizedCompare);
	if (!bSuccess)
	{
		return;
	}
}

void FTestSoftClassPathNetSerializer::TestCloneDynamicState()
{
	const bool bSuccess = Super::TestCloneDynamicState(TestValues.GetData(), TestValues.Num(), SerializerConfig);
	if (!bSuccess)
	{
		return;
	}
}

}
