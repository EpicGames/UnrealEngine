// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/SoftObjectNetSerializers.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/StrongObjectPtr.h"

namespace UE::Net
{

FTestMessage& operator<<(FTestMessage& Message, const FSoftObjectPtr& SoftObjectPtr)
{
	return Message << SoftObjectPtr.ToString();
}

}

namespace UE::Net::Private
{

static FTestMessage& PrintSoftObjectNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

class FTestSoftObjectNetSerializer : public TTestNetSerializerFixture<PrintSoftObjectNetSerializerConfig, FSoftObjectPtr>
{
public:
	FTestSoftObjectNetSerializer() : Super(UE_NET_GET_SERIALIZER(FSoftObjectNetSerializer)) {}

	void TestIsEqual();
	void TestValidate();
	void TestQuantize();
	void TestSerialize();
	void TestCloneDynamicState();

protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	void SetUpTestValues();

	typedef TTestNetSerializerFixture<PrintSoftObjectNetSerializerConfig, FSoftObjectPtr> Super;

	TArray<FSoftObjectPtr> TestValues;
	TArray<TStrongObjectPtr<UObject>> CreatedObjects;

	// Serializer
	static FSoftObjectNetSerializerConfig SerializerConfig;
};

FSoftObjectNetSerializerConfig FTestSoftObjectNetSerializer::SerializerConfig;

UE_NET_TEST_FIXTURE(FTestSoftObjectNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestSoftObjectNetSerializer, TestValidate)
{
	TestValidate();
}

UE_NET_TEST_FIXTURE(FTestSoftObjectNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestSoftObjectNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestSoftObjectNetSerializer, TestCloneDynamicState)
{
	TestCloneDynamicState();
}

void FTestSoftObjectNetSerializer::SetUp()
{
	Super::SetUp();

	if (TestValues.Num() == 0)
	{
		SetUpTestValues();
	}
}

void FTestSoftObjectNetSerializer::TearDown()
{
	CreatedObjects.Reset();
	TestValues.Reset();

	Super::TearDown();
}

void FTestSoftObjectNetSerializer::SetUpTestValues()
{
	TestValues.Add(FSoftObjectPtr());
	TestValues.Add(FSoftObjectPtr(FSoftObjectPath(TEXT("/Script/NonExistingPlugin.NonExistingClass:NonExistingInstance"))));

	// Non-stably named objects are tested in TestObjectNetSerializer.cpp.
}

void FTestSoftObjectNetSerializer::TestValidate()
{
	{
		TArray<bool> ExpectedResults;
		ExpectedResults.Init(true, TestValues.Num());

		const bool bSuccess = Super::TestValidate(TestValues.GetData(), ExpectedResults.GetData(), TestValues.Num(), SerializerConfig);
		if (!bSuccess)
		{
			return;
		}
	}
}

void FTestSoftObjectNetSerializer::TestIsEqual()
{
	const int32 TestValueCount = TestValues.Num();

	TArray<FSoftObjectPtr> CompareValues[2];
	TArray<bool> ExpectedResults[2];

	CompareValues[0] = TestValues;
	ExpectedResults[0].Init(true, TestValueCount);

	CompareValues[1].Reserve(TestValueCount);
	ExpectedResults[1].Reserve(TestValueCount);
	for (const FSoftObjectPtr& Value : TestValues)
	{
		const int32 ValueIndex = static_cast<int32>(&Value - TestValues.GetData());
		const int32 NextValueIndex = (ValueIndex + 1U) % TestValueCount;
		CompareValues[1].Add(TestValues[NextValueIndex]);
		ExpectedResults[1].Add(TestValues[ValueIndex] == TestValues[NextValueIndex]);
	}

	// Do two rounds of testing per config, one where we compare each value with itself and one where we compare against another value.
	for (SIZE_T TestRoundIt : {0, 1})
	{
		// Do both quantized and regular compares
		for (const bool bQuantizedCompare : {true, false})
		{
			const bool bSuccess = Super::TestIsEqual(TestValues.GetData(), CompareValues[TestRoundIt].GetData(), ExpectedResults[TestRoundIt].GetData(), TestValueCount, SerializerConfig, bQuantizedCompare);
			if (!bSuccess)
			{
				return;
			}
		}
	}
}

void FTestSoftObjectNetSerializer::TestQuantize()
{
	const bool bSuccess = Super::TestQuantize(TestValues.GetData(), TestValues.Num(), SerializerConfig);
	if (!bSuccess)
	{
		return;
	}
}

void FTestSoftObjectNetSerializer::TestSerialize()
{
	constexpr bool bQuantizedCompare = false;
	const bool bSuccess = Super::TestSerialize(TestValues.GetData(), TestValues.GetData(), TestValues.Num(), SerializerConfig, bQuantizedCompare);
	if (!bSuccess)
	{
		return;
	}
}

void FTestSoftObjectNetSerializer::TestCloneDynamicState()
{
	const bool bSuccess = Super::TestCloneDynamicState(TestValues.GetData(), TestValues.Num(), SerializerConfig);
	if (!bSuccess)
	{
		return;
	}
}

}
