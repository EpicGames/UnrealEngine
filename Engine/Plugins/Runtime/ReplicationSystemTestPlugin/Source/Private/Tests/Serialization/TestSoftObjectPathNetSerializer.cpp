// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/SoftObjectNetSerializers.h"
#include "Tests/ReplicationSystem/ReplicatedTestObject.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/StrongObjectPtr.h"

namespace UE::Net
{

FTestMessage& operator<<(FTestMessage& Message, const FSoftObjectPath& SoftObjectPath)
{
	return Message << SoftObjectPath.ToString();
}

}

namespace UE::Net::Private
{

static FTestMessage& PrintSoftObjectPathNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

class FTestSoftObjectPathNetSerializer : public TTestNetSerializerFixture<PrintSoftObjectPathNetSerializerConfig, FSoftObjectPath>
{
public:
	FTestSoftObjectPathNetSerializer() : Super(UE_NET_GET_SERIALIZER(FSoftObjectPathNetSerializer)) {}

	void TestIsEqual();
	void TestValidate();
	void TestQuantize();
	void TestSerialize();
	void TestCloneDynamicState();

protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	void SetUpTestValues();

	typedef TTestNetSerializerFixture<PrintSoftObjectPathNetSerializerConfig, FSoftObjectPath> Super;

	TArray<FSoftObjectPath> TestValues;
	TArray<TStrongObjectPtr<UObject>> CreatedObjects;

	// Serializer
	static FSoftObjectPathNetSerializerConfig SerializerConfig;
};

FSoftObjectPathNetSerializerConfig FTestSoftObjectPathNetSerializer::SerializerConfig;

UE_NET_TEST_FIXTURE(FTestSoftObjectPathNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestSoftObjectPathNetSerializer, TestValidate)
{
	TestValidate();
}

UE_NET_TEST_FIXTURE(FTestSoftObjectPathNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestSoftObjectPathNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestSoftObjectPathNetSerializer, TestCloneDynamicState)
{
	TestCloneDynamicState();
}

void FTestSoftObjectPathNetSerializer::SetUp()
{
	Super::SetUp();

	if (TestValues.Num() == 0)
	{
		SetUpTestValues();
	}
}

void FTestSoftObjectPathNetSerializer::TearDown()
{
	CreatedObjects.Reset();
	TestValues.Reset();

	Super::TearDown();
}

void FTestSoftObjectPathNetSerializer::SetUpTestValues()
{
	CreatedObjects.Add(TStrongObjectPtr<UObject>(NewObject<UTestReplicatedIrisObject>()));
	CreatedObjects.Add(TStrongObjectPtr<UObject>(NewObject<UTestReplicatedIrisObject>()));

	TestValues.Add(FSoftObjectPath());
	TestValues.Add(FSoftObjectPath(TEXT("/Script/NonExistingPlugin.NonExistingClass:NonExistingInstance")));

	for (const TStrongObjectPtr<UObject>& Object : CreatedObjects)
	{
		TestValues.Add(FSoftObjectPath(Object.Get()));
		check(TestValues[TestValues.Num() - 1].ResolveObject() != nullptr);
	}
}

void FTestSoftObjectPathNetSerializer::TestValidate()
{
	{
		TArray<bool> ExpectedResults;
		ExpectedResults.Init(true, TestValues.Num());

		const bool bSuccess = Super::TestValidate(TestValues.GetData(), ExpectedResults.GetData(), TestValues.Num(), SerializerConfig);
		if (!bSuccess)
		{
			return;
		}
	}
}

void FTestSoftObjectPathNetSerializer::TestIsEqual()
{
	const int32 TestValueCount = TestValues.Num();

	TArray<FSoftObjectPath> CompareValues[2];
	TArray<bool> ExpectedResults[2];

	CompareValues[0] = TestValues;
	ExpectedResults[0].Init(true, TestValueCount);

	CompareValues[1].Reserve(TestValueCount);
	ExpectedResults[1].Reserve(TestValueCount);
	for (const FSoftObjectPath& Value : TestValues)
	{
		const int32 ValueIndex = static_cast<int32>(&Value - TestValues.GetData());
		const int32 NextValueIndex = (ValueIndex + 1U) % TestValueCount;
		CompareValues[1].Add(TestValues[NextValueIndex]);
		ExpectedResults[1].Add(TestValues[ValueIndex] == TestValues[NextValueIndex]);
	}

	// Do two rounds of testing per config, one where we compare each value with itself and one where we compare against another value.
	for (SIZE_T TestRoundIt : {0, 1})
	{
		// Do both quantized and regular compares
		for (const bool bQuantizedCompare : {true, false})
		{
			const bool bSuccess = Super::TestIsEqual(TestValues.GetData(), CompareValues[TestRoundIt].GetData(), ExpectedResults[TestRoundIt].GetData(), TestValueCount, SerializerConfig, bQuantizedCompare);
			if (!bSuccess)
			{
				return;
			}
		}
	}
}

void FTestSoftObjectPathNetSerializer::TestQuantize()
{
	const bool bSuccess = Super::TestQuantize(TestValues.GetData(), TestValues.Num(), SerializerConfig);
	if (!bSuccess)
	{
		return;
	}
}

void FTestSoftObjectPathNetSerializer::TestSerialize()
{
	constexpr bool bQuantizedCompare = false;
	const bool bSuccess = Super::TestSerialize(TestValues.GetData(), TestValues.GetData(), TestValues.Num(), SerializerConfig, bQuantizedCompare);
	if (!bSuccess)
	{
		return;
	}
}

void FTestSoftObjectPathNetSerializer::TestCloneDynamicState()
{
	const bool bSuccess = Super::TestCloneDynamicState(TestValues.GetData(), TestValues.Num(), SerializerConfig);
	if (!bSuccess)
	{
		return;
	}
}

}
