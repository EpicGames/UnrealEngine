// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/StringNetSerializers.h"
#include "Containers/StringConv.h"

namespace UE::Net::Private
{

static FTestMessage& PrintStringNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

class FTestStringNetSerializer : public TTestNetSerializerFixture<PrintStringNetSerializerConfig, FString>
{
public:
	FTestStringNetSerializer() : Super(UE_NET_GET_SERIALIZER(FStringNetSerializer)) {}

	void TestValidate();
	void TestQuantize();
	void TestIsEqual();
	void TestSerialize();
	void TestCloneDynamicState();

	void TestIllFormedStringIsEqual();
	void TestIllFormedStringSerialize();

protected:
	typedef TTestNetSerializerFixture<PrintStringNetSerializerConfig, FString> Super;

	virtual void SetUp() override;

	// Set of valid strings
	static const char* UTF8Strings[];
	static const SIZE_T UTF8StringCount;
	// Valid and invalid strings. These are the ones to be tested.
	static TArray<FString> WellFormedTestStrings;
	static TArray<FString> IllFormedTestStrings;

	// Serializer
	static FStringNetSerializerConfig SerializerConfig;
};

// These are
const char* FTestStringNetSerializer::UTF8Strings[] = 
{
	// Empty string
	"",
	// Pure ASCII string
	"Just a regular ASCII string",
	// Smiling face with open mouth and tightly-closed eyes, four of circles, euro, copyright
	"\xf0\x9f\x98\x86\xf0\x9f\x80\x9c\xe2\x82\xac\xc2\xa9",
	// A long string consisting of 5000 characters. This is the last valid string to test.
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789"
	"0123456789012345678901234567890123456789012345678901234567890123456789012345678901234567890123456789",
};

const SIZE_T FTestStringNetSerializer::UTF8StringCount = sizeof(UTF8Strings)/sizeof(UTF8Strings[0]);

TArray<FString> FTestStringNetSerializer::WellFormedTestStrings;
TArray<FString> FTestStringNetSerializer::IllFormedTestStrings;

FStringNetSerializerConfig FTestStringNetSerializer::SerializerConfig;

UE_NET_TEST_FIXTURE(FTestStringNetSerializer, TestValidate)
{
	TestValidate();
}

UE_NET_TEST_FIXTURE(FTestStringNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestStringNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestStringNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestStringNetSerializer, TestCloneDynamicState)
{
	TestCloneDynamicState();
}

UE_NET_TEST_FIXTURE(FTestStringNetSerializer, TestIllFormedStringIsEqual)
{
	TestIllFormedStringIsEqual();
}

UE_NET_TEST_FIXTURE(FTestStringNetSerializer, TestIllFormedStringSerialize)
{
	TestIllFormedStringSerialize();
}

void FTestStringNetSerializer::SetUp()
{
	TTestNetSerializerFixture<PrintStringNetSerializerConfig, FString>::SetUp();

	static bool bAreStringsInitialized = false;
	if (!bAreStringsInitialized)
	{
		bAreStringsInitialized = true;

		WellFormedTestStrings.Reserve(UTF8StringCount);
		for (const char* UTF8String : MakeArrayView(UTF8Strings, UTF8StringCount))
		{
			WellFormedTestStrings.Add(FString(FUTF8ToTCHAR(UTF8String).Get()));
		}

		{
			IllFormedTestStrings.Reserve(3);
			IllFormedTestStrings.Add(FString(TEXT("Single low surrogate: '\xD800'.")));
			IllFormedTestStrings.Add(FString(TEXT("Single high surrogate: '\xDC00'.")));
			IllFormedTestStrings.Add(FString(TEXT("Invalid character: '\xFFFF'.")));
		}
	}
}

void FTestStringNetSerializer::TestValidate()
{
	// Check valid strings
	{
		TArray<bool> ExpectedResults;
		ExpectedResults.Init(true, WellFormedTestStrings.Num());

		const bool bSuccess = Super::TestValidate(WellFormedTestStrings.GetData(), ExpectedResults.GetData(), WellFormedTestStrings.Num(), SerializerConfig);
		if (!bSuccess)
		{
			return;
		}
	}
}

void FTestStringNetSerializer::TestQuantize()
{
	const bool bSuccess = Super::TestQuantize(WellFormedTestStrings.GetData(), WellFormedTestStrings.Num(), SerializerConfig);
	if (!bSuccess)
	{
		return;
	}
}

void FTestStringNetSerializer::TestIsEqual()
{
	TArray<FString> CompareValues[2];
	TArray<bool> ExpectedResults[2];

	CompareValues[0] = WellFormedTestStrings;
	ExpectedResults[0].Init(true, WellFormedTestStrings.Num());

	CompareValues[1].Reserve(WellFormedTestStrings.Num());
	ExpectedResults[1].Reserve(WellFormedTestStrings.Num());
	for (int32 ValueIt = 0, ValueEndIt = WellFormedTestStrings.Num(); ValueIt != ValueEndIt; ++ValueIt)
	{
		CompareValues[1].Add(WellFormedTestStrings[(ValueIt + 1) % ValueEndIt]);
		ExpectedResults[1].Add(WellFormedTestStrings[ValueIt].Equals(WellFormedTestStrings[(ValueIt + 1) % ValueEndIt]));
	}

	// Do two rounds of testing per config, one where we compare each value with itself and one where we compare against a value in range.
	for (SIZE_T TestRoundIt : {0, 1})
	{
		// Do both quantized and regular compares
		for (SIZE_T CompareIt : {0, 1})
		{
			bool bQuantizedCompare = CompareIt == 0;
			const bool bSuccess = Super::TestIsEqual(WellFormedTestStrings.GetData(), CompareValues[TestRoundIt].GetData(), ExpectedResults[TestRoundIt].GetData(), WellFormedTestStrings.Num(), SerializerConfig, bQuantizedCompare);
			if (!bSuccess)
			{
				return;
			}
		}
	}
}

void FTestStringNetSerializer::TestSerialize()
{
	constexpr bool bQuantizedCompare = false;
	const bool bSuccess = Super::TestSerialize(WellFormedTestStrings.GetData(), WellFormedTestStrings.GetData(), WellFormedTestStrings.Num(), SerializerConfig, bQuantizedCompare);
	if (!bSuccess)
	{
		return;
	}
}

void FTestStringNetSerializer::TestCloneDynamicState()
{
	const bool bSuccess = Super::TestCloneDynamicState(WellFormedTestStrings.GetData(), WellFormedTestStrings.Num(), SerializerConfig);
	if (!bSuccess)
	{
		return;
	}
}

void FTestStringNetSerializer::TestIllFormedStringIsEqual()
{
	// Do both quantized and regular compares. We expect the strings to be equal to themselves.
	TArray<bool> ExpectedResults;
	ExpectedResults.Init(true, IllFormedTestStrings.Num());
	for (SIZE_T CompareIt : {0, 1})
	{
		const bool bQuantizedCompare = CompareIt == 0;
		const bool bSuccess = Super::TestIsEqual(IllFormedTestStrings.GetData(), IllFormedTestStrings.GetData(), ExpectedResults.GetData(), IllFormedTestStrings.Num(), SerializerConfig, bQuantizedCompare);
		if (!bSuccess)
		{
			return;
		}
	}
}

void FTestStringNetSerializer::TestIllFormedStringSerialize()
{
	// Just make sure nothing blows up when ill-formed strings are replicated.
	// N.B. This test will leak some memory if any of the tests fail.
	for (const FString& SourceString : IllFormedTestStrings)
	{
		alignas(16) uint8 QuantizedStateBuffer[2][128] = {};
		FString TargetString;

		// Quantize
		{
			FNetQuantizeArgs QuantizeArgs;
			QuantizeArgs.Version = SerializerVersionOverride;
			QuantizeArgs.NetSerializerConfig = &SerializerConfig;
			QuantizeArgs.Source = NetSerializerValuePointer(&SourceString);
			QuantizeArgs.Target = NetSerializerValuePointer(QuantizedStateBuffer[0]);
			Serializer.Quantize(Context, QuantizeArgs);
			UE_NET_ASSERT_FALSE_MSG(Context.HasError(), "Quantize failed.");
		}

		// Serialize
		{
			Writer.InitBytes(BitStreamBuffer, sizeof(BitStreamBuffer));
			FNetSerializeArgs SerializeArgs;
			SerializeArgs.Version = SerializerVersionOverride;
			SerializeArgs.NetSerializerConfig = &SerializerConfig;
			SerializeArgs.Source = NetSerializerValuePointer(QuantizedStateBuffer[0]);
			Serializer.Serialize(Context, SerializeArgs);
			Writer.CommitWrites();
			UE_NET_ASSERT_FALSE_MSG(Context.HasError(), "Serialize() reported an error.");
			UE_NET_ASSERT_FALSE_MSG(Writer.IsOverflown(), "FNetBitStreamWriter overflowed.");
		}

		// Deserialize
		{
			Reader.InitBits(BitStreamBuffer, Writer.GetPosBits());
			FNetDeserializeArgs DeserializeArgs;
			DeserializeArgs.Version = SerializerVersionOverride;
			DeserializeArgs.NetSerializerConfig = &SerializerConfig;
			DeserializeArgs.Target = NetSerializerValuePointer(QuantizedStateBuffer[1]);
			Serializer.Deserialize(Context, DeserializeArgs);
			Writer.CommitWrites();
			UE_NET_ASSERT_FALSE_MSG(Context.HasError(), "Deserialize() reported an error.");
			UE_NET_ASSERT_FALSE_MSG(Reader.IsOverflown(), "FNetBitStreamRead overflowed.");
		}

		// Dequantize
		{
			FNetDequantizeArgs DequantizeArgs;
			DequantizeArgs.Version = SerializerVersionOverride;
			DequantizeArgs.NetSerializerConfig = &SerializerConfig;
			DequantizeArgs.Source = NetSerializerValuePointer(QuantizedStateBuffer[1]);
			DequantizeArgs.Target = NetSerializerValuePointer(&TargetString);
			Serializer.Dequantize(Context, DequantizeArgs);
			// Depending on the size of wchar_t an error may be generated or not on certain ill-formed strings.
			//UE_NET_ASSERT_FALSE_MSG(Context.HasError(), "Dequantize() reported an error for string '" << SourceString << '\'');
		}

		// Free memory
		{
			FNetFreeDynamicStateArgs FreeArgs;
			FreeArgs.Version = SerializerVersionOverride;
			FreeArgs.NetSerializerConfig = &SerializerConfig;
			FreeArgs.Source = NetSerializerValuePointer(QuantizedStateBuffer[0]);
			Serializer.FreeDynamicState(Context, FreeArgs);
			FreeArgs.Source = NetSerializerValuePointer(QuantizedStateBuffer[1]);
			Serializer.FreeDynamicState(Context, FreeArgs);
		}
	}
}

}
