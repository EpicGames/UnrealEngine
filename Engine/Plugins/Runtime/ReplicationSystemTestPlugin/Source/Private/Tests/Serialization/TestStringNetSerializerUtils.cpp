// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Iris/Serialization/StringNetSerializerUtils.h"
#include "Containers/StringConv.h"

namespace UE::Net::Private
{

class FTestStringNetSerializerUtils : public FNetworkAutomationTestSuiteFixture
{
public:
	FTestStringNetSerializerUtils() = default;

	void TestEnc32Dec32();
	void TestEnc16Dec16();
	void TestEnc16Dec32();
	void TestEnc32Dec16();

protected:
	static const char32_t TestStringUTF32[];
	static const SIZE_T TestStringUTF32Len;
	static const char16_t TestStringUTF16[];
	static const SIZE_T TestStringUTF16Len;
	// The UTF-8 string is only used to be able to print error messages
	static const char TestStringUTF8[];
};

const char32_t FTestStringNetSerializerUtils::TestStringUTF32[] = U"\x0001F606\x0001F01C\x20AC\xA9";
const SIZE_T FTestStringNetSerializerUtils::TestStringUTF32Len = sizeof(TestStringUTF32)/sizeof(TestStringUTF32[0]);
const char16_t FTestStringNetSerializerUtils::TestStringUTF16[] = u"\xD83D\xDE06\xD83C\xDC1C\x20AC\x00A9";
const SIZE_T FTestStringNetSerializerUtils::TestStringUTF16Len = sizeof(TestStringUTF16)/sizeof(TestStringUTF16[0]);
const char FTestStringNetSerializerUtils::TestStringUTF8[] = "\xf0\x9f\x98\x86\xf0\x9f\x80\x9c\xe2\x82\xac\xc2\xa9";

UE_NET_TEST_FIXTURE(FTestStringNetSerializerUtils, TestEnc32Dec32)
{
	// Encode
	uint32 EncodedBufferLen = FStringNetSerializerUtils::TStringCodec<char32_t>::GetSafeEncodedBufferLength(TestStringUTF32Len);
	uint8* EncodingBuffer = static_cast<uint8*>(FMemory_Alloca(EncodedBufferLen));
	uint32 OutEncodedLen = 0;
	FStringNetSerializerUtils::TStringCodec<char32_t>::Encode(EncodingBuffer, EncodedBufferLen, TestStringUTF32, TestStringUTF32Len, OutEncodedLen);
	UE_NET_ASSERT_GT_MSG(OutEncodedLen, 0U, FString::Printf(TEXT("Test encoding of UTF-32 '%s'."), FUTF8ToTCHAR(TestStringUTF8).Get()));

	// Decode
	uint32 DecodedBufferLen = FStringNetSerializerUtils::TStringCodec<char32_t>::GetSafeDecodedBufferLength(OutEncodedLen);
	char32_t* DecodingBuffer = static_cast<char32_t*>(FMemory_Alloca(DecodedBufferLen * sizeof(char32_t)));
	uint32 OutDecodedLen = 0;
	FStringNetSerializerUtils::TStringCodec<char32_t>::Decode(DecodingBuffer, DecodedBufferLen, EncodingBuffer, OutEncodedLen, OutDecodedLen);
	UE_NET_ASSERT_EQ_MSG(OutDecodedLen, static_cast<uint32>(TestStringUTF32Len), FString::Printf(TEXT("Test UTF-32 decoding of '%s'."), FUTF8ToTCHAR(TestStringUTF8).Get()));
	UE_NET_ASSERT_EQ_MSG(FMemory::Memcmp(DecodingBuffer, TestStringUTF32, TestStringUTF32Len), 0, FString::Printf(TEXT("Test UTF-32 decoding of '%s'."), FUTF8ToTCHAR(TestStringUTF8).Get()));
}

UE_NET_TEST_FIXTURE(FTestStringNetSerializerUtils, TestEnc16Dec16)
{
	// Encode
	uint32 EncodedBufferLen = FStringNetSerializerUtils::TStringCodec<char16_t>::GetSafeEncodedBufferLength(TestStringUTF16Len);
	uint8* EncodingBuffer = static_cast<uint8*>(FMemory_Alloca(EncodedBufferLen));
	uint32 OutEncodedLen = 0;
	FStringNetSerializerUtils::TStringCodec<char16_t>::Encode(EncodingBuffer, EncodedBufferLen, TestStringUTF16, TestStringUTF16Len, OutEncodedLen);
	UE_NET_ASSERT_GT_MSG(OutEncodedLen, 0U, FString::Printf(TEXT("Test encoding of UTF-16 '%s'."), FUTF8ToTCHAR(TestStringUTF8).Get()));

	// Decode
	uint32 DecodedBufferLen = FStringNetSerializerUtils::TStringCodec<char16_t>::GetSafeDecodedBufferLength(OutEncodedLen * sizeof(char16_t));
	char16_t* DecodingBuffer = static_cast<char16_t*>(FMemory_Alloca(DecodedBufferLen));
	uint32 OutDecodedLen = 0;
	FStringNetSerializerUtils::TStringCodec<char16_t>::Decode(DecodingBuffer, DecodedBufferLen, EncodingBuffer, OutEncodedLen, OutDecodedLen);
	UE_NET_ASSERT_EQ_MSG(OutDecodedLen, static_cast<uint32>(TestStringUTF16Len), FString::Printf(TEXT("Test UTF-16 decoding of '%s'."), FUTF8ToTCHAR(TestStringUTF8).Get()));
	UE_NET_ASSERT_EQ_MSG(FMemory::Memcmp(DecodingBuffer, TestStringUTF16, TestStringUTF16Len), 0, FString::Printf(TEXT("Test UTF-16 decoding of '%s'."), FUTF8ToTCHAR(TestStringUTF8).Get()));
}

UE_NET_TEST_FIXTURE(FTestStringNetSerializerUtils, TestEnc16Dec32)
{
	// Encode
	uint32 EncodedBufferLen = FStringNetSerializerUtils::TStringCodec<char16_t>::GetSafeEncodedBufferLength(TestStringUTF16Len);
	uint8* EncodingBuffer = static_cast<uint8*>(FMemory_Alloca(EncodedBufferLen));
	uint32 OutEncodedLen = 0;
	FStringNetSerializerUtils::TStringCodec<char16_t>::Encode(EncodingBuffer, EncodedBufferLen, TestStringUTF16, TestStringUTF16Len, OutEncodedLen);

	// Decode
	uint32 DecodedBufferLen = FStringNetSerializerUtils::TStringCodec<char32_t>::GetSafeDecodedBufferLength(OutEncodedLen);
	char32_t* DecodingBuffer = static_cast<char32_t*>(FMemory_Alloca(DecodedBufferLen * sizeof(char32_t)));
	uint32 OutDecodedLen = 0;
	FStringNetSerializerUtils::TStringCodec<char32_t>::Decode(DecodingBuffer, DecodedBufferLen, EncodingBuffer, OutEncodedLen, OutDecodedLen);
	UE_NET_ASSERT_EQ_MSG(OutDecodedLen, static_cast<uint32>(TestStringUTF32Len), FString::Printf(TEXT("Test UTF-32 decoding of UTF-16 '%s'."), FUTF8ToTCHAR(TestStringUTF8).Get()));
	UE_NET_ASSERT_EQ_MSG(FMemory::Memcmp(DecodingBuffer, TestStringUTF32, TestStringUTF32Len), 0, FString::Printf(TEXT("Test UTF-32 decoding of UTF-16 '%s'."), FUTF8ToTCHAR(TestStringUTF8).Get()));
}

UE_NET_TEST_FIXTURE(FTestStringNetSerializerUtils, TestEnc32Dec16)
{
	// Encode
	uint32 EncodedBufferLen = FStringNetSerializerUtils::TStringCodec<char32_t>::GetSafeEncodedBufferLength(TestStringUTF32Len);
	uint8* EncodingBuffer = static_cast<uint8*>(FMemory_Alloca(EncodedBufferLen));
	uint32 OutEncodedLen = 0;
	FStringNetSerializerUtils::TStringCodec<char32_t>::Encode(EncodingBuffer, EncodedBufferLen, TestStringUTF32, TestStringUTF32Len, OutEncodedLen);

	// Decode
	uint32 DecodedBufferLen = FStringNetSerializerUtils::TStringCodec<char16_t>::GetSafeDecodedBufferLength(OutEncodedLen);
	char16_t* DecodingBuffer = static_cast<char16_t*>(FMemory_Alloca(DecodedBufferLen * sizeof(char16_t)));
	uint32 OutDecodedLen = 0;
	FStringNetSerializerUtils::TStringCodec<char16_t>::Decode(DecodingBuffer, DecodedBufferLen, EncodingBuffer, OutEncodedLen, OutDecodedLen);
	UE_NET_ASSERT_EQ_MSG(OutDecodedLen, static_cast<uint32>(TestStringUTF16Len), FString::Printf(TEXT("Test UTF-16 decoding of UTF-32 '%s'."), FUTF8ToTCHAR(TestStringUTF8).Get()));
	UE_NET_ASSERT_EQ_MSG(FMemory::Memcmp(DecodingBuffer, TestStringUTF16, TestStringUTF16Len), 0, FString::Printf(TEXT("Test UTF-16 decoding of UTF-32 '%s'."), FUTF8ToTCHAR(TestStringUTF8).Get()));
}

}
