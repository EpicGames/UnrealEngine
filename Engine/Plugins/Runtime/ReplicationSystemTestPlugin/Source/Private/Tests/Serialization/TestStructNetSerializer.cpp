// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestStructNetSerializer.h"
#include "TestNetSerializerFixture.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/NetSerializers.h"
#include "MockNetSerializer.h"

namespace UE::Net::Private
{

class FTestStructNetSerializer : public FNetworkAutomationTestSuiteFixture
{
public:
	FTestStructNetSerializer() : FNetworkAutomationTestSuiteFixture() {}

protected:
	virtual void SetUp() override;
	virtual void TearDown() override;

	void TestSerialize();
	void TestDeserialize();
	void TestSerializeDelta();
	void TestDeserializeDelta();
	void TestQuantize();
	void TestDequantize();
	void TestIsEqual();
	void TestValidate();

	constexpr static uint32 StructMemberCount = 3;

	FMockNetSerializerCallCounter MockNetSerializerCallCounter;
	FMockNetSerializerReturnValues MockNetSerializerReturnValues[StructMemberCount];
	FMockNetSerializerConfig SerializerConfigs[StructMemberCount];
	FStructNetSerializerConfig StructNetSerializerConfig;
	const FNetSerializer* StructNetSerializer = &UE_NET_GET_SERIALIZER(FStructNetSerializer);

	FStructForStructNetSerializerTest StructInstance0;
	FStructForStructNetSerializerTest StructInstance1;

private:
	class FNetSerializerRegistryDelegates final : UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
	};

	static FTestStructNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
};

FTestStructNetSerializer::FNetSerializerRegistryDelegates FTestStructNetSerializer::NetSerializerRegistryDelegates;


constexpr uint32 FTestStructNetSerializer::StructMemberCount;

UE_NET_TEST_FIXTURE(FTestStructNetSerializer, TestSerialize)
{
	TestSerialize();
}

UE_NET_TEST_FIXTURE(FTestStructNetSerializer, TestDeserialize)
{
	TestDeserialize();
}

UE_NET_TEST_FIXTURE(FTestStructNetSerializer, TestSerializeDelta)
{
	TestSerializeDelta();
}

UE_NET_TEST_FIXTURE(FTestStructNetSerializer, TestDeserializeDelta)
{
	TestDeserializeDelta();
}

UE_NET_TEST_FIXTURE(FTestStructNetSerializer, TestQuantize)
{
	TestQuantize();
}

UE_NET_TEST_FIXTURE(FTestStructNetSerializer, TestDequantize)
{
	TestDequantize();
}

UE_NET_TEST_FIXTURE(FTestStructNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestStructNetSerializer, TestValidate)
{
	TestValidate();
}

//

void FTestStructNetSerializer::SetUp()
{
	StructNetSerializerConfig.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(StaticStruct<FStructForStructNetSerializerTest>());
	FReplicationStateDescriptor* Descriptor = const_cast<FReplicationStateDescriptor*>(StructNetSerializerConfig.StateDescriptor.GetReference());
	check(Descriptor->MemberCount == StructMemberCount);

	const FNetSerializer& MockNetSerializer = UE_NET_GET_SERIALIZER(FMockNetSerializer);
	for (uint32 It = 0, EndIt = StructMemberCount; It != EndIt; ++It)
	{
		FMockNetSerializerConfig& SerializerConfig = SerializerConfigs[It];
		SerializerConfig.CallCounter = &MockNetSerializerCallCounter;
		SerializerConfig.ReturnValues = &MockNetSerializerReturnValues[It];

		FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = const_cast<FReplicationStateMemberSerializerDescriptor&>(Descriptor->MemberSerializerDescriptors[It]);
		MemberSerializerDescriptor.Serializer = &MockNetSerializer;
		MemberSerializerDescriptor.SerializerConfig = &SerializerConfig;
	}
}

void FTestStructNetSerializer::TearDown()
{
}

void FTestStructNetSerializer::TestSerialize()
{
	FNetBitStreamWriter BitWriter;
	FNetSerializationContext Context(&BitWriter);
	FNetSerializeArgs Args = {};
	Args.Version = StructNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&StructNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(&StructInstance0);
	StructNetSerializer->Serialize(Context, Args);

	UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.Serialize, StructMemberCount);
}

void FTestStructNetSerializer::TestDeserialize()
{
	FNetBitStreamReader BitReader;
	FNetSerializationContext Context(&BitReader);
	FNetDeserializeArgs Args = {};
	Args.Version = StructNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&StructNetSerializerConfig);
	Args.Target = NetSerializerValuePointer(&StructInstance0);
	StructNetSerializer->Deserialize(Context, Args);

	UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.Deserialize, StructMemberCount);
}

void FTestStructNetSerializer::TestSerializeDelta()
{
	FNetBitStreamWriter BitWriter;
	FNetSerializationContext Context(&BitWriter);
	FNetSerializeDeltaArgs Args = {};
	Args.Version = StructNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&StructNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(&StructInstance0);
	Args.Prev = NetSerializerValuePointer(&StructInstance1);
	StructNetSerializer->SerializeDelta(Context, Args);

	UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.SerializeDelta, StructMemberCount);
}

void FTestStructNetSerializer::TestDeserializeDelta()
{
	FNetBitStreamReader BitReader;
	FNetSerializationContext Context(&BitReader);
	FNetDeserializeDeltaArgs Args = {};
	Args.Version = StructNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&StructNetSerializerConfig);
	Args.Target = NetSerializerValuePointer(&StructInstance0);
	Args.Prev = NetSerializerValuePointer(&StructInstance1);
	StructNetSerializer->DeserializeDelta(Context, Args);

	UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.DeserializeDelta, StructMemberCount);
}

void FTestStructNetSerializer::TestQuantize()
{
	FNetSerializationContext Context;
	FNetQuantizeArgs Args = {};
	Args.Version = StructNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&StructNetSerializerConfig);
	Args.Target = NetSerializerValuePointer(&StructInstance0);
	Args.Source = NetSerializerValuePointer(&StructInstance1);
	StructNetSerializer->Quantize(Context, Args);

	UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.Quantize, StructMemberCount);
}

void FTestStructNetSerializer::TestDequantize()
{
	FNetSerializationContext Context;
	FNetDequantizeArgs Args = {};
	Args.Version = StructNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&StructNetSerializerConfig);
	Args.Target = NetSerializerValuePointer(&StructInstance0);
	Args.Source = NetSerializerValuePointer(&StructInstance1);
	StructNetSerializer->Dequantize(Context, Args);

	UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.Dequantize, StructMemberCount);
}

void FTestStructNetSerializer::TestIsEqual()
{
	FNetSerializationContext Context;
	FNetIsEqualArgs Args = {};
	Args.Version = StructNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&StructNetSerializerConfig);
	Args.Source0 = NetSerializerValuePointer(&StructInstance0);
	Args.Source1 = NetSerializerValuePointer(&StructInstance1);
	Args.bStateIsQuantized = false;

	// Check all equal
	{
		for (uint32 It = 0, EndIt = StructMemberCount; It != EndIt; ++It)
		{
			MockNetSerializerReturnValues[It].bIsEqual = true;
		}
		StructNetSerializer->IsEqual(Context, Args);

		UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.IsEqual, StructMemberCount);
	}

	// Check one inequal
	{
		for (uint32 It = 0, EndIt = StructMemberCount; It != EndIt; ++It)
		{
			MockNetSerializerReturnValues[It].bIsEqual = true;
		}
		MockNetSerializerReturnValues[0].bIsEqual = false;
		StructNetSerializer->IsEqual(Context, Args);

		UE_NET_ASSERT_GE(MockNetSerializerCallCounter.IsEqual, 1U);
	}

	// Check another one inequal
	{
		for (uint32 It = 0, EndIt = StructMemberCount; It != EndIt; ++It)
		{
			MockNetSerializerReturnValues[It].bIsEqual = true;
		}
		MockNetSerializerReturnValues[1].bIsEqual = false;
		StructNetSerializer->IsEqual(Context, Args);

		UE_NET_ASSERT_GE(MockNetSerializerCallCounter.IsEqual, 2U); // unlikely that equality is checked in random order so we expect two calls
	}
}

void FTestStructNetSerializer::TestValidate()
{
	FNetSerializationContext Context;
	FNetValidateArgs Args = {};
	Args.Version = StructNetSerializer->Version;
	Args.NetSerializerConfig = NetSerializerConfigParam(&StructNetSerializerConfig);
	Args.Source = NetSerializerValuePointer(&StructInstance0);

	// Check all valid
	{
		for (uint32 It = 0, EndIt = StructMemberCount; It != EndIt; ++It)
		{
			MockNetSerializerReturnValues[It].bValidate = true;
		}
		StructNetSerializer->Validate(Context, Args);

		UE_NET_ASSERT_EQ(MockNetSerializerCallCounter.Validate, StructMemberCount);
	}

	// Check one invalid
	{
		for (uint32 It = 0, EndIt = StructMemberCount; It != EndIt; ++It)
		{
			MockNetSerializerReturnValues[It].bValidate = true;
		}
		MockNetSerializerReturnValues[0].bValidate = false;
		StructNetSerializer->Validate(Context, Args);

		UE_NET_ASSERT_GE(MockNetSerializerCallCounter.Validate, 1U);
	}

	// Check another one invalid
	{
		for (uint32 It = 0, EndIt = StructMemberCount; It != EndIt; ++It)
		{
			MockNetSerializerReturnValues[It].bValidate = true;
		}
		MockNetSerializerReturnValues[1].bValidate = false;
		StructNetSerializer->Validate(Context, Args);

		UE_NET_ASSERT_GE(MockNetSerializerCallCounter.Validate, 2U); // unlikely that equality is checked in random order so we expect two calls
	}
}

static const FName NAME_StructMemberForStructNetSerializerTest(TEXT("StructMemberForStructNetSerializerTest"));
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(NAME_StructMemberForStructNetSerializerTest, FMockNetSerializer);

FTestStructNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(NAME_StructMemberForStructNetSerializerTest);
}

void FTestStructNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(NAME_StructMemberForStructNetSerializerTest);
}

}
