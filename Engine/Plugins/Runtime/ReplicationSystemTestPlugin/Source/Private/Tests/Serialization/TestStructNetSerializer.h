// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/ObjectMacros.h"
#include "TestStructNetSerializer.generated.h"

USTRUCT()
struct FStructMemberForStructNetSerializerTest
{
	GENERATED_BODY()
};

USTRUCT()
struct FStructForStructNetSerializerTest
{
	GENERATED_BODY()

	UPROPERTY()
	uint32 Member0;

	UPROPERTY()
	uint32 Member1;

	UPROPERTY()
	uint32 Member2;
};
