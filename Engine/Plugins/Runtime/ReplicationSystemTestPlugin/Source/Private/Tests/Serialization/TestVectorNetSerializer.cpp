// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestNetSerializerFixture.h"
#include "Iris/Serialization/VectorNetSerializers.h"
#include <limits>

namespace UE::Net::Private
{

static FTestMessage& PrintVectorNetSerializerConfig(FTestMessage& Message, const FNetSerializerConfig& InConfig)
{
	return Message;
}

class FTestVectorNetSerializer : public TTestNetSerializerFixture<PrintVectorNetSerializerConfig, FVector>
{
public:
	FTestVectorNetSerializer() : TTestNetSerializerFixture<PrintVectorNetSerializerConfig, FVector>(UE_NET_GET_SERIALIZER(FVectorNetSerializer)) {}

	void TestIsEqual();
	void TestSerialize();

protected:
	static const FVector Values[];
	static const SIZE_T ValueCount;
};

struct FTestFloatTriplet
{
	FVector::FReal X;
	FVector::FReal Y;
	FVector::FReal Z;
	FVector::FReal Padding;
};

const FTestFloatTriplet BadVectorValues[] =
{
	{-INFINITY, NAN, -NAN, 0.0f},
	{-INFINITY, -INFINITY, -INFINITY, 0.0f},
	{NAN, NAN, NAN, 0.0f},
	{-NAN, -NAN, -NAN, 0.0f}
};

const FVector FTestVectorNetSerializer::Values[] =
{
	FVector(1.f),
	FVector(0.0f, 0.0f, 1.0f),
	FVector(0.0f, 0.0f, -1.0f),
	FVector(1.0f, 0.0f, 0.0f),
	FVector(-1.0f, 0.0f, 0.0f),
	FVector(0.0f, 1.0f, 0.0f),
	FVector(0.0f, -1.0f, 0.0f),
	*(const FVector*)(&BadVectorValues[0]),
	*(const FVector*)(&BadVectorValues[1]),
	*(const FVector*)(&BadVectorValues[2]),
	*(const FVector*)(&BadVectorValues[3]),
	FVector(-0.0f),
	FVector(0.0f),
	FVector(std::numeric_limits<FVector::FReal>::max()),
	FVector(std::numeric_limits<FVector::FReal>::lowest()),
	FVector(47.11f),
	FVector(1.26698556E-23)
};
const SIZE_T FTestVectorNetSerializer::ValueCount = sizeof(Values)/sizeof(Values[0]);

UE_NET_TEST_FIXTURE(FTestVectorNetSerializer, TestIsEqual)
{
	TestIsEqual();
}

UE_NET_TEST_FIXTURE(FTestVectorNetSerializer, TestSerialize)
{
	TestSerialize();
}

void FTestVectorNetSerializer::TestIsEqual()
{
	FVector CompareValues[2][sizeof(Values)/sizeof(Values[0])];
	bool ExpectedResults[2][sizeof(Values)/sizeof(Values[0])];

	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		// We need to memcopy the data as the vectors assignment operator will modify the value in debug builds AND we really want to test NAN etc
		FPlatformMemory::Memcpy(&CompareValues[0][ValueIt], &Values[ValueIt], sizeof(FVector));
		ExpectedResults[0][ValueIt] = true;
	}
	for (SIZE_T ValueIt = 0; ValueIt < ValueCount; ++ValueIt)
	{
		// We need to memcopy the data as the vectors assignment operator will modify the value in debug builds AND we really want to test NAN etc
		FPlatformMemory::Memcpy(&CompareValues[1][ValueIt], &Values[(ValueIt + 1) % ValueCount], sizeof(FVector));
		ExpectedResults[1][ValueIt] = false;
	}

	const FVectorNetSerializerConfig Config;
	for (SIZE_T TestRoundIt = 0, TestRoundEndIt = 2; TestRoundIt != TestRoundEndIt; ++TestRoundIt)
	{
		constexpr bool bQuantizedCompare = false;
		bool bSuccess = TTestNetSerializerFixture<PrintVectorNetSerializerConfig, FVector>::TestIsEqual(Values, CompareValues[TestRoundIt], ExpectedResults[TestRoundIt], ValueCount, Config, bQuantizedCompare);
		if (!bSuccess)
		{
			return;
		}
	}
}

void FTestVectorNetSerializer::TestSerialize()
{
	const FVectorNetSerializerConfig Config;
	constexpr bool bQuantizedCompare = false;
	TTestNetSerializerFixture<PrintVectorNetSerializerConfig, FVector>::TestSerialize(Values, Values, ValueCount, Config, bQuantizedCompare);
}

}
