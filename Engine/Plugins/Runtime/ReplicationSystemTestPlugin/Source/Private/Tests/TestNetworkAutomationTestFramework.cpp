// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAutomationTest.h"
#include "NetworkAutomationTestMacros.h"
#include "Net/Core/PushModel/PushModelMacros.h"
#include "HAL/IConsoleManager.h"

namespace UE::Net::Private
{

UE_NET_TEST(NetworkAutomationTest, TestCanUseTestMacro)
{
}

class FNetworkAutomationTestMacrosFixture : public FNetworkAutomationTestSuiteFixture
{
protected:
	bool bTestCanAccessProtectedMembers;
};

UE_NET_TEST_FIXTURE(FNetworkAutomationTestMacrosFixture, TestCanUseTestFixtureMacro)
{
}

UE_NET_TEST_FIXTURE(FNetworkAutomationTestMacrosFixture, TestCanAccessProtectedMembersInFixture)
{
	bTestCanAccessProtectedMembers = true;
}

// Test assert macros
UE_NET_TEST(NetworkAutomationTest, TestAssertEQ)
{
	UE_NET_ASSERT_EQ(true, true);
	UE_NET_ASSERT_EQ(false, false);
	UE_NET_ASSERT_EQ(-0.0f, +0.0f);
	UE_NET_ASSERT_EQ(751216, 751216);
	UE_NET_ASSERT_EQ(UINT_MAX, UINT_MAX);
}

UE_NET_TEST(NetworkAutomationTest, TestAssertNE)
{
	UE_NET_ASSERT_NE(true, false);
	UE_NET_ASSERT_NE(false, true);
}

UE_NET_TEST(NetworkAutomationTest, TestAssertLT)
{
	UE_NET_ASSERT_LT(-1, 2);
	UE_NET_ASSERT_LT(0x80000000U, ~0U);
	UE_NET_ASSERT_LT(-1.0f, 2.0f);
}

UE_NET_TEST(NetworkAutomationTest, TestAssertLE)
{
	UE_NET_ASSERT_LE(-1, 2);
	UE_NET_ASSERT_LE(0x80000000U, ~0U);
	UE_NET_ASSERT_LE(-1.0f, 2.0f);

	UE_NET_ASSERT_LE(2, 2);
	UE_NET_ASSERT_LE(~0U, ~0U);
	UE_NET_ASSERT_LE(2.0f, 2.0f);
}

UE_NET_TEST(NetworkAutomationTest, TestAssertGT)
{
	UE_NET_ASSERT_GT(2, -1);
	UE_NET_ASSERT_GT(~0U, 0x80000000U);
	UE_NET_ASSERT_GT(2.0f, -1.0f);
}

UE_NET_TEST(NetworkAutomationTest, TestAssertGE)
{
	UE_NET_ASSERT_GE(2, -1);
	UE_NET_ASSERT_GE(~0U, 0x80000000U);
	UE_NET_ASSERT_GE(2.0f, -1.0f);

	UE_NET_ASSERT_GE(2, 2);
	UE_NET_ASSERT_GE(~0U, ~0U);
	UE_NET_ASSERT_GE(2.0f, 2.0f);
}

UE_NET_TEST(NetworkAutomationTest, TestAssertTrue)
{
	UE_NET_ASSERT_TRUE(true);
}

UE_NET_TEST(NetworkAutomationTest, TestAssertFalse)
{
	UE_NET_ASSERT_FALSE(false);
}

static constexpr const TCHAR* ExpectedExpectFailureMessage = TEXT("This expect failure is expected.");

UE_NET_TEST(NetworkAutomationTest, TestExpectEQ)
{
	UE_NET_EXPECT_EQ(true, true);
	UE_NET_EXPECT_EQ(false, false);
	UE_NET_EXPECT_EQ(-0.0f, +0.0f);
	UE_NET_EXPECT_EQ(751216, 751216);
	UE_NET_EXPECT_EQ(UINT_MAX, UINT_MAX);

#if !UE_NET_WITH_LOW_LEVEL_TESTS
	SetSuppressWarningsFromSummary(true);
	UE_NET_EXPECT_EQ_MSG(UINT_MAX, ~UINT_MAX, ExpectedExpectFailureMessage);
#endif
}

UE_NET_TEST(NetworkAutomationTest, TestExpectNE)
{
	UE_NET_EXPECT_NE(true, false);
	UE_NET_EXPECT_NE(false, true);

#if !UE_NET_WITH_LOW_LEVEL_TESTS
	SetSuppressWarningsFromSummary(true);
	UE_NET_EXPECT_NE_MSG(false, false, ExpectedExpectFailureMessage);
#endif
}

UE_NET_TEST(NetworkAutomationTest, TestExpectLT)
{
	UE_NET_EXPECT_LT(-1, 2);
	UE_NET_EXPECT_LT(0x80000000U, ~0U);
	UE_NET_EXPECT_LT(-1.0f, 2.0f);

#if !UE_NET_WITH_LOW_LEVEL_TESTS
	SetSuppressWarningsFromSummary(true);
	UE_NET_EXPECT_LT_MSG(-1.0f, -2.0f, ExpectedExpectFailureMessage);
#endif
}

UE_NET_TEST(NetworkAutomationTest, TestExpectLE)
{
	UE_NET_EXPECT_LE(-1, 2);
	UE_NET_EXPECT_LE(0x80000000U, ~0U);
	UE_NET_EXPECT_LE(-1.0f, 2.0f);

	UE_NET_EXPECT_LE(2, 2);
	UE_NET_EXPECT_LE(~0U, ~0U);
	UE_NET_EXPECT_LE(2.0f, 2.0f);

#if !UE_NET_WITH_LOW_LEVEL_TESTS
	SetSuppressWarningsFromSummary(true);
	UE_NET_EXPECT_LE_MSG(2.0f, 1.0f, ExpectedExpectFailureMessage);
#endif
}

UE_NET_TEST(NetworkAutomationTest, TestExpectGT)
{
	UE_NET_EXPECT_GT(2, -1);
	UE_NET_EXPECT_GT(~0U, 0x80000000U);
	UE_NET_EXPECT_GT(2.0f, -1.0f);

#if !UE_NET_WITH_LOW_LEVEL_TESTS
	SetSuppressWarningsFromSummary(true);
	UE_NET_EXPECT_GT_MSG(2.0f, 3.0f, ExpectedExpectFailureMessage);
#endif
}

UE_NET_TEST(NetworkAutomationTest, TestExpectGE)
{
	UE_NET_EXPECT_GE(2, -1);
	UE_NET_EXPECT_GE(~0U, 0x80000000U);
	UE_NET_EXPECT_GE(2.0f, -1.0f);

	UE_NET_EXPECT_GE(2, 2);
	UE_NET_EXPECT_GE(~0U, ~0U);
	UE_NET_EXPECT_GE(2.0f, 2.0f);

#if !UE_NET_WITH_LOW_LEVEL_TESTS
	SetSuppressWarningsFromSummary(true);
	UE_NET_EXPECT_GE_MSG(2.0f, 3.0f, ExpectedExpectFailureMessage);
#endif
}

UE_NET_TEST(NetworkAutomationTest, TestExpectTrue)
{
	UE_NET_EXPECT_TRUE(true);

#if !UE_NET_WITH_LOW_LEVEL_TESTS
	SetSuppressWarningsFromSummary(true);
	UE_NET_EXPECT_TRUE_MSG(false, ExpectedExpectFailureMessage);
#endif
}

UE_NET_TEST(NetworkAutomationTest, TestExpectFalse)
{
	UE_NET_EXPECT_FALSE(false);

#if !UE_NET_WITH_LOW_LEVEL_TESTS
	SetSuppressWarningsFromSummary(true);
	UE_NET_EXPECT_FALSE_MSG(true, ExpectedExpectFailureMessage);
#endif
}

UE_NET_TEST(NetworkAutomationTest, TestLog)
{
	UE_NET_LOG("Testing log functionality without streaming operator.");
	UE_NET_LOG("Testing log functionality") << " with streaming operator.";
}

UE_NET_TEST(NetworkAutomationTest, TestLogPrimitiveTypes)
{
	UE_NET_LOG("float ") << MAX_flt;
	UE_NET_LOG("double ") << MAX_dbl;

	UE_NET_LOG("bool ") << true;

	UE_NET_LOG("int8 ") << (int8(1) << 6);
	UE_NET_LOG("int16 ") << (int16(1) << 14);
	UE_NET_LOG("int32 ") << (int32(1) << 30);
	UE_NET_LOG("int64 ") << (int64(1) << 62);

	UE_NET_LOG("uint8 ") << (uint8(1) << 7);
	UE_NET_LOG("uint16 ") << (uint16(1) << 15);
	UE_NET_LOG("uint32 ") << (uint32(1) << 31);
	UE_NET_LOG("uint64 ") << (uint64(1) << 63);

	UE_NET_LOG("char ") << char('P');
	UE_NET_LOG("TCHAR ") << TEXT('\x24C5');

	UE_NET_LOG("const char* ") << "a char string";
	UE_NET_LOG("const TCHAR* ") << TEXT("a TCHAR string that will last \x221E");

	UE_NET_LOG("size_t ") << size_t(1);
	
	UE_NET_LOG("uintptr_t ") << uintptr_t(4919);
	
	UE_NET_LOG("ptrdiff_t ") << ptrdiff_t(-1);

	UE_NET_LOG("void* ") << this;
	UE_NET_LOG("nullptr_t ") << nullptr;
}

UE_NET_TEST(NetworkAutomationTest, TestPushModelEnabled)
{
#if !WITH_PUSH_MODEL
	UE_NET_FAIL("Push model support not compiled in.");
#endif

	IConsoleVariable* PushModelCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Net.IsPushModelEnabled"));
	UE_NET_ASSERT_NE(PushModelCVar, nullptr);
	UE_NET_ASSERT_TRUE(PushModelCVar->GetBool());
}

}
