// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"

class FArchive;

/**
 * IPlatformFile implementation for use in tests.
 *
 * Register a non-owned FArchive under a virtual path with AddArchive. OpenRead
 * returns a valid IFileHandle for any registered path. Each handle maintains
 * its own read position by seeking the underlying archive before every read,
 * so multiple handles opened on the same archive work correctly when used
 * serially. Timestamps reported for registered paths are FDateTime::Now().
 *
 * All operations on paths not present in the archive map are forwarded to the
 * lower-level IPlatformFile.
 */
class FUserSuppliedArchivePlatformFile : public IPlatformFile
{
public:
	FUserSuppliedArchivePlatformFile() : LowerLevel(nullptr) {}

	FUserSuppliedArchivePlatformFile(FUserSuppliedArchivePlatformFile&&) = delete;
	FUserSuppliedArchivePlatformFile& operator=(FUserSuppliedArchivePlatformFile&&) = delete;
	FUserSuppliedArchivePlatformFile(const FUserSuppliedArchivePlatformFile&) = delete;
	FUserSuppliedArchivePlatformFile& operator=(const FUserSuppliedArchivePlatformFile&) = delete;

	virtual bool ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const override { return true; }
	virtual bool Initialize(IPlatformFile* Inner, const TCHAR* CmdLine) override { LowerLevel = Inner; return true; }
	virtual IPlatformFile* GetLowerLevel() override { return LowerLevel; }
	virtual void SetLowerLevel(IPlatformFile* NewLowerLevel) override { LowerLevel = NewLowerLevel; }
	virtual const TCHAR* GetName() const override { return TEXT("UserSuppliedArchivePlatformFile"); }

	virtual bool FileExists(const TCHAR* Filename) override;
	virtual int64 FileSize(const TCHAR* Filename) override;
	virtual bool DeleteFile(const TCHAR* Filename) override;
	virtual bool IsReadOnly(const TCHAR* Filename) override;
	virtual bool MoveFile(const TCHAR* To, const TCHAR* From) override { return LowerLevel ? LowerLevel->MoveFile(To, From) : false; }
	virtual bool SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override;
	virtual FDateTime GetTimeStamp(const TCHAR* Filename) override;
	virtual void SetTimeStamp(const TCHAR* Filename, FDateTime DateTime) override;
	virtual FDateTime GetAccessTimeStamp(const TCHAR* Filename) override;
	virtual FString GetFilenameOnDisk(const TCHAR* Filename) override;
	virtual IFileHandle* OpenRead(const TCHAR* Filename, bool bAllowWrite = false) override;
	virtual IFileHandle* OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override { return LowerLevel ? LowerLevel->OpenWrite(Filename, bAppend, bAllowRead) : nullptr; }
	virtual bool DirectoryExists(const TCHAR* Directory) override { return LowerLevel ? LowerLevel->DirectoryExists(Directory) : false; }
	virtual bool CreateDirectory(const TCHAR* Directory) override { return LowerLevel ? LowerLevel->CreateDirectory(Directory) : false; }
	virtual bool DeleteDirectory(const TCHAR* Directory) override { return LowerLevel ? LowerLevel->DeleteDirectory(Directory) : false; }
	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override;
	virtual bool IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor) override { return LowerLevel ? LowerLevel->IterateDirectory(Directory, Visitor) : false; }
	virtual bool IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor) override { return LowerLevel ? LowerLevel->IterateDirectoryStat(Directory, Visitor) : false; }

	/** Register a non-owned FArchive under a virtual file path. */
	void AddArchive(const TCHAR* Filename, FArchive* Archive);

private:
	static FString NormalizePath(const TCHAR* Filename)
	{
		FString Path(Filename);
		FPaths::NormalizeFilename(Path);
		return Path;
	}

	IPlatformFile* LowerLevel;
	TMap<FString, FArchive*> Archives;
};

/**
 * RAII helper that installs an FUserSuppliedArchivePlatformFile at the top of
 * the platform file chain and restores the original chain on destruction.
 *
 * FArchive objects registered via AddArchive are served by OpenRead for the
 * duration of the scope.
 */
class FScopedUserSuppliedArchivePlatformFile
{
public:
	FScopedUserSuppliedArchivePlatformFile()
	{
		IPlatformFile& Current = FPlatformFileManager::Get().GetPlatformFile();
		PlatformFile.Initialize(&Current, TEXT(""));
		FPlatformFileManager::Get().SetPlatformFile(PlatformFile);
	}

	~FScopedUserSuppliedArchivePlatformFile()
	{
		FPlatformFileManager::Get().SetPlatformFile(*PlatformFile.GetLowerLevel());
	}

	void AddArchive(const TCHAR* Filename, FArchive* Archive)
	{
		PlatformFile.AddArchive(Filename, Archive);
	}

private:
	FUserSuppliedArchivePlatformFile PlatformFile;
};
