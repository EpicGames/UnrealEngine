// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class ReplicationSystemTestPlugin : ModuleRules
	{
		public ReplicationSystemTestPlugin(ReadOnlyTargetRules Target) : base(Target)
		{
			// We never want to precompile this plugin
			PrecompileForTargets = PrecompileTargetsType.None;

			PrivateIncludePaths.AddRange(
				new string[]
				{
					Path.Combine(GetModuleDirectory("IrisCore"), "Private"),
				}
				);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"Projects",
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"Engine",
					"NetCore",
				}
				);

			if (Target.IsTestTarget)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"LowLevelTestsRunner",
					}
					);
			}
			else if (Target.Platform != UnrealTargetPlatform.LinuxArm64)
			{
				// For validation of compatibility with low-level tests even when not running them
				PrivateDependencyModuleNames.Add("Catch2");
			}
			else
			{
				PrivateIncludePathModuleNames.Add("Catch2");
			}

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

			PrivateDefinitions.Add(String.Format("UE_NET_WITH_LOW_LEVEL_TESTS={0}", Target.ExplicitTestsTarget ? "1" : "0"));

			SetupIrisSupport(Target);
		}
	}
}
