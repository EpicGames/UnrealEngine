<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">
  <UIVisualizer ServiceId="{A452AFEA-3DF6-46BB-9177-C0B08F318025}" Id="1" MenuName="Add to Image Watch" />
  <Type Name="UE::Net::FNetObjectPrioritizerImage">
    <UIVisualizer ServiceId="{A452AFEA-3DF6-46BB-9177-C0B08F318025}" Id="1" /> 
  </Type>
  <Type Name="UE::Net::FNetObjectPrioritizerImage">
    <Expand>
      <Synthetic Name="[type]"> 
        <DisplayString>UINT8</DisplayString> 
      </Synthetic>
      <Item Name="[channels]">1</Item>
      <Item Name="[width]">ImageWidth</Item>
      <Item Name="[height]">ImageHeight</Item>
      <Item Name="[data]">GreyScaleData.AllocatorInstance.Data</Item>
      <Item Name="[stride]">ImageWidth</Item> 
    </Expand> 
  </Type>   
</AutoVisualizer>
